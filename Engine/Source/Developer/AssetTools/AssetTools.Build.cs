// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AssetTools : ModuleRules
{
	public AssetTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"Merge",
				"UnrealEd",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"Engine",
                "InputCore",
				"ApplicationCore",
				"Slate",
				"SourceControl",
				"PropertyEditor",
				"Kismet",
				"Landscape",
                "Foliage",
                "Projects",
				"RHI",
				"MaterialEditor",
				"ToolMenus",
				"ToolWidgets",
				"PhysicsCore",
				"DeveloperSettings",
				"ClassViewer",
				"EngineSettings",
				"InterchangeCore",
				"InterchangeEngine",
				"PhysicsUtilities",
				"AssetRegistry"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Analytics",
				"ContentBrowser",
				"CollectionManager",
				"DesktopPlatform",
				"EditorWidgets",
				"MainFrame",
				"MessageLog",
				"PackagesDialog",
				"Persona",
                "AnimationEditor",
				"VirtualTexturingEditor"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"ContentBrowser",
				"CollectionManager",
				"CurveTableEditor",
				"DataTableEditor",
				"DesktopPlatform",
				"EditorWidgets",
				"GameProjectGeneration",
                "ActorPickerMode",
				"MainFrame",
				"MessageLog",
				"PackagesDialog",
				"Persona",
				"FontEditor",
                "AudioEditor",
                "SkeletonEditor",
                "SkeletalMeshEditor",
                "AnimationEditor",
                "AnimationBlueprintEditor",
                "AnimationModifiers",
                "VirtualTexturingEditor"
            }
		);

		CircularlyReferencedDependentModules.AddRange(
			new string[] {
				"UnrealEd",
				"Landscape",
				"Kismet",
				"MaterialEditor",
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
