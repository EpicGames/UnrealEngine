// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"
#include "Containers/ContainersFwd.h"
#include "UObject/CoreRedirects.h"
#include "UObject/CoreRedirects/CoreRedirectsContext.h"

#define UE_API ASSETTOOLS_API

/**
 * Delegate called on when patch operation completes
 * @param	SrcFilePath		Path of file being read for patching
 * @param	DstFilePath		Path of file being written to after patching
 */
DECLARE_DELEGATE_TwoParams(FAssetHeaderPatcherCompletionDelegate, const FString& /*SrcFilePath*/, const FString& /*DstFilePath*/)


UE_INTERNAL struct FAssetHeaderPatcher
{
	static const EUnrealEngineObjectUE5Version MinimumSupportedUE5FileVersion = EUnrealEngineObjectUE5Version::ADD_SOFTOBJECTPATH_LIST;

	enum class EResult
	{
		NotStarted,
		Cancelled,
		InProgress,
		Success,
		ErrorFailedToLoadSourceAsset,
		ErrorFailedToDeserializeSourceAsset,
		ErrorUnexpectedSectionOrder,
		ErrorBadOffset,
		ErrorUnkownSection,
		ErrorFailedToOpenDestinationFile,
		ErrorFailedToWriteToDestinationFile,
		ErrorEmptyRequireSection,
	};

	UE_INTERNAL struct FContext
	{
		FContext() = default;

		/**
		 * Context used for patching. Contains all information for how object and package references
		 * will be changed as part of patching. 
		 * *
		 * When bInGatherDependentPackages is true, the provided long package name (/Root/Folder/Package) to 
		 * destination long package name mapping will be used to find any dependent packages that must also 
		 * be patched due to internal references. The mapping provided in InSrcAndDstPackagePaths will be used 
		 * to determine the filepath on disk to write when patching.
		 *
		 * @param	InSrcAndDstPackagePaths Map of all long package names (/Root/Folder/Package) that are patchable 
					and to which new name they should be patched to. Note, this should be all possible remappings
					so that references to unpatched assets may still be corrected when patching assets. If a subset
					of packages should be written to disk during patching, pass that list in InSrcPackagePathsToPatch.
		 * @param	bInGatherDependentPackages If true (default), upon creating the context GatherDependentPackages() will be called.
		 * @param	InSrcPackagePathsToPatch Optional. List of long package names (/Root/Folder/Package) to be patched on disk. 
					This set of package names must be a subset or complete set of the keys in InSrcAndDstPackagePaths. If not
					provided, the keys in InSrcAndDstPackagePaths will be used instead.
		 **/
		UE_API FContext(
			const TMap<FString, FString>& InSrcAndDstPackagePaths, 
			const bool bInGatherDependentPackages = true,
			const TArray<FString>* InSrcPackagePathsToPatch = nullptr);

		/**
		 * Context used for patching. Contains all information for how object and package references
		 * will be changed as part of patching. 
		 * 
		 * When patching, package paths to patch will be deduced by the filepath mappings provided in InSrcAndDstFilePathsForRemapping. All assets
		 * under InSrcRoot will be written as package paths under a mountpoint located at InSrcBaseDir.
		 * 
		 * e.g. Path "C:/User/Repo/Project/Content/Skeletons/Player.uasset" -> "/InSrcRoot/Skeletons/Player" 
				when InSrcBaseDir=C:/User/Repo/Project (/Content is assumed internally)
		 * 
		 * @param	InSrcRoot The root mount point for assets to be patched
		 * @param	InDstRoot The new root mount point for patched assets to be placed under
		 * @param	InSrcBaseDir Path to the directory holding the /Content/ directory for assets to patch
		 * @param	InSrcAndDstFilePathsForRemapping Map of filepaths of all files possible to be patched and where to write the patched 
					version to. Note, this should be all possible remappings so that references to unpatched assets may still 
					be corrected when patching assets. If a subset of files should be written to disk during patching, 
					pass that list in InSrcFilePathsToPatch.
		 * @param	InMountPointReplacements Map of root mountpoints (name only, no "/" prefix or suffix) to replace when patching
		 * @param	InSrcFilePathsToPatch Optional. List of filepaths for the files to be patched on disk. This set of files must be a 
					subset or complete set of the keys in InSrcAndDstFilePathsForRemapping. If not provided the keys in InSrcAndDstFilePathsForRemapping 
					will be used instead.
		 **/
		UE_API FContext(
			const FString& InSrcRoot,
			const FString& InDstRoot,
			const FString& InSrcBaseDir,
			const FString& InDstBaseDir,
			const TMap<FString, FString>& InSrcAndDstFilePathsForRemapping,
			const TMap<FString, FString>& InMountPointReplacements,
			const TArray<FString>* InSrcFilePathsToPatch = nullptr);

		/*
		* Returns the mapping of source long package names to destination package paths used when patching.
		* This mapping may include more packages than initially supplied to the FContext
		* if GatherDependentPackages has already been called. 
		* Note, this map can be invalidated by calls to GatherDependentPackages()
		*/
		const TMap<FString, FString>& GetLongPackagePathRemapping() const
		{
			return PackagePathRenameMap;
		};

	protected:
		friend FAssetHeaderPatcher;

		UE_API void AddVerseMounts();
		UE_API void GatherDependentPackages();
		UE_API void GenerateFilePathsFromPackagePaths(const TArray<FString>* InFilePathsToPatch = nullptr);
		UE_API void GeneratePackagePathsFromFilePaths(const FString& InSrcRoot, const FString& InDstRoot, const FString& InSrcBaseDir, const FString& InDstBaseDir);
		UE_API void GenerateAdditionalRemappings();
	
		TArray<FString> VerseMountPoints;
		// PackagePath remapping information to use while patching
		TMap<FString, FString> PackagePathRenameMap;
		// FilePath remapping information to use while patching
		TMap<FString, FString> FilePathRenameMap;
		// Map of Src -> Dst for files on disk to patch
		TMap<FString, FString> FilePathsToPatchMap;

		// Todo: Make TSet once FCoreRedirect GetTypeHash is implemented
		TArray<FCoreRedirect> Redirects;
		mutable FCoreRedirectsContext RedirectsContext;

		// String mappings are only used for best-effort replacements. These will be error-prone 
		// and we should strive for more structured data formats to guard against errors here
		TMap<FString, FString> VerseStringReplacements;	
		TMap<FString, FString> StringMountReplacements;
	};

	FAssetHeaderPatcher() = default;
	FAssetHeaderPatcher(const FContext& InContext) { SetContext(InContext); }
	
	/*
	* Resets the patcher state and sets a new patching context.
	* It is an error to call while patching is already in progress.
	*/
	UE_API void SetContext(const FContext& InContext);

	/*
	* PatchAsync optional parameters
	*/
	struct FPatchAsyncParams
	{
		/** Optional value to know how many files are expected to be read/written during patching */
		int32* OutNumFilesToPatch;

		/** Optional value used to know how the patcher is progressing (useful for progress bars) */
		int32* OutNumFilesPatched;

		/** Thread-safe delegate invoked for each successful file copy */
		FAssetHeaderPatcherCompletionDelegate OnSuccess;

		/** Thread-safe delegate invoked for each failed file copy */
		FAssetHeaderPatcherCompletionDelegate OnError;

		/** Task priority */
		UE::Tasks::ETaskPriority TaskPriority;

		FPatchAsyncParams()
			: OutNumFilesToPatch(nullptr)
			, OutNumFilesPatched(nullptr)
			, TaskPriority(UE::Tasks::ETaskPriority::Default)
		{}
	};

	/*
	* Schedules the reading of source files determined by the patcher context, as well as the writing of the patched versions
	* of all source files read.
	*/
	UE_API UE::Tasks::FTask PatchAsync(const FPatchAsyncParams& Params = {});

	/*
	* Returns the status of any inflight patching operations. In the case of multiple errors, the last seen error will be reported.
	* Per file error status codes can be returned with GetErrorFiles().
	*/
	EResult GetPatchResult() const
	{
		return Status;
	}

	/*
	* Returns source file -> destination mapping for all files that were patched successfully.
	*/
	TMap<FString, FString> GetPatchedFiles() const
	{
		if (IsPatching())
		{
			return TMap<FString, FString>();
		}

		return PatchedFiles;
	}

	/*
	* Returns true if the patcher encountered errors (even if patching was cancelled)
	*/
	bool HasErrors()
	{
		FScopeLock Lock(&ErroredFilesLock);
		return !!ErroredFiles.Num();
	}

	/*
	* Returns a map of all files that had an error during patching with an error code to provide context as to the cause of the error.
	*/
	TMap<FString, EResult> GetErrorFiles()
	{
		FScopeLock Lock(&ErroredFilesLock);
		return ErroredFiles;
	}

	/*
	* Returns true if the patcher is still in theprocess of patching.
	*/
	bool IsPatching() const 
	{ 
		return !PatchingTask.IsCompleted(); 
	}

	/*
	* Cancels an in-flight patching operation. Patching work on individual files that has already started will run to completion
	* however any files that have not started patching will be skipped. Even after cancelling, one must wait for the patcher to complete
	* by waiting on the GetPatchingTask() explciitly or until IsPatching returns false.
	* 
	* @return true if an in-flight patching operation was cancelled. If no patching operation is underway, returns false.
	*/
	bool CancelPatching() 
	{
		if (!IsPatching())
		{
			return false;
		}

		bCancelled = true;
		Status = EResult::Cancelled;

		return true;
	}

	/*
	* Returns the task for all patcher work underway. Waiting on this task will guarantee all patch work is completed.
	*/
	UE::Tasks::FTask GetPatchingTask() const
	{
		return PatchingTask;
	}

	/**
	 * Patches object and package references contained within InSrcAsset using the mapping provided to InContext. The 
	 * patched asset will be written to InDstAsset. 
	 *
	 * @param InSrcAsset Long package name (/Root/Folder/Package) to read in to be patched.
	 * @param InDstAsset Long package name (/Root/Folder/Package) where the patched package will be written to.
	 * @param InContext Context for how the patching will be performed. Contains all remapping information to the patcher.
	 * @return Success patching was successful and the InDstAsset package was written. Returns an error status otherwise.
	 **/
	static UE_API EResult DoPatch(const FString& InSrcAsset, const FString& InDstAsset, const FContext& InContext);

private:
	UE_API void Reset();

	FContext Context;

	TMap<FString, EResult> ErroredFiles;
	FCriticalSection  ErroredFilesLock;

	TMap<FString, FString> PatchedFiles;

	UE::Tasks::FTask PatchingTask;
	EResult Status = EResult::NotStarted;
	std::atomic<bool> bCancelled = false;
};
UE_INTERNAL ASSETTOOLS_API FString LexToString(FAssetHeaderPatcher::EResult InResult);

#undef UE_API
