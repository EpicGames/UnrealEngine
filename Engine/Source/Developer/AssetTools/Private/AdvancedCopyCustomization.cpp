// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedCopyCustomization.h"
#include "Containers/UnrealString.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IPluginManager.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AdvancedCopyCustomization)


#define LOCTEXT_NAMESPACE "AdvancedCopyCustomization"


UAdvancedCopyCustomization::UAdvancedCopyCustomization(const class FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShouldGenerateRelativePaths(true)
{
	FilterForExcludingDependencies.PackagePaths.Add(TEXT("/Engine"));
	for (TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		if (Plugin->GetType() != EPluginType::Project)
		{
			FilterForExcludingDependencies.PackagePaths.Add(FName(*("/" + Plugin->GetName())));
		}
	}

	FilterForExcludingDependencies.bRecursivePaths = true;
	FilterForExcludingDependencies.bRecursiveClasses = true;
	FilterForExcludingDependencies.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
	FilterForExcludingDependencies.ClassPaths.Add(ULevel::StaticClass()->GetClassPathName());
	FilterForExcludingDependencies.ClassPaths.Add(UMapBuildDataRegistry::StaticClass()->GetClassPathName());

}

void UAdvancedCopyCustomization::SetPackageThatInitiatedCopy(const FString& InBasePackage)
{
	FString TempPackage = InBasePackage;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> DependencyAssetData;
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.GetAssetsByPackageName(FName(*InBasePackage), DependencyAssetData);
	// We found a folder
	if (DependencyAssetData.Num() == 0)
	{
		// Take off the name of the folder we copied so copied files are still nested
		TempPackage.Split(TEXT("/"), &TempPackage, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	}
	
	if (!TempPackage.EndsWith(TEXT("/")))
	{
		TempPackage += TEXT("/");
	}
	PackageThatInitiatedCopy = TempPackage;
}

#undef LOCTEXT_NAMESPACE

