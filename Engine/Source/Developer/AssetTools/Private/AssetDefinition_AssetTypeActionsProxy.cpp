// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AssetTypeActionsProxy.h"

#include "IAssetTools.h"
#include "Misc/AssetFilterData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_AssetTypeActionsProxy)

#define LOCTEXT_NAMESPACE "UAssetDefinition_AssetTypeActionsProxy"

void UAssetDefinition_AssetTypeActionsProxy::Initialize(const TSharedRef<IAssetTypeActions>& NewActions)
{
	AssetType = NewActions;
}

FText UAssetDefinition_AssetTypeActionsProxy::GetAssetDisplayName() const
{
	return AssetType->GetName();
}

FText UAssetDefinition_AssetTypeActionsProxy::GetAssetDisplayName(const FAssetData& AssetData) const
{
	return AssetType->GetDisplayNameFromAssetData(AssetData);
}

FText UAssetDefinition_AssetTypeActionsProxy::GetAssetDescription(const FAssetData& AssetData) const
{
	return AssetType->GetAssetDescription(AssetData);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AssetTypeActionsProxy::GetAssetCategories() const
{
	if (bAreAssetCategoriesInitialized)
	{
		return AssetCategories;
	}

	bAreAssetCategoriesInitialized = true;

	IAssetTools& AssetTools = IAssetTools::Get();
	
	TArray<FAdvancedAssetCategory> AdvancedAssetCategories;
    AssetTools.GetAllAdvancedAssetCategories(AdvancedAssetCategories);
	
	const uint32 CategoryBits = AssetType->GetCategories();

	if (CategoryBits & EAssetTypeCategories::Basic)
	{
		AssetCategories.Add(FAssetCategoryPath({ EAssetCategoryPaths::Basic }));
	}
	
	for (const FAdvancedAssetCategory& Category : AdvancedAssetCategories)
	{
		if (Category.CategoryType & CategoryBits)
		{
			TArray<FText> CategoryTextArray;
			CategoryTextArray.Add(Category.CategoryName);
			for (const FText& Submenu : AssetType->GetSubMenus())
			{
				CategoryTextArray.Add(Submenu);
			}
		
			AssetCategories.Add(FAssetCategoryPath(CategoryTextArray));
		}
	}

	return AssetCategories;
}

TSoftClassPtr<UObject> UAssetDefinition_AssetTypeActionsProxy::GetAssetClass() const
{
	if (UClass* SupportedClass = AssetType->GetSupportedClass())
	{
		return TSoftClassPtr<UObject>(SupportedClass);
	}

	return TSoftClassPtr<UObject>(FSoftObjectPath(AssetType->GetClassPathName()));
}

FLinearColor UAssetDefinition_AssetTypeActionsProxy::GetAssetColor() const
{
	return FLinearColor(AssetType->GetTypeColor());
}

FAssetSupportResponse UAssetDefinition_AssetTypeActionsProxy::CanRename(const FAssetData& InAsset) const
{
    FText OutErrorMsg;
	if (AssetType->CanRename(InAsset, &OutErrorMsg))
	{
	     return FAssetSupportResponse::Supported();
	}
	else
	{
	    return OutErrorMsg.IsEmpty() ? FAssetSupportResponse::NotSupported() : FAssetSupportResponse::Error(OutErrorMsg);
	}
}

FAssetSupportResponse UAssetDefinition_AssetTypeActionsProxy::CanDuplicate(const FAssetData& InAsset) const
{
    FText OutErrorMsg;
	if (AssetType->CanDuplicate(InAsset, &OutErrorMsg))
	{
	     return FAssetSupportResponse::Supported();
	}
	else
	{
	    return OutErrorMsg.IsEmpty() ? FAssetSupportResponse::NotSupported() : FAssetSupportResponse::Error(OutErrorMsg);
	}
}

FAssetSupportResponse UAssetDefinition_AssetTypeActionsProxy::CanLocalize(const FAssetData& InAsset) const
{
	return AssetType->CanLocalize() ? FAssetSupportResponse::Supported() : FAssetSupportResponse::NotSupported();
}

bool UAssetDefinition_AssetTypeActionsProxy::CanImport() const
{
	return AssetType->IsImportedAsset();
}

bool UAssetDefinition_AssetTypeActionsProxy::CanMerge() const
{
	return AssetType->CanMerge();
}

EAssetCommandResult UAssetDefinition_AssetTypeActionsProxy::Merge(const FAssetAutomaticMergeArgs& MergeArgs) const
{
	AssetType->Merge(MergeArgs.LocalAsset);

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_AssetTypeActionsProxy::Merge(const FAssetManualMergeArgs& MergeArgs) const
{
	FOnAssetMergeResolved MergeCallback = MergeArgs.ResolutionCallback;
	AssetType->Merge(MergeArgs.BaseAsset, MergeArgs.RemoteAsset, MergeArgs.LocalAsset,
		FOnMergeResolved::CreateLambda([MergeCallback](UPackage* MergedPackage, EMergeResult::Type Result){
		FAssetMergeResults Results;
		Results.MergedPackage = MergedPackage;
		Results.Result = static_cast<EAssetMergeResult>(Result);
		MergeCallback.ExecuteIfBound(Results);
	}));

	return EAssetCommandResult::Handled;
}

FAssetOpenSupport UAssetDefinition_AssetTypeActionsProxy::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	const bool bIsSupported = AssetType->SupportsOpenedMethod(static_cast<EAssetTypeActivationOpenedMethod>(OpenSupportArgs.OpenMethod));

	FAssetOpenSupport Support(OpenSupportArgs.OpenMethod, bIsSupported);

	if (AssetType->ShouldForceWorldCentric())
	{
		Support.RequiredToolkitMode = EToolkitMode::WorldCentric;
	}
	
	return Support;
}

TArray<FAssetData> UAssetDefinition_AssetTypeActionsProxy::PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	return AssetType->GetValidAssetsForPreviewOrEdit(ActivateArgs.Assets, ActivateArgs.ActivationMethod == EAssetActivationMethod::Previewed);
}

EAssetCommandResult UAssetDefinition_AssetTypeActionsProxy::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	EAssetTypeActivationMethod::Type ActivationType = EAssetTypeActivationMethod::Opened;
	switch(ActivateArgs.ActivationMethod)
	{
	case EAssetActivationMethod::DoubleClicked:
		ActivationType = EAssetTypeActivationMethod::DoubleClicked;
		break;
	case EAssetActivationMethod::Opened:
		ActivationType = EAssetTypeActivationMethod::Opened;
		break;
	case EAssetActivationMethod::Previewed:
		ActivationType = EAssetTypeActivationMethod::Previewed;
		break;
	}

	const TArray<UObject*> Assets = ActivateArgs.LoadObjects<UObject>();
	const bool bResult = AssetType->AssetsActivatedOverride(Assets, ActivationType);
	return bResult ? EAssetCommandResult::Handled : EAssetCommandResult::Unhandled;
}

EAssetCommandResult UAssetDefinition_AssetTypeActionsProxy::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	AssetType->OpenAssetEditor(OpenArgs.LoadObjects<UObject>(), static_cast<EAssetTypeActivationOpenedMethod>(OpenArgs.OpenMethod), OpenArgs.ToolkitHost);
	return EAssetCommandResult::Handled;
}

void UAssetDefinition_AssetTypeActionsProxy::BuildFilters(TArray<FAssetFilterData>& OutFilters) const
{
	if (AssetType->CanFilter())
	{
		// If this asset definition doesn't have any categories it can't have any filters.  Filters need to have a
		// category to be displayed.
		if (GetAssetCategories().Num() == 0)
		{
			return;
		}
		
		FAssetFilterData Data;
		Data.Name = AssetType->GetFilterName().ToString();
		Data.DisplayText = AssetType->GetName();
		Data.FilterCategories = GetAssetCategories();
		AssetType->BuildBackendFilter(Data.Filter);
		OutFilters.Add(MoveTemp(Data));
	}
}

FText UAssetDefinition_AssetTypeActionsProxy::GetObjectDisplayNameText(UObject* Object) const
{
	return FText::FromString(AssetType->GetObjectDisplayName(Object));
}

EAssetCommandResult UAssetDefinition_AssetTypeActionsProxy::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	AssetType->PerformAssetDiff(DiffArgs.OldAsset, DiffArgs.NewAsset, DiffArgs.OldRevision, DiffArgs.NewRevision);
	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_AssetTypeActionsProxy::LoadThumbnailInfo(const FAssetData& Asset) const
{
	return AssetType->GetThumbnailInfo(Asset.GetAsset());
}

const FSlateBrush* UAssetDefinition_AssetTypeActionsProxy::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return AssetType->GetThumbnailBrush(InAssetData, InClassName);
}

const FSlateBrush* UAssetDefinition_AssetTypeActionsProxy::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return AssetType->GetIconBrush(InAssetData, InClassName);
}

TSharedPtr<SWidget> UAssetDefinition_AssetTypeActionsProxy::GetThumbnailOverlay(const FAssetData& InAssetData) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return AssetType->GetThumbnailOverlay(InAssetData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetDefinition_AssetTypeActionsProxy::GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
