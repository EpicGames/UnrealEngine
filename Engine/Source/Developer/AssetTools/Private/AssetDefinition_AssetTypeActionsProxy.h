// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAssetTypeActions.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_AssetTypeActionsProxy.generated.h"

UCLASS(transient)
class UAssetDefinition_AssetTypeActionsProxy : public UAssetDefinitionDefault
{
	GENERATED_BODY() 
public:
	UAssetDefinition_AssetTypeActionsProxy() { }
	
	void Initialize(const TSharedRef<IAssetTypeActions>& NewActions);
	
	const TSharedPtr<IAssetTypeActions>& GetAssetType() const { return AssetType; }

	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDisplayName(const FAssetData& InAssetData) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual FText GetAssetDescription(const FAssetData& InAssetData) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	virtual FAssetSupportResponse CanRename(const FAssetData& InAsset) const override;
	virtual FAssetSupportResponse CanDuplicate(const FAssetData& InAsset) const override;
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override;
	virtual bool CanImport() const override;
	virtual bool CanMerge() const override;
	virtual EAssetCommandResult Merge(const FAssetAutomaticMergeArgs& MergeArgs) const override;
	virtual EAssetCommandResult Merge(const FAssetManualMergeArgs& MergeArgs) const override;

	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual TArray<FAssetData> PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

	virtual void BuildFilters(TArray<FAssetFilterData>& OutFilters) const override;

	// We don't need to implement GetSourceFiles, the default one will / should work for any existing type.
	// virtual EAssetCommandResult GetSourceFiles(const FAssetSourceFileArgs& SourceFileArgs, TArray<FAssetSourceFile>& OutSourceAssets) const override;

	virtual FText GetObjectDisplayNameText(UObject* Object) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& Asset) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& InAssetData) const override;
	virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const override;

protected:
	virtual bool CanRegisterStatically() const override { return false; }
	
private:
	TSharedPtr<IAssetTypeActions> AssetType;
	
	mutable bool bAreAssetCategoriesInitialized = false;
	mutable TArray<FAssetCategoryPath> AssetCategories;
};
