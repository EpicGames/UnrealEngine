// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetFixUpRedirectors.h"
#include "Algo/Sort.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRenameManager.h"
#include "AssetTools.h"
#include "CollectionManagerModule.h"
#include "Dialogs/Dialogs.h"
#include "Engine/Blueprint.h"
#include "FileHelpers.h"
#include "HAL/FileManager.h"
#include "ICollectionManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "SDiscoveringAssetsDialog.h"
#include "Settings/EditorProjectSettings.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectHash.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "AssetFixUpRedirectors"

struct FRedirectorRefs
{
	TStrongObjectPtr<const UObjectRedirector> Redirector;
	FName RedirectorPackageName;
	TArray<FName> ReferencingPackageNames;

	// Referencing packages which could not be updated to remove references to this redirector
	TArray<FName> LockedReferencerPackageNames;
	TArray<FName> FailedReferencerPackageNames;

	// Possible failure reason unrelated to above lists of package names.
	// If this is not empty we may not delete the redirector - we may still be able to remove some references to it.
	TArray<FText> OtherFailures;

	bool bSCCError = false; // Failure to check the redirector itself out of source control etc

	explicit FRedirectorRefs(FName PackageName)
		: Redirector(nullptr)
		, RedirectorPackageName(PackageName)
	{
	}

	explicit FRedirectorRefs(const UObjectRedirector* InRedirector)
		: Redirector(InRedirector)
		, RedirectorPackageName(InRedirector->GetOutermost()->GetFName())
	{
	}
};

// What the user chooses to do from the report dialog
enum class EFixupReportDecision
{
	None,
	DeleteUnreferencedRedirectors
};

enum class EFixupReportItemType
{
	Redirector,             // A redirector which has remaining referencers
	UnreferencedRedirector, // A redirector which has no remaining referencers
	LockedRedirector,       // A redirector that could not be checked out of source control
	LockedReferencer,       // A referencer which could not be checked out of source control
	FailedReferencer,       // A referencer which could not be saved for some reason
	Message,                // Another reason this redirector could not be deleted
};

// An item presented in the report dialog
struct FFixupReportItem
{
	FText Text;
	EFixupReportItemType Type;
	TArray<TSharedPtr<FFixupReportItem>> Children;

	explicit FFixupReportItem(EFixupReportItemType InType, FText InText)
		: Text(MoveTemp(InText))
		, Type(InType)
	{
	}

	const FSlateBrush* GetIcon() const
	{
		switch (Type)
		{
			case EFixupReportItemType::Redirector:
				return FAppStyle::Get().GetBrush("Icons.WarningWithColor");
			case EFixupReportItemType::UnreferencedRedirector:
				return FAppStyle::Get().GetBrush("Icons.SuccessWithColor");
			case EFixupReportItemType::LockedRedirector:
				// Share icon type for both kinds of locked items, through
			case EFixupReportItemType::LockedReferencer:
				return FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Locked");
			case EFixupReportItemType::FailedReferencer:
				return FAppStyle::Get().GetBrush("Icons.ErrorWithColor");
			case EFixupReportItemType::Message:
				return FAppStyle::Get().GetBrush("Icons.ErrorWithColor");
			default:
				checkf(false, TEXT("Unhandled EFixupReportItemType"));
				return nullptr;
		}
	}

	FText GetName() const
	{
		switch (Type)
		{
			default:
				// For redirectors and referencers, the Text is the asset name/path
				return Text;
			case EFixupReportItemType::Message:
				// For messages the Text is the error message so show 'Other'
				return LOCTEXT("Other", "Other");
		}
	}

	FText GetDescription() const
	{
		switch (Type)
		{
			case EFixupReportItemType::Redirector:
				return FText::Format(LOCTEXT("RedirectorIssueCount", "{0} issues"), Children.Num());
			case EFixupReportItemType::UnreferencedRedirector:
				return LOCTEXT("UnreferencedRedirector", "All references to this redirector have been removed.");
			case EFixupReportItemType::LockedRedirector:
				return LOCTEXT("LockedRedirector", "Redirector could not be checked out or marked for delete.");
			case EFixupReportItemType::LockedReferencer:
				return LOCTEXT("LockedReferencer", "Referencer could not be checked out of source control.");
			case EFixupReportItemType::FailedReferencer:
				return LOCTEXT("FailedReferencer", "Referencer could not be saved successfully.");
			case EFixupReportItemType::Message:
				return Text;
			default:
				checkf(false, TEXT("Unhandled EFixupReportItemType"));
				return FText::GetEmpty();
		}
	}
};

namespace FixupRedirectorsReport
{
const FName ColumnID_Expander("Expander");
const FName ColumnID_Icon("Icon");
const FName ColumnID_Name("Name");
const FName ColumnID_Description("Description");
} // namespace FixupRedirectorsReport

class SFixupRedirectorsReportTableRow : public SMultiColumnTableRow<TSharedPtr<FFixupReportItem>>
{
public:
	SLATE_BEGIN_ARGS(SFixupRedirectorsReportTableRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FFixupReportItem> InItem)
	{
		Item = InItem;
		SMultiColumnTableRow<TSharedPtr<FFixupReportItem>>::Construct(FSuperRowType::FArguments().Padding(0.0f), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (ColumnName == FixupRedirectorsReport::ColumnID_Expander)
		{
			return SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this))
				.StyleSet(ExpanderStyleSet)
				.ShouldDrawWires(true);
		}
		else if (ColumnName == FixupRedirectorsReport::ColumnID_Icon)
		{
			return SNew(SImage)
				.Image(Item->GetIcon());
		}
		else if (ColumnName == FixupRedirectorsReport::ColumnID_Name)
		{
			// clang-format off
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(Item->GetName())
				];
			// clang-format on
		}
		else if (ColumnName == FixupRedirectorsReport::ColumnID_Description)
		{
			return SNew(STextBlock)
				.Text(Item->GetDescription());
		}
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FFixupReportItem> Item;
};

class SFixupRedirectorsReport : public SModalEditorDialog<bool>
{
public:
	SLATE_BEGIN_ARGS(SFixupRedirectorsReport)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TArray<FRedirectorRefs>& RedirectorRefsList, bool InCanDelete)
	{
		bCanDelete = InCanDelete;
		Sizing = ESizingRule::UserSized;
		MinDimensions = FVector2D(1000.0f, 600.f);

		const UAssetToolsImpl& AssetTools = UAssetToolsImpl::Get();

		bool bAnyRedirectorsToDelete = false;
		for (const FRedirectorRefs& Redirector : RedirectorRefsList)
		{
			TSharedPtr<FFixupReportItem>& NewItem = TreeItems.Emplace_GetRef(MakeShared<FFixupReportItem>(
				EFixupReportItemType::Redirector,
				FText::FromString(Redirector.Redirector ? AssetTools.GetUserFacingPathName(*Redirector.Redirector) : AssetTools.GetUserFacingLongPackageName(Redirector.RedirectorPackageName))));
			{
				TArray<FString> LockedReferencers = AssetTools.GetUserFacingLongPackageNames(Redirector.LockedReferencerPackageNames);
				LockedReferencers.Sort();
				for (FString& LockedReferencer : LockedReferencers)
				{
					NewItem->Children.Emplace(MakeShared<FFixupReportItem>(
						EFixupReportItemType::LockedReferencer,
						FText::FromString(MoveTemp(LockedReferencer))));
				}
			}
			{
				TArray<FString> FailedReferencers = AssetTools.GetUserFacingLongPackageNames(Redirector.FailedReferencerPackageNames);
				FailedReferencers.Sort();
				for (FString& FailedReferencer : FailedReferencers)
				{
					NewItem->Children.Emplace(MakeShared<FFixupReportItem>(
						EFixupReportItemType::FailedReferencer,
						FText::FromString(MoveTemp(FailedReferencer))));
				}
			}
			for (FText Issue : Redirector.OtherFailures)
			{
				NewItem->Children.Emplace(MakeShared<FFixupReportItem>(
					EFixupReportItemType::Message,
					Issue));
			}

			if (Redirector.bSCCError)
			{
				NewItem->Type = EFixupReportItemType::LockedRedirector;
				bAnyProblems = true;
			}
			else if (NewItem->Children.IsEmpty())
			{
				NewItem->Type = EFixupReportItemType::UnreferencedRedirector;
				bAnyRedirectorsToDelete = true;
			}
			else
			{
				bAnyProblems = true;
			}
		}

		if (bCanDelete && bAnyRedirectorsToDelete)
		{
			MinDimensions.Y = 800.0f; // Slightly bigger default size to account for help text
		}

		TreeItems.Sort([](const TSharedPtr<FFixupReportItem>& A, const TSharedPtr<FFixupReportItem>& B)
		{
			// Sort nodes with problems first.
			const bool bAHasProblems = A->Type != EFixupReportItemType::UnreferencedRedirector;
			const bool bBHasProblems = B->Type != EFixupReportItemType::UnreferencedRedirector;
			if (bAHasProblems != bBHasProblems)
			{
				return bAHasProblems;
			}
			return A->Text.ToString() < B->Text.ToString();
		});

		TSharedRef<SHeaderRow> HeaderRowWidget =
			SNew(SHeaderRow)
			+ SHeaderRow::Column(FixupRedirectorsReport::ColumnID_Icon)
				  .DefaultLabel(FText::GetEmpty())
				  .HAlignHeader(EHorizontalAlignment::HAlign_Left)
				  .FixedWidth(20.0f)
				  .HAlignCell(HAlign_Center)
			+ SHeaderRow::Column(FixupRedirectorsReport::ColumnID_Expander)
				  .DefaultLabel(FText::GetEmpty())
				  .HAlignHeader(EHorizontalAlignment::HAlign_Left)
				  .FixedWidth(16.0f)
			+ SHeaderRow::Column(FixupRedirectorsReport::ColumnID_Name)
				  .DefaultLabel(LOCTEXT("Column_Name", "Name"))
				  .HAlignHeader(EHorizontalAlignment::HAlign_Left)
				  .FillWidth(1.0f)
			+ SHeaderRow::Column(FixupRedirectorsReport::ColumnID_Description)
				  .DefaultLabel(FText::GetEmpty())
				  .HAlignHeader(EHorizontalAlignment::HAlign_Left)
				  .FillWidth(1.0f);

		// clang-format off
		TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0,0,0,10)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
						.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
						[
							SNew(STextBlock)
							.Text(this, &SFixupRedirectorsReport::GetIntroText)
							.Font(FAppStyle::GetFontStyle("BoldFont"))
							.ShadowOffset(FVector2D(1.0f, 1.0f))
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SAssignNew(TreeView, STreeView<TSharedPtr<FFixupReportItem>>)
						.HeaderRow(HeaderRowWidget)
						.TreeItemsSource(&TreeItems)
						.SelectionMode(ESelectionMode::None)
						.OnGenerateRow(this, &SFixupRedirectorsReport::HandleGenerateTreeViewRow)
						.OnGetChildren(this, &SFixupRedirectorsReport::HandleGetChildren)
					]	
				]
			];
		if (bCanDelete && bAnyRedirectorsToDelete)
		{
			// Large bottom row with scary delete buttons and long explanations
			VerticalBox->AddSlot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Bottom)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NoBorder"))
							[
								SNew(STextBlock)
								.Text(this, &SFixupRedirectorsReport::GetDeleteExplanationText)
								.AutoWrapText(true)
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NoBorder"))
							.VAlign(VAlign_Bottom)
							[
								SNew(SButton)
								.HAlign(HAlign_Center)
								.Text(LOCTEXT("DeleteRedirectorsButtonText", "Delete Unreferenced Redirectors"))
								.ToolTipText(LOCTEXT("DeleteTooltipText", "Delete redirectors which are no longer referenced"))
								.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
								.OnClicked(this, &SFixupRedirectorsReport::CloseDeleteRedirectors)
							]
						]
					]

					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0, 10, 0, 0)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NoBorder"))
							// Only show the help text for keeping if deletion is also an option
							[
								SNew(STextBlock)
								.Text(this, &SFixupRedirectorsReport::GetKeepExplanationText)
								.AutoWrapText(true)
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NoBorder"))
							.VAlign(VAlign_Bottom)
							[
								SAssignNew(KeepButton, SButton)
								.HAlign(HAlign_Center)
								.Text(this, &SFixupRedirectorsReport::GetKeepRedirectorsButtonText)
								.ToolTipText(this, &SFixupRedirectorsReport::GetKeepRedirectorsButtonToolTipText)
								.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
								.OnClicked(this, &SFixupRedirectorsReport::CloseKeepRedirectors)
							]
						]
					]
				];
		}
		else
		{
			// Simpler bottom row with just an "Ok" box
			VerticalBox->AddSlot()
				.AutoHeight()
				.Padding(0, 10, 0, 0)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.VAlign(VAlign_Bottom)
					.HAlign(HAlign_Center)
					[
						SAssignNew(KeepButton, SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("Ok", "Ok"))
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
						.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
						.OnClicked(this, &SFixupRedirectorsReport::CloseKeepRedirectors)
					]
				];
		}

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("AssetDeleteDialog.Background"))
			.Padding(10.0f)
			[
				VerticalBox.ToSharedRef()	
			]
		];
		// clang-format on 
	}

private:
	FReply CloseKeepRedirectors()
	{
		ProvideResult(false);
		return FReply::Handled();
	}
	FReply CloseDeleteRedirectors()
	{
		ProvideResult(true);
		return FReply::Handled();
	}
	
	virtual TSharedPtr<SWidget> GetWidgetToFocusOnActivate() override
	{
		return KeepButton;
	}
	
	FText GetIntroText() const
	{
		if (bAnyProblems)
		{
			return LOCTEXT("ReportIntro_SomeProblems", "Some references to redirectors could not be updated and still remain.");
		}
		else 
		{
			return LOCTEXT("ReportIntro_Problems", "All references to redirectors were successfully updated.");
		}
	}
	
	FText GetDeleteExplanationText() const
	{
		return LOCTEXT("RedirectorDeleteExplanation", 
			"Delete all redirectors which are no longer referenced locally.\n\n"
			"Use with caution as these redirectors may be referenced"
			" - for example in other source control branches or in other users' workspaces." 
			);
	}
	
	FText GetKeepExplanationText() const
	{
		return LOCTEXT("RedirectorKeepExplanation", 
			"Keep all redirectors, both referenced and unreferenced.\n\n"
			"This prevents issues caused by references to them in other workspaces or source control branches.");
	}

	FText GetKeepRedirectorsButtonText() const
	{
		return bCanDelete ? LOCTEXT("KeepRedirectors", "Keep Redirectors") : LOCTEXT("Ok", "Ok");
	}

	FText GetKeepRedirectorsButtonToolTipText() const
	{
		return bCanDelete ? LOCTEXT("KeepRedirectorsToolTipText", "Do not delete any redirectors") : FText::GetEmpty();
	}

	TSharedRef<ITableRow> HandleGenerateTreeViewRow(TSharedPtr<FFixupReportItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		return SNew(SFixupRedirectorsReportTableRow, InOwnerTable, InItem);
	}

	void HandleGetChildren(TSharedPtr<FFixupReportItem> InItem, TArray<TSharedPtr<FFixupReportItem>>& OutChildren)
	{
		OutChildren = InItem->Children;
	}

	TSharedPtr<SButton> KeepButton;
	TSharedPtr<STreeView<TSharedPtr<FFixupReportItem>>> TreeView;
	TArray<TSharedPtr<FFixupReportItem>> TreeItems;
	bool bCanDelete; // Whether we offer the option to delete redirectors 
	bool bAnyProblems; // Whether there were any redirectors that couldn't be fully fixed up
};

void FAssetFixUpRedirectors::FixupReferencers(const TArray<UObjectRedirector*>& Objects, const bool bCheckoutDialogPrompt, ERedirectFixupMode FixupMode) const
{
	// Transform array into TWeakObjectPtr array
	TArray<TWeakObjectPtr<UObjectRedirector>> ObjectWeakPtrs;
	for (auto Object : Objects)
	{
		ObjectWeakPtrs.Add(Object);
	}

	if (ObjectWeakPtrs.Num() > 0)
	{
		// If the asset registry is still loading assets, we cant check for referencers, so we must open the Discovering Assets dialog until it is done
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			// Open a dialog asking the user to wait while assets are being discovered
			SDiscoveringAssetsDialog::OpenDiscoveringAssetsDialog(
				SDiscoveringAssetsDialog::FOnAssetsDiscovered::CreateSP(this, &FAssetFixUpRedirectors::ExecuteFixUp, ObjectWeakPtrs, bCheckoutDialogPrompt, FixupMode)
				);
		}
		else
		{
			// No need to wait, attempt to fix references now.
			ExecuteFixUp(ObjectWeakPtrs, bCheckoutDialogPrompt, FixupMode);
		}
	}
}

void FAssetFixUpRedirectors::ExecuteFixUp(TArray<TWeakObjectPtr<UObjectRedirector>> Objects, const bool bCheckoutDialogPrompt, ERedirectFixupMode FixupMode) const
{
	TGuardValue<bool> Guard(bIsFixupReferencersInProgress, true);

	TArray<FRedirectorRefs> RedirectorRefsList;
	for (TWeakObjectPtr<UObjectRedirector> Object : Objects)
	{
		if (UObjectRedirector* ObjectRedirector = Object.Get())
		{
			RedirectorRefsList.Emplace(ObjectRedirector);
		}
	}

	if (RedirectorRefsList.Num() == 0)
	{
		return;
	}
	
	// Check if we can delete redirectors - if so we need to perform source control operations on them
	bool bMayDeleteRedirectors = false;
	switch (FixupMode)
	{
		case ERedirectFixupMode::DeleteFixedUpRedirectors:
			bMayDeleteRedirectors = true;
			break;
		case ERedirectFixupMode::PromptForDeletingRedirectors:
			const UEditorProjectAssetSettings* Settings = GetDefault<UEditorProjectAssetSettings>();
			bMayDeleteRedirectors = Settings && Settings->bPromptToDeleteUnreferencedRedirectors;
			break;	
	}

	// Gather all referencing packages for all redirectors that are being fixed.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	bool bAnyRefs = false;
	for (FRedirectorRefs& RedirectorRefs : RedirectorRefsList)
	{
		AssetRegistryModule.Get().GetReferencers(RedirectorRefs.RedirectorPackageName, RedirectorRefs.ReferencingPackageNames);
		bAnyRefs = bAnyRefs || RedirectorRefs.ReferencingPackageNames.Num() != 0;
	}
	
	if (!bAnyRefs && !bMayDeleteRedirectors)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoPackagesToSave", "No referencing assets found to be resaved."));
		return;
	}
	
	// Build back-reference lookup for later
	TMultiMap<FName, FRedirectorRefs*> ReferencingAssetToRedirector;
	for (FRedirectorRefs& RedirectorRefs : RedirectorRefsList)
	{
		for (FName PackageName : RedirectorRefs.ReferencingPackageNames)
		{
			ReferencingAssetToRedirector.Add(PackageName, &RedirectorRefs);	
		}
	}

	// Update Package Status for all selected redirectors if SCC is enabled and we may delete
	if (bMayDeleteRedirectors && ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		// Update the source control server availability to make sure we can do the rename operation
		if (!SourceControlProvider.IsAvailable())
		{
			if (SourceControlProvider.Login() != ECommandResult::Succeeded)
			{
				// We have failed to connect to source control even though it is enabled. This is critical and we can not continue
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "SourceControl_ServerUnresponsive", "Revision Control is unresponsive. Please check your connection and try again.") );
				return;
			}
		}

		TArray<UPackage*> PackagesToAddToSCCUpdate;
		for (const FRedirectorRefs& RedirectorRefs : RedirectorRefsList)
		{
			PackagesToAddToSCCUpdate.Add(RedirectorRefs.Redirector->GetOutermost());
		}

		if (SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), PackagesToAddToSCCUpdate) != ECommandResult::Succeeded)
		{
			// We have failed to update source control even though it is enabled. This is critical and we can not continue
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "SourceControl_UpdateFailed", "Revision Control update failed. Please check your connection and try again.") );
			return;
		}
	}

	// Load all referencing packages.
	TSet<UPackage*> ReferencingPackagesToSave;
	TSet<UPackage*> LoadedPackages;
	bool bCancel = false;
	{
		FScopedSlowTask SlowTask(static_cast<float>(RedirectorRefsList.Num()), LOCTEXT( "LoadingReferencingPackages", "Loading Referencing Packages..." ) );
		SlowTask.MakeDialog(true);
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		// Load all packages that reference each redirector, if possible
		for (FRedirectorRefs& RedirectorRefs : RedirectorRefsList)
		{
			SlowTask.EnterProgressFrame(1);
			if (SlowTask.ShouldCancel())
			{
				bCancel = true;
				break;
			}
			if (bMayDeleteRedirectors && ISourceControlModule::Get().IsEnabled())
			{
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(RedirectorRefs.Redirector->GetOutermost(), EStateCacheUsage::Use);
				const bool bValidSCCState = !SourceControlState.IsValid() || SourceControlState->IsAdded() || SourceControlState->IsCheckedOut() || SourceControlState->CanCheckout() || !SourceControlState->IsSourceControlled() || SourceControlState->IsIgnored();

				if (!bValidSCCState)
				{
					RedirectorRefs.bSCCError = true;
					// Continue to load the referencers because we may still be able to fix them up 
				}
			}

			// Load all referencers
			for (FName ReferencingPackageName : RedirectorRefs.ReferencingPackageNames)
			{
				FNameBuilder PackageName{ReferencingPackageName};

				// Find the package in memory. If it is not in memory, try to load it
				UPackage* Package = FindPackage(nullptr, *PackageName);
				if (!Package)
				{
					Package = LoadPackage(nullptr, *PackageName, LOAD_None);
					if (Package)
					{
						LoadedPackages.Add(Package);
					}
				}

				if (Package)
				{
					if (Package->HasAnyPackageFlags(PKG_CompiledIn))
					{
						// This is a script reference
						RedirectorRefs.OtherFailures.Add(FText::Format(LOCTEXT("RedirectorFixupFailed_CodeReference", "Redirector is referenced by code. Package: {0}"), FText::FromName(ReferencingPackageName)));
					}
					else
					{
						// If we found a valid package, mark it for save
						ReferencingPackagesToSave.Add(Package);
					}
				}
			}
		}
	}

	ON_SCOPE_EXIT {
		// If any packages were loaded during the fixup process, make sure we unload them here
		if (!LoadedPackages.IsEmpty())
		{
			FText ErrorMessage;
			UPackageTools::UnloadPackages(LoadedPackages.Array(), ErrorMessage, true);
			if (!ErrorMessage.IsEmpty())
			{
				FTextBuilder Builder;
				Builder.AppendLine(LOCTEXT("ErrorsUnloadingPackages", "Errors were encountered unloading packages which were loaded to update redirector references. Some assets may still be loaded. "));
				Builder.AppendLine();
				Builder.AppendLine(ErrorMessage);
				FMessageDialog::Open(EAppMsgType::Ok, Builder.ToText());
			}
		}
	};
	
	if (bCancel)
	{
		return;
	}

	// Add all referencing packages objects that aren't RF_Standalone to the root set to avoid them being GC'd during the following processing
	TArray<TStrongObjectPtr<UObject>> RootedObjects;
	for (UPackage* Package : ReferencingPackagesToSave)
	{
		ForEachObjectWithPackage(Package, [&RootedObjects](UObject* Object)
		{
			RootedObjects.Emplace(Object);
			return true;
		}, EGetObjectsFlags::None, RF_Standalone, EInternalObjectFlags::RootSet);
	}

	// Reset loaders of assets used in level instances to allow referencing packages to be saved
	TSet<FName> WorldAssetsNeedingLoadersReset;
	for (UPackage* Package : ReferencingPackagesToSave)
	{
		TArray<FAssetData> ReferencingPackageAssets;
		if (AssetRegistryModule.Get().GetAssetsByPackageName(Package->GetFName(), ReferencingPackageAssets, /*bIncludeOnlyOnDiskAssets=*/true))
		{
			for (const FAssetData& Asset : ReferencingPackageAssets)
			{
				if (!Asset.GetOptionalOuterPathName().IsNone())
				{
					WorldAssetsNeedingLoadersReset.Add(FSoftObjectPath(Asset.GetOptionalOuterPathName().ToString()).GetLongPackageFName());
				}
			}
		}
	}
	for (const FName& WorldAsset : WorldAssetsNeedingLoadersReset)
	{
		ULevelInstanceSubsystem::ResetLoadersForWorldAsset(*WorldAsset.ToString());
	}
	
	// Check out all referencing packages, leave redirectors for assets referenced by packages that are not checked out and remove those packages from the save list.
	bool bUserAcceptedCheckout = true; // If source control is disabled, assume checkout was selected
	if (ISourceControlModule::Get().IsEnabled() && ReferencingPackagesToSave.Num() > 0)
	{
		TArray<UPackage*> PackagesCheckedOutOrMadeWritable;
		TArray<UPackage*> PackagesNotNeedingCheckout;
		if (bCheckoutDialogPrompt)
		{
			bUserAcceptedCheckout = FEditorFileUtils::PromptToCheckoutPackages(false, ReferencingPackagesToSave.Array(), &PackagesCheckedOutOrMadeWritable, &PackagesNotNeedingCheckout);
		}
		else
		{
			const bool bErrorIfAlreadyCheckedOut = false;
			const bool bConfirmPackageBranchCheckOutStatus = false;
			FEditorFileUtils::CheckoutPackages(ReferencingPackagesToSave.Array(), &PackagesCheckedOutOrMadeWritable, bErrorIfAlreadyCheckedOut, bConfirmPackageBranchCheckOutStatus);
		}

		if (bUserAcceptedCheckout)
		{
			TSet<UPackage*> PackagesThatCouldNotBeCheckedOut = ReferencingPackagesToSave;
			for (UPackage* Package : PackagesCheckedOutOrMadeWritable)
			{
				PackagesThatCouldNotBeCheckedOut.Remove(Package);
			}
			for (UPackage* Package : PackagesNotNeedingCheckout)
			{
				PackagesThatCouldNotBeCheckedOut.Remove(Package);
			}

			for (UPackage* Package : PackagesThatCouldNotBeCheckedOut)
			{
				FName PackageName = Package->GetFName(); // Key iterator requires we copy this

				// Note which redirectors will still be required because this package could not be checked out 
				for (auto It = ReferencingAssetToRedirector.CreateKeyIterator(PackageName); It; ++It)
				{
					It.Value()->LockedReferencerPackageNames.Add(Package->GetFName());
				}

				// Don't save anything that wasn't checked out 
				ReferencingPackagesToSave.Remove(Package);
			}
		}
	}

	if (bUserAcceptedCheckout)
	{
		// If any referencing packages are left read-only, the checkout failed or SCC was not enabled. Trim them from the save list and leave redirectors.
		for (auto It = ReferencingPackagesToSave.CreateIterator(); It; ++It)
		{
			UPackage* Package = *It;
			if (!ensure(Package))
			{
				It.RemoveCurrent();
				continue;
			}
			
			// If the file is read only
			FString Filename;
			if (FPackageName::DoesPackageExist(Package->GetName(), &Filename) 
			&& IFileManager::Get().IsReadOnly(*Filename))
			{
				// Note which redirectors will still be required because this package was read only
				FName PackageName = Package->GetFName(); // Key iterator requires we copy this
				for (auto RedirectorIt = ReferencingAssetToRedirector.CreateKeyIterator(PackageName); RedirectorIt; ++RedirectorIt)
				{
					RedirectorIt.Value()->LockedReferencerPackageNames.Add(Package->GetFName());
				}

				// Remove the package from the save list
				It.RemoveCurrent();
			}
		}

		// Fix up referencing FSoftObjectPaths
		{
			TSet<UPackage*> PackagesToCheck = ReferencingPackagesToSave;

			TArray<UPackage*> Tmp;
			FEditorFileUtils::GetDirtyWorldPackages(Tmp);
			FEditorFileUtils::GetDirtyContentPackages(Tmp);
			PackagesToCheck.Append(Tmp);

			TMap<FSoftObjectPath, FSoftObjectPath> RedirectorMap;
			for (const FRedirectorRefs& RedirectorRef : RedirectorRefsList)
			{
				const UObjectRedirector* Redirector = RedirectorRef.Redirector.Get();
				FSoftObjectPath OldPath = FSoftObjectPath(Redirector);
				FSoftObjectPath NewPath = FSoftObjectPath(Redirector->DestinationObject);

				RedirectorMap.Add(OldPath, NewPath);
				if (UBlueprint* Blueprint = Cast<UBlueprint>(Redirector->DestinationObject))
				{
					// Add redirect for class and default as well
					RedirectorMap.Add(FString::Printf(TEXT("%s_C"), *OldPath.ToString()), FString::Printf(TEXT("%s_C"), *NewPath.ToString()));
					RedirectorMap.Add(FString::Printf(TEXT("%s.Default__%s_C"), *OldPath.GetLongPackageName(), *OldPath.GetAssetName()), FString::Printf(TEXT("%s.Default__%s_C"), *NewPath.GetLongPackageName(), *NewPath.GetAssetName()));
				}
			}

			UAssetToolsImpl::Get().AssetRenameManager->RenameReferencingSoftObjectPaths(PackagesToCheck.Array(), RedirectorMap);
		}

		// Save all packages that were referencing any of the assets that were moved without redirectors
		TArray<UPackage*> FailedToSave;
		if (ReferencingPackagesToSave.Num() > 0)
		{
			// Get the list of filenames before calling save because some of the saved packages can get GCed if they are empty packages
			const TArray<FString> Filenames = USourceControlHelpers::PackageFilenames(ReferencingPackagesToSave.Array());

			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			FEditorFileUtils::PromptForCheckoutAndSave(ReferencingPackagesToSave.Array(), bCheckDirty, bPromptToSave, &FailedToSave);
			for (UPackage* Package : FailedToSave)
			{
				FName PackageName = Package->GetFName(); // Key iterator requires we copy this
				for (auto It = ReferencingAssetToRedirector.CreateKeyIterator(PackageName); It; ++It)	
				{
					It.Value()->FailedReferencerPackageNames.Add(Package->GetFName());
				}
			}

			ISourceControlModule::Get().QueueStatusUpdate(Filenames);
		}

		// Save any collections that were referencing any of the redirectors
		FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

		// Find all collections that were referenced by any of the redirectors that are potentially going to be removed and attempt to re-save them
		// The redirectors themselves will have already been fixed up, as collections do that once the asset registry has been populated, 
		// however collections lazily re-save redirector fix-up to avoid SCC issues, so we need to force that now
		for (FRedirectorRefs& RedirectorRefs : RedirectorRefsList)
		{
			// Follow each link in the redirector, and notify the collections manager that it is going to be removed - this will force it to re-save any required collections
			for (const UObjectRedirector* Redirector = RedirectorRefs.Redirector.Get(); Redirector; Redirector = Cast<UObjectRedirector>(Redirector->DestinationObject))
			{
				const FSoftObjectPath RedirectorObjectPath = FSoftObjectPath(Redirector);
				FText Error;
				if (!CollectionManagerModule.Get().HandleRedirectorDeleted(RedirectorObjectPath, &Error))
				{
					RedirectorRefs.OtherFailures.Add(FText::Format(LOCTEXT("RedirectorFixupFailed_CollectionsFailedToSave", "Referencing collection(s) failed to save: {0}"), Error));
				}
			}
		}

		// Wait for package referencers to be updated
		// Load the asset registry module
		{
			TArray<FString> AssetPaths;
			for (const FRedirectorRefs& Redirector : RedirectorRefsList)
			{
				AssetPaths.AddUnique(FPackageName::GetLongPackagePath(Redirector.RedirectorPackageName.ToString()) / TEXT("")); // Ensure trailing slash

				for (const auto& Referencer : Redirector.ReferencingPackageNames)
				{
					AssetPaths.AddUnique(FPackageName::GetLongPackagePath(Referencer.ToString()) / TEXT("")); // Ensure trailing slash
				}
			}
			AssetRegistryModule.Get().ScanPathsSynchronous(AssetPaths, true);
		}

		// Show user report and check whether we should delete
		bool bCanDelete = false;
		switch (FixupMode)
		{
			case ERedirectFixupMode::DeleteFixedUpRedirectors:
				bCanDelete = true;
				break;
			case ERedirectFixupMode::PromptForDeletingRedirectors:
				const UEditorProjectAssetSettings* Settings = GetDefault<UEditorProjectAssetSettings>();
				bCanDelete = Settings && Settings->bPromptToDeleteUnreferencedRedirectors;
				break;
		}
		

		TSharedPtr<SFixupRedirectorsReport> Dialog = SNew(SFixupRedirectorsReport, RedirectorRefsList, bCanDelete);
		const bool bDoDelete = Dialog->ShowModalDialog(LOCTEXT("RedirectorUpdateReport", "Redirector Update Report"));
		if (!bDoDelete)
		{
			return;
		}

		TArray<UObject*> ObjectsToDelete;
		for (FRedirectorRefs& RedirectorRefs : RedirectorRefsList)
		{
			if (RedirectorRefs.OtherFailures.IsEmpty() 
			&&  RedirectorRefs.LockedReferencerPackageNames.IsEmpty()
			&& 	RedirectorRefs.FailedReferencerPackageNames.IsEmpty())
			{
				ensure(RedirectorRefs.Redirector);
				// Add all redirectors found in this package to the redirectors to delete list.
				// All redirectors in this package should be fixed up.
				UPackage* RedirectorPackage = RedirectorRefs.Redirector->GetOutermost();
				bool bContainsAtLeastOneOtherAsset = false;
				ForEachObjectWithOuter(RedirectorPackage, [&ObjectsToDelete, &bContainsAtLeastOneOtherAsset](UObject* Obj)
				{
					if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Obj))
					{
						Redirector->RemoveFromRoot();
						ObjectsToDelete.Add(Redirector);
					}
					else
					{
						bContainsAtLeastOneOtherAsset = true;
					}
				});

				if ( !bContainsAtLeastOneOtherAsset )
				{
					RedirectorPackage->RemoveFromRoot();
					ObjectsToDelete.Add(RedirectorPackage);
				}
			}
		}
		
		// Release all redirector references before executing delete operation
		RedirectorRefsList.Empty();

		if ( ObjectsToDelete.Num() > 0 )
		{
			ObjectTools::DeleteObjects(ObjectsToDelete, false);
		}
	}
}

#undef LOCTEXT_NAMESPACE
