// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRedirectorRefs;
enum class ERedirectFixupMode;

class FAssetFixUpRedirectors : public TSharedFromThis<FAssetFixUpRedirectors>
{
public:
	/**
	 * Fix up references to the specified redirectors.
	 * @param bCheckoutDialogPrompt indicates whether to prompt the user with files checkout dialog or silently attempt to checkout all necessary files.
	 */
	void FixupReferencers(const TArray<UObjectRedirector*>& Objects, bool bCheckoutDialogPrompt, ERedirectFixupMode FixupMode) const;

	/** Returns whether redirectors are being fixed up. */
	bool IsFixupReferencersInProgress() const { return bIsFixupReferencersInProgress; }

private:

	/** The core code of the fixup operation */
	void ExecuteFixUp(TArray<TWeakObjectPtr<UObjectRedirector>> Objects, bool bCheckoutDialogPrompt, ERedirectFixupMode FixupMode) const;

private:

	mutable bool bIsFixupReferencersInProgress = false;
};
