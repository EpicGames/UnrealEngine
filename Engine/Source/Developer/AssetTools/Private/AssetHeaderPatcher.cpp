// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetHeaderPatcher.h"

#include "Algo/Copy.h"
#include "Algo/Sort.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/PackageReader.h"
#include "Containers/ContainersFwd.h"
#include "Internationalization/GatherableTextData.h"
#include "Misc/Base64.h"
#include "Misc/EnumerateRange.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/RedirectCollector.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/ImportTypeHierarchy.h"
#include "UObject/CoreRedirects.h"
#include "UObject/CoreRedirects/CoreRedirectsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/Linker.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectResource.h"
#include "UObject/Package.h"
#include "UObject/PackageFileSummary.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogAssetHeaderPatcher, Log, All);

#define DEBUG_ASSET_HEADER_PATCHING 0

namespace
{
	// If working on header patching, this is very helpful for dumping what is patched and reviewing the files in a folder comparison of your favourite diff program.
	FString DumpOutputDirectory;
	static FAutoConsoleVariableRef CVarDumpOutputDirectory(
		TEXT("AssetHeaderPatcher.DebugDumpDir"),
		DumpOutputDirectory,
		TEXT("'Before'/'After' text representations of each package processed during patching will be written out to the provided absolute filesystem path. Useful for comparing what was patched.")
	);

	// Tag 'Key' names that are generally large blobs of data that can't/shouldn't be patched
	const TCHAR* TagsToIgnore[] =
	{
		TEXT("FiBData")
	};

	const FStringView InvalidObjectPathCharacters(INVALID_OBJECTPATH_CHARACTERS);

	bool SplitLongPackageName(FStringView LongPackageName, FStringView& PackageRoot, FStringView& PackagePath, FStringView& PackageName)
	{
		if (LongPackageName.IsEmpty() || LongPackageName[0] != TEXT('/'))
		{
			return false;
		}

		PackageRoot = FStringView(LongPackageName.GetData() + 1); // + 1 to skip the leading '/'
		int32 SeparatorPos;
		if (!PackageRoot.FindChar(TEXT('/'), SeparatorPos))
		{
			return false;
		}
		PackageRoot.LeftInline(SeparatorPos);

		const int32 PackagePathOffset = PackageRoot.Len() + 2; // + 2 for the leading and trailing '/'
		if (LongPackageName.Len() < PackagePathOffset || !LongPackageName.FindLastChar(TEXT('/'), SeparatorPos))
		{
			return false;
		}

		// May be empty. If the PackageName is off the root there is no PackagePath
		const int32 PackagePathLen = SeparatorPos - (PackagePathOffset - 1);
		check(PackagePathLen >= 0);
		PackagePath = FStringView(LongPackageName.GetData() + PackagePathOffset, PackagePathLen - !!PackagePathLen);

		const int32 PackageNameOffset = PackagePathOffset + PackagePath.Len() + !PackagePath.IsEmpty();
		PackageName = FStringView(LongPackageName.GetData() + PackageNameOffset, LongPackageName.Len() - PackageNameOffset);

		return true;
	}

	FStringView Find(const TMap<FString, FString>& Table, FStringView Needle)
	{
		uint32 NeedleHash = TMap<FString, FString>::KeyFuncsType::GetKeyHash<FStringView>(Needle);
		const FString* MaybeNewItem = Table.FindByHash<FStringView>(NeedleHash, Needle);
		if (MaybeNewItem)
		{
			return *MaybeNewItem;
		}

		return {};
	}
}

FString LexToString(FAssetHeaderPatcher::EResult InResult)
{
	switch (InResult)
	{
		case FAssetHeaderPatcher::EResult::NotStarted: return TEXT("Not Started");
		case FAssetHeaderPatcher::EResult::Cancelled: return TEXT("Cancelled");
		case FAssetHeaderPatcher::EResult::InProgress: return TEXT("In Progress");
		case FAssetHeaderPatcher::EResult::Success: return TEXT("Success");
		case FAssetHeaderPatcher::EResult::ErrorFailedToLoadSourceAsset: return TEXT("Failed to load source asset");
		case FAssetHeaderPatcher::EResult::ErrorFailedToDeserializeSourceAsset: return TEXT("Failed to deserialize source asset");
		case FAssetHeaderPatcher::EResult::ErrorUnexpectedSectionOrder: return TEXT("Unexpected section order");
		case FAssetHeaderPatcher::EResult::ErrorBadOffset: return TEXT("Bad offset");
		case FAssetHeaderPatcher::EResult::ErrorUnkownSection: return TEXT("Unknown section");
		case FAssetHeaderPatcher::EResult::ErrorFailedToOpenDestinationFile: return TEXT("Failed to open destination file");
		case FAssetHeaderPatcher::EResult::ErrorFailedToWriteToDestinationFile: return TEXT("Failed to write to destination file");
		case FAssetHeaderPatcher::EResult::ErrorEmptyRequireSection: return TEXT("Empty required section");
		default: return TEXT("Unknown");
	}
}

FAssetHeaderPatcher::FContext::FContext(const TMap<FString, FString>& SourceAndDestPackages, const bool bInGatherDependentPackages, const TArray<FString>* InSrcPackagePathsToPatch)
	: PackagePathRenameMap(SourceAndDestPackages)
{
	AddVerseMounts();

	if (bInGatherDependentPackages)
	{
		GatherDependentPackages();
	}

	GenerateFilePathsFromPackagePaths(InSrcPackagePathsToPatch);
	GenerateAdditionalRemappings();
}

FAssetHeaderPatcher::FContext::FContext(
	const FString& InSrcRoot,
	const FString& InDstRoot,
	const FString& InSrcBaseDir,
	const FString& InDstBaseDir,
	const TMap<FString, FString>& InSrcAndDstFilePaths,
	const TMap<FString, FString>& InMountPointReplacements,
	const TArray<FString>* InSrcFilePathsToPatch)
	: FilePathRenameMap(InSrcAndDstFilePaths)
	, StringMountReplacements(InMountPointReplacements)
{

	// If we have a patch file list, verify we have mappings for the passed in files and update our FilePathsToPatchMap.
	// Otherwise use the complete mapping information from InSrcAndDstFilePaths.
	if (!InSrcFilePathsToPatch)
	{
		FilePathsToPatchMap = InSrcAndDstFilePaths;
	}
	else
	{
		FilePathsToPatchMap.Reserve(InSrcFilePathsToPatch->Num());
		for (const FString& SrcFile : *InSrcFilePathsToPatch)
		{
			const FString* DstFile = InSrcAndDstFilePaths.Find(SrcFile);
			if (!DstFile)
			{
				UE_LOGF(LogAssetHeaderPatcher, Warning, "No remapping for file %ls was provided when patching. This file will not be patched.", *SrcFile);
			}
			else
			{
				FilePathsToPatchMap.Emplace(SrcFile, *DstFile);
			}
		}
	}

	AddVerseMounts();
	GeneratePackagePathsFromFilePaths(InSrcRoot, InDstRoot, InSrcBaseDir, InDstBaseDir);
	GenerateAdditionalRemappings();
}

void FAssetHeaderPatcher::FContext::AddVerseMounts()
{
	// Todo: Expose this so callers provide this data
	VerseMountPoints.Add(TEXT("localhost"));
}

void FAssetHeaderPatcher::FContext::GenerateFilePathsFromPackagePaths(const TArray<FString>* InFilePathsToPatch)
{
	if (InFilePathsToPatch)
	{
		FilePathsToPatchMap.Reserve(InFilePathsToPatch->Num());
	}
	FilePathRenameMap.Reserve(PackagePathRenameMap.Num());

	// Construct all source and destination filenames from our package map
	for (const TTuple<FString, FString>& Package : PackagePathRenameMap)
	{
		const FString& SrcPackageName = Package.Key;
		const FString& DestPackage = Package.Value;
		FString SrcFilename;

		// To consider: Allow the caller to provide their own file filter
		if (FPackageName::IsVersePackage(SrcPackageName))
		{
			// Verse packages are not header patchable.
			// They are also not Packages as far as DoesPackageExist tells me.
			// But they are real files that in template copying have already been done, so we dont want a warning message.
			continue;
		}

		if (FPackageName::DoesPackageExist(SrcPackageName, &SrcFilename))
		{
			FString DestFilename = FPackageName::LongPackageNameToFilename(DestPackage, FString(FPathViews::GetExtension(SrcFilename, true)));
			FilePathRenameMap.Add({ SrcFilename, DestFilename });

			// If we have a patch file list, verify we have mappings for the passed in packages and update our FilePathsToPatchMap.
			// Otherwise use the complete mapping information from PackagePathRenameMap.
			if (InFilePathsToPatch)
			{
				if (!InFilePathsToPatch->Contains(SrcPackageName))
				{
					UE_LOGF(LogAssetHeaderPatcher, Warning, "No remapping for package %ls was provided. This package will not be patched.", *SrcPackageName);
				}
				else
				{
					FilePathsToPatchMap.Add({ SrcFilename, DestFilename });
				}
			}
			else
			{
				// Patch all files we have a remapping for whose src asset exists on disk
				FilePathsToPatchMap.Add({ SrcFilename, DestFilename });
			}
		}
		else
		{
			UE_LOGF(LogAssetHeaderPatcher, Warning, "{%ls} package does not exist, and will not be patched.", *SrcPackageName);
		}
	}
}

void FAssetHeaderPatcher::FContext::GeneratePackagePathsFromFilePaths(
	const FString& InSrcRoot,
	const FString& InDstRoot,
	const FString& InSrcBaseDir,
	const FString& InDstBaseDir)
{
	const FString SrcContentPath = FPaths::Combine(InSrcBaseDir, TEXTVIEW("Content"));
	const FString DstContentPath = FPaths::Combine(InDstBaseDir, TEXTVIEW("Content"));
	for (const TTuple<FString, FString>& SourceAndDest : FilePathRenameMap)
	{
		const FString& SrcFileName = SourceAndDest.Key;
		const FString& DstFileName = SourceAndDest.Value;

		FStringView SrcRelativePkgPath;
		FStringView DstRelativePkgPath;
		if (FPathViews::TryMakeChildPathRelativeTo(SrcFileName, SrcContentPath, SrcRelativePkgPath) &&
			FPathViews::TryMakeChildPathRelativeTo(DstFileName, DstContentPath, DstRelativePkgPath))
		{
			// chop the extension
			SrcRelativePkgPath = FPathViews::GetBaseFilenameWithPath(SrcRelativePkgPath);
			DstRelativePkgPath = FPathViews::GetBaseFilenameWithPath(DstRelativePkgPath);

			if (!SrcRelativePkgPath.IsEmpty() &&
				!DstRelativePkgPath.IsEmpty() &&
				!SrcRelativePkgPath.EndsWith(TEXTVIEW("/")) &&
				!DstRelativePkgPath.EndsWith(TEXTVIEW("/")))
			{
				PackagePathRenameMap.Add(
					FPaths::Combine(TEXTVIEW("/"), InSrcRoot, SrcRelativePkgPath),
					FPaths::Combine(TEXTVIEW("/"), InDstRoot, DstRelativePkgPath));
			}
		}
	}
}

void FAssetHeaderPatcher::FContext::GatherDependentPackages()
{
	// Paths under the __External root drop the package root, so create mappings, per plugin, 
	// we can leverage when handling those cases where the package path may have been remapped
	TMap<FString, TMap<FString, FString>> PluginExternalMappings;
	for (const TPair<FString, FString>& SrcDstPair : PackagePathRenameMap)
	{
		const FString& Src = SrcDstPair.Key;
		const FString& Dst = SrcDstPair.Value;

		FStringView SrcPackageRoot;
		FStringView SrcPackagePath;
		FStringView SrcPackageName;
		SplitLongPackageName(Src, SrcPackageRoot, SrcPackagePath, SrcPackageName);

		FStringView DstPackageRoot;
		FStringView DstPackagePath;
		FStringView DstPackageName;
		SplitLongPackageName(Dst, DstPackageRoot, DstPackagePath, DstPackageName);

		TMap<FString, FString>& ExternalMappings = PluginExternalMappings.FindOrAddByHash(GetTypeHash(SrcPackageRoot), FString(SrcPackageRoot));
		FStringView SrcPath = SrcPackagePath.IsEmpty() ? SrcPackageName : SrcPackagePath;
		FStringView DstPath = DstPackagePath.IsEmpty() ? DstPackageName : DstPackagePath;
		ExternalMappings.Add(FString(SrcPath), FString(DstPath));

		// if there is a path
		if (!SrcPackagePath.IsEmpty())
		{
			// add the local path/asset for the case of maps (which we cannot tell at this point)
			ExternalMappings.Add(FString(SrcPath.GetData()), FString(DstPath.GetData()));
		}

		// While iterating mappings, add any mountpoint changes
		if (SrcPackageRoot != DstPackageRoot)
		{
			const uint32 SrcPackageRootHash = GetTypeHash(SrcPackageRoot);
			FString* RemappedRoot = StringMountReplacements.FindByHash(SrcPackageRootHash, SrcPackageRoot);
			if (RemappedRoot)
			{
				UE_CLOGF(DstPackageRoot != *RemappedRoot, LogAssetHeaderPatcher, Warning,
					"Found conflicting mountpoint remapping: /%ls/ -> /%ls/ and /%ls/ -> /%ls/. The second mapping will be used to overwrite the first.",
					*FString(SrcPackageRoot), **RemappedRoot,
					*FString(SrcPackageRoot), *FString(DstPackageRoot));
			}
			StringMountReplacements.AddByHash(SrcPackageRootHash, FString(SrcPackageRoot), FString(DstPackageRoot));
		}
	}

	TMap<FString, FString> Result;
	IAssetRegistry& Registry = *IAssetRegistry::Get();

	TArray< TTuple<FString, FString> > ToProcess;
	Algo::Copy(PackagePathRenameMap, ToProcess);

	TStringBuilder<NAME_SIZE> SrcDependencyBuilder;
	while (ToProcess.Num())
	{
		TTuple<FString, FString> Package = ToProcess.Pop();

		if (Result.Contains(Package.Key))
		{
			continue;
		}

		// Become a patching name even if it doesn't have a file.
		Result.Add({ Package.Key, Package.Value });

		TArray<FName> Dependencies;
		if (!Registry.GetDependencies(FName(*Package.Key), Dependencies))
		{
			continue;
		}

		FStringView SrcPackageRoot = FPackageName::SplitPackageNameRoot(Package.Key, nullptr);
		FStringView DstPackageRoot = FPackageName::SplitPackageNameRoot(Package.Value, nullptr);
		for (const FName Dependency : Dependencies)
		{
			Dependency.ToString(SrcDependencyBuilder);
			FStringView SrcDependency = SrcDependencyBuilder.ToView();

			if (PackagePathRenameMap.FindByHash(GetTypeHash(SrcDependency), SrcDependency))
			{
				// We already handled this mapping
				continue;
			}

			FStringView SrcDependencyPackageRoot;
			FStringView SrcDependencyPackagePath;
			FStringView SrcDependencyPackageName;
			SplitLongPackageName(SrcDependency, SrcDependencyPackageRoot, SrcDependencyPackagePath, SrcDependencyPackageName);
			check(!SrcDependencyPackageRoot.IsEmpty());

			// Only consider dependency paths that are for the same package as our src->dst mapping
			// If the src mapping doesn't begin with a '/' the package name will be empty, since the path isn't a package path
			if (SrcDependencyPackageRoot != SrcPackageRoot)
			{
				continue;
			}

			TStringBuilder<NAME_SIZE> DstDependencyString;

			// Special handling for external references. The __External[Actors__|Objects__] directory is always under the package root, may contain an
			// arbitrary amount of subdirs but then ends with two hash subdirs. The path between the __External[Actors__|Objects__] and the two hash dirs
			// may need remapping so we look at our external mappings to do so.
			bool bHasExternalActorDir = SrcDependencyPackagePath.StartsWith(FPackagePath::GetExternalActorsFolderName());
			bool bHasExternalObjectsDir = !bHasExternalActorDir && SrcDependencyPackagePath.StartsWith(FPackagePath::GetExternalObjectsFolderName());
			if (bHasExternalActorDir || bHasExternalObjectsDir)
			{
				int32 RightPartStartPos;
				if (!SrcDependencyPackagePath.FindChar(TEXT('/'), RightPartStartPos))
				{
					// This is a path to only the special directory, skip it no remapping is needed
					continue;
				}
				RightPartStartPos++; // Skip past the '/'

				// Find the start of the two hash dirs
				// e.g. __ExternalActors__/path/of/interest/A/A9, we only want 'path/of/interest'
				FStringView ExternalPackagePath(SrcDependencyPackagePath.GetData() + RightPartStartPos, SrcDependencyPackagePath.Len() - RightPartStartPos);
				int32 HashDirStartPos = 0;
				int32 NumHashDirsToStrip = 2;
				while (NumHashDirsToStrip--)
				{
					if (ExternalPackagePath.FindLastChar(TEXT('/'), HashDirStartPos))
					{
						ExternalPackagePath.LeftChopInline(ExternalPackagePath.Len() - HashDirStartPos);
					}
				}

				// Our __External[Actors|Objects]__ path is malformed
				if (HashDirStartPos == INDEX_NONE)
				{
					continue;
				}

				const int32 HashPathOffset = RightPartStartPos + HashDirStartPos;
				FStringView HashPath(SrcDependencyPackagePath.GetData() + HashPathOffset, SrcDependencyPackagePath.Len() - HashPathOffset);
				const TMap<FString, FString>* ExternalMappings = PluginExternalMappings.FindByHash(GetTypeHash(SrcPackageRoot), SrcPackageRoot);
				if (!ExternalMappings)
				{
					// We have no mapping for this dependency's external actors/objects
					continue;
				}
				const FString* DstExternalPackagePath = ExternalMappings->FindByHash(GetTypeHash(ExternalPackagePath), ExternalPackagePath);

				DstDependencyString.AppendChar(TEXT('/'));
				DstDependencyString.Append(DstPackageRoot);
				DstDependencyString.AppendChar(TEXT('/'));
				DstDependencyString.Append(bHasExternalActorDir ? FPackagePath::GetExternalActorsFolderName() : FPackagePath::GetExternalObjectsFolderName());
				DstDependencyString.AppendChar(TEXT('/'));
				DstDependencyString.Append(DstExternalPackagePath ? *DstExternalPackagePath : ExternalPackagePath);
				DstDependencyString.Append(HashPath); // HashPath already contains the leading '/'
				DstDependencyString.AppendChar(TEXT('/'));
				DstDependencyString.Append(SrcDependencyPackageName);
			}
			else
			{
				// We aren't handling a special directory so replace the package root
				DstDependencyString.AppendChar(TEXT('/'));
				DstDependencyString.Append(DstPackageRoot);
				DstDependencyString.AppendChar(TEXT('/'));

				if (!SrcDependencyPackagePath.IsEmpty())
				{
					DstDependencyString.Append(SrcDependencyPackagePath);
					DstDependencyString.AppendChar(TEXT('/'));
				}

				DstDependencyString.Append(SrcDependencyPackageName);
			}

			// If a dep start with the package name, then we are going to copy the asset.
			// but we need to recurse on this asset as it may have sub dependencies we don't know of yet.
			ToProcess.Add({ FString(SrcDependency), DstDependencyString.ToString() });
		}
	}

	PackagePathRenameMap = MoveTemp(Result);
}

void FAssetHeaderPatcher::FContext::GenerateAdditionalRemappings()
{
	TArray<FCoreRedirect> ExternalObjectRedirects;
	TStringBuilder<24> ExternalActorsFolderBuilder;
	ExternalActorsFolderBuilder << FPackagePath::GetExternalActorsFolderName() << TEXT("/");
	const FStringView ExternalActorsFolder = ExternalActorsFolderBuilder.ToView();

	TStringBuilder<24> ExternalObjectsFolderBuilder;
	ExternalObjectsFolderBuilder << FPackagePath::GetExternalObjectsFolderName() << TEXT("/");
	const FStringView ExternalObjectsFolder = ExternalObjectsFolderBuilder.ToView();

	TStringBuilder<NAME_SIZE> SrcNameBuilder;
	TStringBuilder<NAME_SIZE> DstNameBuilder;
	for (const TTuple<FString, FString>& Package : PackagePathRenameMap)
	{
		const FString& SrcNameString = Package.Key;
		const FString& DstNameString = Package.Value;

		bool bIsExternalObjectOrActor = false;
		FStringView SrcPackageName;
		{
			FStringView SrcPackageRoot;
			FStringView SrcPackagePath;
			if (!ensure(SplitLongPackageName(SrcNameString, SrcPackageRoot, SrcPackagePath, SrcPackageName))
				|| SrcPackagePath.StartsWith(ExternalActorsFolder)
				|| SrcPackagePath.StartsWith(ExternalObjectsFolder))
			{
				bIsExternalObjectOrActor = true;
			}
		}

		// /Path/To/Package mapping
		{
			FCoreRedirect PackageRedirect(ECoreRedirectFlags::Type_Package,
				FCoreRedirectObjectName(SrcNameString),
				FCoreRedirectObjectName(DstNameString));

			if (bIsExternalObjectOrActor)
			{
				// The other mappings below don't apply to ExternalActors or ExternalObjects so we skip them
				// now that we have a PackagePath mapping for them
				ExternalObjectRedirects.Emplace(MoveTemp(PackageRedirect));
				continue;
			}
			else
			{
				Redirects.Emplace(MoveTemp(PackageRedirect));
			}
		}

		FStringView DstPackageName = FPathViews::GetBaseFilename(DstNameString);

		// Path.ObjectName mapping
		{
			SrcNameBuilder.Reset();
			SrcNameBuilder.Append(SrcNameString);
			SrcNameBuilder.AppendChar(TEXT('.'));
			SrcNameBuilder.Append(SrcPackageName);

			DstNameBuilder.Reset();
			DstNameBuilder.Append(DstNameString);
			DstNameBuilder.AppendChar(TEXT('.'));
			DstNameBuilder.Append(DstPackageName);

			FCoreRedirect PackageObjectRedirect(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Type_Object,
				FCoreRedirectObjectName(SrcNameBuilder.ToString()),
				FCoreRedirectObjectName(DstNameBuilder.ToString()));
			Redirects.Emplace(MoveTemp(PackageObjectRedirect));
		}

		// Path.ObjectName.* mapping
		{
			SrcNameBuilder.Reset();
			SrcNameBuilder.Append(SrcNameString);
			SrcNameBuilder.AppendChar(TEXT('.'));
			SrcNameBuilder.Append(SrcPackageName);
			SrcNameBuilder.AppendChar(TEXT('.'));

			DstNameBuilder.Reset();
			DstNameBuilder.Append(DstNameString);
			DstNameBuilder.AppendChar(TEXT('.'));
			DstNameBuilder.Append(DstPackageName);
			DstNameBuilder.AppendChar(TEXT('.'));

			FCoreRedirect ObjectRedirect(ECoreRedirectFlags::Option_MatchPrefix | ECoreRedirectFlags::Type_Object,
				FCoreRedirectObjectName(SrcNameBuilder.ToString()),
				FCoreRedirectObjectName(DstNameBuilder.ToString()));
			Redirects.Emplace(MoveTemp(ObjectRedirect));
		}

		// Path.Object.PersistentLevel.* mapping
		{
			FName PersistentLevelFName(NAME_PersistentLevel);
			SrcNameBuilder.Reset();
			SrcNameBuilder.Append(SrcNameString);
			SrcNameBuilder.AppendChar(TEXT('.'));
			SrcNameBuilder.Append(SrcPackageName);
			SrcNameBuilder.AppendChar(TEXT('.'));
			PersistentLevelFName.AppendString(SrcNameBuilder);
			SrcNameBuilder.AppendChar(TEXT('.'));

			DstNameBuilder.Reset();
			DstNameBuilder.Append(DstNameString);
			DstNameBuilder.AppendChar(TEXT('.'));
			DstNameBuilder.Append(DstPackageName);
			DstNameBuilder.AppendChar(TEXT('.'));
			PersistentLevelFName.AppendString(DstNameBuilder);
			DstNameBuilder.AppendChar(TEXT('.'));

			FCoreRedirect ObjectRedirect(ECoreRedirectFlags::Option_MatchPrefix | ECoreRedirectFlags::Type_Object,
				FCoreRedirectObjectName(SrcNameBuilder.ToString()),
				FCoreRedirectObjectName(DstNameBuilder.ToString()));
			Redirects.Emplace(MoveTemp(ObjectRedirect));
		}

		// MaterialFunctionInterface "EditorOnlyData"
		{
			SrcNameBuilder.Reset();
			SrcNameBuilder.Append(SrcNameString);
			SrcNameBuilder.AppendChar(TEXT('.'));
			SrcNameBuilder.Append(SrcPackageName);
			SrcNameBuilder.Append(TEXT("EditorOnlyData"));

			DstNameBuilder.Reset();
			DstNameBuilder.Append(DstNameString);
			DstNameBuilder.AppendChar(TEXT('.'));
			DstNameBuilder.Append(DstPackageName);
			DstNameBuilder.Append(TEXT("EditorOnlyData"));

			FCoreRedirect BlueprintClassRedirect(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Package,
				FCoreRedirectObjectName(SrcNameBuilder.ToString()),
				FCoreRedirectObjectName(DstNameBuilder.ToString()));
			Redirects.Emplace(MoveTemp(BlueprintClassRedirect));
		}

		// Compiled Blueprint class names
		{
			SrcNameBuilder.Reset();
			SrcNameBuilder.Append(SrcNameString);
			SrcNameBuilder.AppendChar(TEXT('.'));
			SrcNameBuilder.Append(SrcPackageName);
			SrcNameBuilder.Append(TEXT("_C"));

			DstNameBuilder.Reset();
			DstNameBuilder.Append(DstNameString);
			DstNameBuilder.AppendChar(TEXT('.'));
			DstNameBuilder.Append(DstPackageName);
			DstNameBuilder.Append(TEXT("_C"));

			FCoreRedirect BlueprintClassRedirect(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Package,
				FCoreRedirectObjectName(SrcNameBuilder.ToString()),
				FCoreRedirectObjectName(DstNameBuilder.ToString()));
			Redirects.Emplace(MoveTemp(BlueprintClassRedirect));
		}

		// Blueprint generated class default object
		{
			SrcNameBuilder.Reset();
			SrcNameBuilder.Append(SrcNameString);
			SrcNameBuilder.AppendChar(TEXT('.'));
			SrcNameBuilder.Append(DEFAULT_OBJECT_PREFIX);
			SrcNameBuilder.Append(SrcPackageName);
			SrcNameBuilder.Append(TEXT("_C"));

			DstNameBuilder.Reset();
			DstNameBuilder.Append(DstNameString);
			DstNameBuilder.AppendChar(TEXT('.'));
			DstNameBuilder.Append(DEFAULT_OBJECT_PREFIX);
			DstNameBuilder.Append(DstPackageName);
			DstNameBuilder.Append(TEXT("_C"));

			FCoreRedirect DefaultBlueprintClassRedirect(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Package,
				FCoreRedirectObjectName(SrcNameBuilder.ToString()),
				FCoreRedirectObjectName(DstNameBuilder.ToString()));
			Redirects.Emplace(MoveTemp(DefaultBlueprintClassRedirect));
		}
	}

	// For best-effort string matches. Intentionally excluding external objects as AssetRegistry Tag data
	// can't refer to these paths in a manner that we can't deduce from the redirects themselves
	for (auto& Redirect : Redirects)
	{
		const FCoreRedirectObjectName& SrcName = Redirect.OldName;
		const FCoreRedirectObjectName& DstName = Redirect.NewName;

		// Tag data can contain VersePaths which are like Top-Level Asset Paths
		// but with a mountpoint prefix and only '/' delimiters
		for (FString& VerseMount : VerseMountPoints)
		{
			SrcNameBuilder.Reset();
			SrcNameBuilder.AppendChar(TEXT('/'));
			SrcNameBuilder.Append(VerseMount);
			SrcName.PackageName.AppendString(SrcNameBuilder);
			SrcNameBuilder.AppendChar(TEXT('/'));
			SrcName.ObjectName.AppendString(SrcNameBuilder);

			DstNameBuilder.Reset();
			DstNameBuilder.AppendChar(TEXT('/'));
			DstNameBuilder.Append(VerseMount);
			DstName.PackageName.AppendString(DstNameBuilder);
			DstNameBuilder.AppendChar(TEXT('/'));
			DstName.ObjectName.AppendString(DstNameBuilder);
			VerseStringReplacements.Add(SrcNameBuilder.ToString(), DstNameBuilder.ToString());
		}
	}

	// Now that we have generated the string matches above, add the external redirects
	Redirects.Append(ExternalObjectRedirects);

	// Add prefix redirects for any mountpoint replacements
	TMap<FString, FString> FormattedStringMountReplacements;
	FormattedStringMountReplacements.Reserve(StringMountReplacements.Num());
	for (const auto& MountPointPair : StringMountReplacements)
	{
		const FString& SrcMountPoint = MountPointPair.Key;
		const FString& DstMountPoint = MountPointPair.Value;

		SrcNameBuilder.Reset();
		SrcNameBuilder.AppendChar(TEXT('/'));
		SrcNameBuilder.Append(SrcMountPoint);
		SrcNameBuilder.AppendChar(TEXT('/'));

		DstNameBuilder.Reset();
		DstNameBuilder.AppendChar(TEXT('/'));
		DstNameBuilder.Append(DstMountPoint);
		DstNameBuilder.AppendChar(TEXT('/'));

		FCoreRedirect MountRedirect(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix,
			FCoreRedirectObjectName(SrcNameBuilder.ToString()),
			FCoreRedirectObjectName(DstNameBuilder.ToString()));
		Redirects.Emplace(MoveTemp(MountRedirect));

		// Store off the actual mount path prefix to make patching easier later
		FormattedStringMountReplacements.Add(SrcNameBuilder.ToString(), DstNameBuilder.ToString());
	}
	StringMountReplacements = MoveTemp(FormattedStringMountReplacements);
}

// To override writing of FName's to ensure they have been patched
class FNamePatchingWriter final : public FArchiveProxy
{
public:
	FNamePatchingWriter(FArchive& InAr, const TMap<FNameEntryId, int32>& InNameToIndexMap)
		: FArchiveProxy(InAr)
		, NameToIndexMap(InNameToIndexMap)
	{
	}

	virtual ~FNamePatchingWriter() { }

	virtual FArchive& operator<<(FName& Name) override
	{
		FNameEntryId EntryId = Name.GetDisplayIndex();
		const int32* MaybeIndex = NameToIndexMap.Find(EntryId);

		if (MaybeIndex == nullptr)
		{
			ErrorMessage += FString::Printf(TEXT("Cannot serialize FName '%s' because it is not in the name table for %s\n"), *Name.ToString(), *GetArchiveName());
			SetCriticalError();
			return *this;
		}

		int32 Index = *MaybeIndex;
		int32 Number = Name.GetNumber();

		FArchive& Ar = *this;
		Ar << Index;
		Ar << Number;

		return *this;
	}

	const FString& GetErrorMessage() const
	{
		return ErrorMessage;
	}

private:
	const TMap<FNameEntryId, int32>& NameToIndexMap;
	FString ErrorMessage;
};

enum class EPatchedSection
{
	Summary,
	NameTable,
	SoftPathTable,
	GatherableTextDataTable,
	SearchableNamesMap,
	ImportTable,
	ExportTable,
	SoftPackageReferencesTable,
	ImportTypeHierarchyMap,
	ThumbnailTable,
	AssetRegistryData,
	AssetRegistryDependencyData,
};

struct FSectionData
{
	EPatchedSection Section = EPatchedSection::Summary;
	int64 Offset = 0;
	int64 Size = 0;
	bool bRequired = false;
};

enum class ESummaryOffset
{
	NameTable,
	SoftObjectPathList,
	GatherableTextDataTable,
	ImportTable,
	ExportTable,
	CellImportTable,
	CellExportTable,
	DependsTable,
	SoftPackageReferenceList,
	SearchableNamesMap,
	ImportTypeHierarchyMap,
	ThumbnailTable,
	AssetRegistryData,
	WorldTileInfoData,
	PreloadDependency, // Should not be present - only for cooked data
	BulkData,
	PayloadToc
};

// To override MemoryReaders FName method
class FReadFNameAs2IntFromMemoryReader final : public FLargeMemoryReader
{
public:
	FReadFNameAs2IntFromMemoryReader(TArray<FName>& InNameTable, const uint8* InData, const int64 Num, ELargeMemoryReaderFlags InFlags = ELargeMemoryReaderFlags::None, const FName InArchiveName = NAME_None)
		: FLargeMemoryReader(InData, Num, InFlags, InArchiveName)
		, NameTable(InNameTable)
	{
	}

	// FLargeMemoryReader falls back to FMemoryArchive's imp of this method.
	// which uses strings as the format for FName.
	// We need the 2xint32 version when decoding the current file formats. 
	virtual FArchive& operator<<(FName& OutName) override
	{
		int32 NameIndex;
		int32 Number;
		FArchive& Ar = *this;
		Ar << NameIndex;
		Ar << Number;

		if (NameTable.IsValidIndex(NameIndex))
		{
			FNameEntryId MappedName = NameTable[NameIndex].GetDisplayIndex();
			OutName = FName::CreateFromDisplayId(MappedName, Number);
		}
		else
		{
			OutName = FName();
			SetCriticalError();
		}

		return *this;
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FReadFNameAs2IntFromMemoryReader");
	}
private:
	TArray<FName>& NameTable;
};

struct FSummaryOffsetMeta
{
	// NOTE: The offsets in Summary get to a max of 312 bytes.
	// So we could drop this to a uint16 but that is probably overkill at this point.
	uint32 Offset : 31;
	uint32 bIs64Bit : 1;

	int64 Value(FPackageFileSummary& Summary) const
	{
		intptr_t Ptr = reinterpret_cast<intptr_t>(&Summary) + Offset;
		if (bIs64Bit)
		{
			return *reinterpret_cast<int64*>(Ptr);
		}
		else
		{
			return *reinterpret_cast<int32*>(Ptr);
		}
	}

	void PatchOffsetValue(FPackageFileSummary& Summary, int64 Value) const
	{
		intptr_t Ptr = reinterpret_cast<intptr_t>(&Summary) + Offset;
		if (bIs64Bit)
		{
			int64& Dst = *reinterpret_cast<int64*>(Ptr);
			Dst += Value;
		}
		else
		{
			int32& Dst = *reinterpret_cast<int32*>(Ptr);
			*reinterpret_cast<int32*>(Ptr) = IntCastChecked<int32>((int64)Dst + Value);
		}
	}
};

void PatchSummaryOffsets(FPackageFileSummary& Dst, int64 OffsetFrom, int64 OffsetDelta)
{
	if (!OffsetDelta)
	{
		return;
	}

	constexpr FSummaryOffsetMeta OffsetTable[] = {

#define UE_POPULATE_OFFSET_INFO(NAME)					\
			(uint32)STRUCT_OFFSET(FPackageFileSummary, NAME),	\
			std::is_same_v<decltype(((FPackageFileSummary*)0)->NAME), int64>

				{ UE_POPULATE_OFFSET_INFO(NameOffset) },
				{ UE_POPULATE_OFFSET_INFO(SoftObjectPathsOffset) },
				{ UE_POPULATE_OFFSET_INFO(GatherableTextDataOffset) },
				{ UE_POPULATE_OFFSET_INFO(MetaDataOffset) },
				{ UE_POPULATE_OFFSET_INFO(ImportOffset) },
				{ UE_POPULATE_OFFSET_INFO(ExportOffset) },
				{ UE_POPULATE_OFFSET_INFO(CellImportOffset) },
				{ UE_POPULATE_OFFSET_INFO(CellExportOffset) },
				{ UE_POPULATE_OFFSET_INFO(DependsOffset) },
				{ UE_POPULATE_OFFSET_INFO(SoftPackageReferencesOffset) },
				{ UE_POPULATE_OFFSET_INFO(SearchableNamesOffset) },
				{ UE_POPULATE_OFFSET_INFO(ImportTypeHierarchiesOffset) },
				{ UE_POPULATE_OFFSET_INFO(ThumbnailTableOffset) },
				{ UE_POPULATE_OFFSET_INFO(AssetRegistryDataOffset) },
				{ UE_POPULATE_OFFSET_INFO(BulkDataStartOffset) },
				{ UE_POPULATE_OFFSET_INFO(WorldTileInfoDataOffset) },
				{ UE_POPULATE_OFFSET_INFO(PreloadDependencyOffset) },
				{ UE_POPULATE_OFFSET_INFO(PayloadTocOffset) },

	#undef UE_POPULATE_OFFSET_INFO
	};

	for (const FSummaryOffsetMeta& OffsetData : OffsetTable)
	{
		if (OffsetData.Value(Dst) > OffsetFrom)
		{
			OffsetData.PatchOffsetValue(Dst, OffsetDelta);
		}
	}
};

FAssetDataTagMap MakeTagMap(const TArray<UE::AssetRegistry::FDeserializeTagData>& TagData)
{
	FAssetDataTagMap Out;
	Out.Reserve(TagData.Num());
	for (const UE::AssetRegistry::FDeserializeTagData& Tag : TagData)
	{
		if (!Tag.Key.IsEmpty() && !Tag.Value.IsEmpty())
		{
			Out.Add(*Tag.Key, Tag.Value);
		}
	}

	return Out;
}

// The information we need in the task to do patching.
class FAssetHeaderPatcherInner
{
public:
	using EResult = FAssetHeaderPatcher::EResult;

	struct FThumbnailEntry
	{
		FString ObjectShortClassName;
		FString ObjectPathWithoutPackageName;
		int32 FileOffset = 0;
	};

	FAssetHeaderPatcherInner(const FString& InSrcAsset, const FString& InDstAsset, const TMap<FString, FString>& InVerseStringReplacements, const TMap<FString, FString>& InStringMountPointReplacements, FArchive* InDstArchive = nullptr)
		: SrcAsset(InSrcAsset)
		, DstAsset(InDstAsset)
		, VerseStringReplacements(InVerseStringReplacements)
		, StringMountPointReplacements(InStringMountPointReplacements)
		, DstArchive(InDstArchive)
		, bPatchPrimaryAssetTag(false)
		, bIsPackagePathInNametable(false)
		, bIsNonOneFilePerActorPackage(false)
	{
		for (auto TagToIgnore : TagsToIgnore)
		{
			IgnoredTags.Add(TagToIgnore);
		}
	}

	// Reset anything not set via construction. Used for testing
	void ResetInternalState()
	{
		AssetRegistryData = FAssetRegistryData();
		HeaderInformation = FHeaderInformation();
		AddedNames.Empty();
		ExportTable.Empty();
		ImportTable.Empty();
		ImportTablePatchedNames.Empty();
		ImportNameToImportTableIndexLookup.Empty();
		GatherableTextDataTable.Empty();
		NameTable.Empty();
		NameToIndexMap.Empty();
		RenameMap.Empty();
		SearchableNamesMap.Empty();
		SoftObjectPathTable.Empty();
		SoftPackageReferencesTable.Empty();
		Summary = FPackageFileSummary();
		ImportTypeHierarchyMap.Empty();
		ThumbnailTable.Empty();
		UnchangedNames.Empty();
	}

	bool DoPatch(FString& InOutString);
	bool DoPatch(FName& InOutName);
	bool DoPatch(FSoftObjectPath& InOutSoft);
	bool DoPatch(FTopLevelAssetPath& InOutPath);
	bool DoPatch(FGatherableTextData& InOutGatherablerTextData);
	bool DoPatch(FThumbnailEntry& InOutThumbnailEntry);
	bool DoPatch(UE::Serialization::Private::FTypeResource& InOutTypeResource);
	bool RemapFName(FName SrcName, FName DstName);
	bool AddFName(FName DstName);
	bool ShouldReplaceMountPoint(const FStringView InPath, FStringView& OutSrcMountPoint, FStringView& OutDstMountPoint);
	FCoreRedirectObjectName GetFullObjectNameFromObjectResource(const FObjectResource& InResource, bool bIsExport, bool bWalkImportsOnly = false);

	struct FExportPatch
	{
		int32 TableIndex;
		FName ObjectName;
	};

	struct FImportPatch
	{
		int32 TableIndex;
		FName ObjectName;
		FPackageIndex OuterIndex;
		FName ClassName;
		FName ClassPackage;
#if WITH_EDITORONLY_DATA
		FName PackageName;
#endif
		bool bUsedInGame = true;
	};

	void GetExportTablePatches(TArray<FExportPatch>& OutExportPatches);
	FAssetHeaderPatcher::EResult GetImportTablePatches(TArray<FImportPatch>& OutImportPatches, int32& OutNewImportCount);
	void PatchExportAndImportTables(const TArray<FExportPatch>& InExportPatches, const TArray<FImportPatch>& InImportPatches, const int32 InNewImportCount);
	void PatchNameTable();

	FAssetHeaderPatcher::EResult PatchHeader();
	FAssetHeaderPatcher::EResult PatchHeader_Deserialize();
	FAssetHeaderPatcher::EResult PatchHeader_PatchSections();
	FAssetHeaderPatcher::EResult PatchHeader_WriteDestinationFile();
	void DumpState(FStringView InDir);

	TSet<FString> IgnoredTags;

	const FString& SrcAsset;
	const FString& DstAsset;
	const TMap<FString, FString>& VerseStringReplacements;
	const TMap<FString, FString>& StringMountPointReplacements;
	FArchive* DstArchive = nullptr;
	TUniquePtr<FArchive> DstArchiveOwner;

	TArray64<uint8> SrcBuffer;

	struct FHeaderInformation
	{
		int64 SummarySize = -1;
		int64 NameTableSize = -1;
		int64 SoftObjectPathListSize = -1;
		int64 GatherableTextDataSize = -1;
		int64 ImportTableSize = -1;
		int64 ExportTableSize = -1;
		int64 SoftPackageReferencesListSize = -1;
		int64 ThumbnailTableSize = -1;
		int64 SearchableNamesMapSize = -1;
		int64 ImportTypeHierarchiesMapSize = -1;
		int64 AssetRegistryDataSize = -1;
		int64 PackageTrailerSize = -1;
	};

	FHeaderInformation HeaderInformation;
	FPackageFileSummary Summary;
	FName OriginalPackagePath;					 // e.g. "/MountName/TopLevelPackageName"
	FName OriginalNonOneFilePerActorPackagePath; // e.g. "/MountName/MountName"
	FName DstPackagePath;						 // OriginalPackagePath, or the remapped name of it if it was remapped
	FString OriginalPrimaryAssetName;			 // e.g. "MountName"
	bool bPatchPrimaryAssetTag;
	bool bIsPackagePathInNametable;
	bool bIsNonOneFilePerActorPackage;

	// NameTable Members
	TArray<FName> NameTable;
	TMap<FNameEntryId, int32> NameToIndexMap;
	TSet<FNameEntryId> UnchangedNames;
	TMap<FNameEntryId, FNameEntryId> RenameMap;
	TSet<FNameEntryId> AddedNames;
	// Export/Import table
	TArray<TPair<FCoreRedirectObjectName, FCoreRedirectObjectName>> ImportTablePatchedNames;
	TMap<FCoreRedirectObjectName, int32> ImportNameToImportTableIndexLookup;

	TArray<FSoftObjectPath> SoftObjectPathTable;
	TArray<FGatherableTextData> GatherableTextDataTable;
	TArray<FObjectImport> ImportTable;
	TArray<FObjectExport> ExportTable;
	TArray<FName> SoftPackageReferencesTable;
	TMap<FPackageIndex, TArray<FName>> SearchableNamesMap;
	TArray<FThumbnailEntry> ThumbnailTable;
	TMap<FPackageIndex, UE::Serialization::Private::FImportTypeHierarchy> ImportTypeHierarchyMap;

	// Asset registry data information
	struct FAssetRegistryObjectData
	{
		UE::AssetRegistry::FDeserializeObjectPackageData ObjectData;
		TArray<UE::AssetRegistry::FDeserializeTagData> TagData;
	};

	struct FAssetRegistryData
	{
		int64 SectionSize = -1;
		UE::AssetRegistry::FDeserializePackageData PkgData;
		TArray<FAssetRegistryObjectData> ObjectData;
		int64 DependencyDataSectionSize = -1;
		TMap<int32, bool> ImportIndexUsedInGame;
		TMap<FName, bool> SoftPackageReferenceUsedInGame;
		TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>> ExtraPackageDependencies;
	};
	FAssetRegistryData AssetRegistryData;
};

FAssetHeaderPatcher::EResult FAssetHeaderPatcher::DoPatch(const FString& InSrcAsset, const FString& InDstAsset, const FContext& InContext)
{
	FAssetHeaderPatcherInner Inner(InSrcAsset, InDstAsset, InContext.VerseStringReplacements, InContext.StringMountReplacements);
	if (!FFileHelper::LoadFileToArray(Inner.SrcBuffer, *Inner.SrcAsset))
	{
		UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to load %ls", *Inner.SrcAsset);
		return FAssetHeaderPatcherInner::EResult::ErrorFailedToLoadSourceAsset;
	}
	else
	{
		// Swap in the CoreRedirect context for the patcher since we might be running on a different thread with a separate context
		// We do not use a FScopeCoreRedirectsContext as that will copy into a new context but we want to re-use the patcher's context
		FCoreRedirectsContext& OriginalContext = FCoreRedirectsContext::GetThreadContext();
		FCoreRedirectsContext::SetThreadContext(InContext.RedirectsContext);
		ON_SCOPE_EXIT{ FCoreRedirectsContext::SetThreadContext(OriginalContext); };

		return Inner.PatchHeader();
	}
}

void FAssetHeaderPatcher::Reset()
{
	ErroredFiles.Empty();
	PatchedFiles.Empty();

	PatchingTask = UE::Tasks::FTask();
	Status = EResult::NotStarted;
	bCancelled = false;
}
void FAssetHeaderPatcher::SetContext(const FContext& InContext)
{
	checkf(!IsPatching(), TEXT("Cannot set the patcher context while patching"));
	Context = InContext;

	// Copy the global context into our own to inherit any global redirects already loaded
	Context.RedirectsContext = FCoreRedirectsContext(FCoreRedirectsContext::GetGlobalContext());
	Context.RedirectsContext.InitializeContext();

	// Disable validation of the CoreRedirects as these redirects are handmade and should not need extra validation.
	// RedirectionSummary is unimportant for patching and will result in unnecessary allocations when we add our redirects.
	// Leave DebugMode on since if someone wants to debug, we should not prevent them.
	FCoreRedirectsContext::EFlags NewFlags = Context.RedirectsContext.GetFlags();
	NewFlags = NewFlags & ~(FCoreRedirectsContext::EFlags::ValidateAddedRedirects | FCoreRedirectsContext::EFlags::UseRedirectionSummary);
	Context.RedirectsContext.SetFlags(NewFlags);

	{
		// Swap the thread context to the patcher's FCoreRedirectsContext so we may populate it once and share it with the task threads
		// We do not use a FScopeCoreRedirectsContext as that will copy into a new context but we want to re-use the patcher's context
		FCoreRedirectsContext& OriginalContext = FCoreRedirectsContext::GetThreadContext();
		FCoreRedirectsContext::SetThreadContext(Context.RedirectsContext);
		ON_SCOPE_EXIT{ FCoreRedirectsContext::SetThreadContext(OriginalContext); };

		FCoreRedirects::AddRedirectList(Context.Redirects, TEXT("Asset Header Patcher"));
	}

	Reset();
}

UE::Tasks::FTask FAssetHeaderPatcher::PatchAsync(const FPatchAsyncParams& Params)
{
	PatchedFiles = Context.FilePathsToPatchMap;
	if (Params.OutNumFilesToPatch)
	{
		*Params.OutNumFilesToPatch = PatchedFiles.Num();
	}

	// Spawn tasks (Scatter)
	TArray<UE::Tasks::FTask> PatchAssetTasks;

	// Note we are scheduling and launching tasks one at a time rather than preparing all jobs and launching all at once.
	// While this means more overhead scheduling, it means that we won't have many tasks all hit the filesystem at the same time
	// attempting to read and (more importantly) write to disk at the exact same time.
#if DEBUG_ASSET_HEADER_PATCHING
	constexpr bool bSingleThreaded = true; // Useful for debugging
#else
	constexpr bool bSingleThreaded = false;
#endif
	for (const TTuple<FString, FString>& Filename : PatchedFiles)
	{
		auto DoPatchFn = [this, 
			SrcFilename = Filename.Key,
			DstFilename = Filename.Value,
			NumPatched = Params.OutNumFilesPatched,
			OnSuccess = Params.OnSuccess,
			OnError = Params.OnError]()
			{
				// Even if we are cancelled, increment our progress
				if (NumPatched)
				{
					// We don't support C++20 in all modules and platforms yet and avoid using atomic_ref as a result
					FPlatformAtomics::InterlockedAdd((volatile int32*)NumPatched, 1);
				}

				if (bCancelled)
				{
					return;
				}

				FAssetHeaderPatcher::EResult Result = FAssetHeaderPatcher::DoPatch(SrcFilename, DstFilename, Context);
				if (Result != FAssetHeaderPatcher::EResult::Success)
				{
					FScopeLock Lock(&ErroredFilesLock);
					// Don't lose our cancelled state, even when there are errors
					if (Status != EResult::Cancelled)
					{
						Status = Result;
					}
					ErroredFiles.Add(SrcFilename, Result);

					OnError.ExecuteIfBound(SrcFilename, DstFilename);
				}
				else
				{
					OnSuccess.ExecuteIfBound(SrcFilename, DstFilename);
				}
			};

		if constexpr (bSingleThreaded)
		{
			DoPatchFn();
		}
		else
		{
			PatchAssetTasks.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(DoPatchFn), Params.TaskPriority));
		}
	}

	// Once all tasks have completed, remove the redirects before we declare Patching complete
	PatchingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]()
		{
			if (Status != EResult::Cancelled && ErroredFiles.IsEmpty())
			{
				Status = EResult::Success;
			}

			{
				FScopeLock Lock(&ErroredFilesLock);
				for (auto& ErroredFile : ErroredFiles)
				{
					PatchedFiles.Remove(ErroredFile.Key);
				}
			}

		},
		UE::Tasks::Prerequisites(PatchAssetTasks),
		bSingleThreaded ? UE::Tasks::ETaskPriority::Default : Params.TaskPriority);

	Status = EResult::InProgress;

	return PatchingTask;
}

FAssetHeaderPatcher::EResult FAssetHeaderPatcherInner::PatchHeader()
{
	FAssetHeaderPatcher::EResult Result = PatchHeader_Deserialize();
	if (Result != EResult::Success)
	{
		return Result;
	}
	
	if (DumpOutputDirectory.IsEmpty())
	{
		Result = PatchHeader_PatchSections();
		if (Result != EResult::Success)
		{
			return Result;
		}
	}
	else
	{
		FString BaseDir = DumpOutputDirectory;
		FPaths::NormalizeDirectoryName(BaseDir);

		FString BeforeDir = BaseDir / FString(TEXT("Before"));
		FPaths::RemoveDuplicateSlashes(BeforeDir);
		DumpState(BeforeDir);

		Result = PatchHeader_PatchSections();
		if (Result != EResult::Success)
		{
			return Result;
		}

		FString AfterDir = BaseDir / FString(TEXT("After"));
		FPaths::RemoveDuplicateSlashes(AfterDir);
		DumpState(AfterDir);
	}

	return PatchHeader_WriteDestinationFile();
}

FAssetHeaderPatcher::EResult FAssetHeaderPatcherInner::PatchHeader_Deserialize()
{
	FReadFNameAs2IntFromMemoryReader MemAr(NameTable, SrcBuffer.GetData(), SrcBuffer.Num());

	MemAr << Summary;
	HeaderInformation.SummarySize = MemAr.Tell();

	// Summary.PackageName isn't always serialized. In such cases, determine the package name from the file name
	if (Summary.PackageName.IsEmpty() || Summary.PackageName.Equals(TEXT("None")))
	{
		// e.g. "../../Some/Long/Path/MyPlugin/Plugins/MyPackage/Content/TopLevelAssetName.uasset"
		TStringView Path(SrcAsset);

		static const TStringView ContentDir(TEXT("/Content/"));
		int32 Pos = Path.Find(ContentDir, ESearchCase::IgnoreCase);
		if (Pos <= 0)
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Cannot patch '%ls': Package header is missing a 'PackageName' string, nor could a PackageName be deduced.", *SrcAsset);
			return FAssetHeaderPatcher::EResult::ErrorEmptyRequireSection;
		}
		
		int32 MountNamePos;
		TStringView LeftPath(Path.GetData(), Pos);
		if (!LeftPath.FindLastChar(TEXT('/'), MountNamePos))
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Cannot patch '%ls': Package header is missing a 'PackageName' string, nor could a PackageName be deduced.", *SrcAsset);
			return FAssetHeaderPatcher::EResult::ErrorEmptyRequireSection;
		}

		int32 ExtensionPos;
		TStringView RightPath(Path.GetData() + Pos + ContentDir.Len());
		if (!RightPath.FindLastChar(TEXT('.'), ExtensionPos))
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Cannot patch '%ls': Package header is missing a 'PackageName' string, nor could a PackageName be deduced.", *SrcAsset);
			return FAssetHeaderPatcher::EResult::ErrorEmptyRequireSection;
		}

		TStringView MountName(LeftPath.GetData() + MountNamePos, (Pos - MountNamePos) + 1); // + 1 so we can include the '/' from "/Content"
		TStringView AssetPath(RightPath.GetData(), ExtensionPos);
		Summary.PackageName.Empty(MountName.Len() + AssetPath.Len());
		Summary.PackageName.Append(MountName);
		Summary.PackageName.Append(AssetPath);
	}

	// Store the original name as an FName as it will be used when
	// patching paths for other objects in the package
	{
		OriginalPackagePath = FName(Summary.PackageName, NAME_NO_NUMBER_INTERNAL);

		// Some ObjectPaths have an implied package, however when it comes to 
		// non-One File Per Actor packages, the implied package is the map package
		// so we determine which package we are and cache the map name in case we need it
		{
			bIsNonOneFilePerActorPackage = false;
			TStringBuilder<256> PathBuilder;
			PathBuilder.AppendChar(TEXT('/'));
			PathBuilder.Append(FPackagePath::GetExternalActorsFolderName());
			PathBuilder.AppendChar(TEXT('/'));
			if (Summary.PackageName.Contains(PathBuilder))
			{
				bIsNonOneFilePerActorPackage = true;
			}
			else
			{
				PathBuilder.Reset();
				PathBuilder.AppendChar(TEXT('/'));
				PathBuilder.Append(FPackagePath::GetExternalObjectsFolderName());
				PathBuilder.AppendChar(TEXT('/'));
				bIsNonOneFilePerActorPackage = Summary.PackageName.Contains(PathBuilder);
			}

			int32 SlashPos = INDEX_NONE;
			FStringView PackageRoot(Summary.PackageName);
			if (!PackageRoot.FindChar(TEXT('/'), SlashPos) || SlashPos != 0)
			{
				UE_LOGF(LogAssetHeaderPatcher, Error, "Cannot patch '%ls': PackageName is malformed.", *SrcAsset);
				return FAssetHeaderPatcher::EResult::ErrorFailedToDeserializeSourceAsset;
			}

			PackageRoot.RightChopInline(1); // Drop the first slash
			if (!PackageRoot.FindChar(TEXT('/'), SlashPos))
			{
				UE_LOGF(LogAssetHeaderPatcher, Error, "Cannot patch '%ls': PackageName is malformed.", *SrcAsset);
				return FAssetHeaderPatcher::EResult::ErrorFailedToDeserializeSourceAsset;
			}

			PathBuilder.Reset();
			PathBuilder.AppendChar(TEXT('/'));
			PathBuilder.Append(PackageRoot.GetData(), SlashPos);
			PathBuilder.AppendChar(TEXT('/'));
			PathBuilder.Append(PackageRoot.GetData(), SlashPos);
			OriginalNonOneFilePerActorPackagePath = FName(PathBuilder);

			// While here set the OriginalPrimaryAssetName which is used in AssetRegistry Tag lookups for GameFeatureData
			bPatchPrimaryAssetTag = FPathViews::GetBaseFilename(Summary.PackageName) == TEXT("GameFeatureData");
			OriginalPrimaryAssetName.Empty();
			OriginalPrimaryAssetName.Append(PackageRoot.GetData(), SlashPos);
		}
	}

	// set version numbers so components branch correctly
	MemAr.SetUEVer(Summary.GetFileVersionUE());
	MemAr.SetLicenseeUEVer(Summary.GetFileVersionLicenseeUE());
	MemAr.SetEngineVer(Summary.SavedByEngineVersion);
	MemAr.SetCustomVersions(Summary.GetCustomVersionContainer());
	if (Summary.GetPackageFlags() & PKG_FilterEditorOnly)
	{
		MemAr.SetFilterEditorOnly(true);
	}

	if (Summary.DataResourceOffset > 0)
	{
		// Should only be set in cooked data. If that changes, we need to add code to patch it
		UE_LOGF(LogAssetHeaderPatcher, Error, "Asset %ls has an unexpected DataResourceOffset", *SrcAsset);
		return EResult::ErrorUnexpectedSectionOrder;
	}
	if (Summary.CellExportCount > 0 || Summary.CellImportCount > 0)
	{
		UE_LOGF(LogAssetHeaderPatcher, Error, "Asset %ls contains unexpected VCells", *SrcAsset);
		return EResult::ErrorUnexpectedSectionOrder;
	}

	if (Summary.NameCount > 0)
	{
		MemAr.Seek(Summary.NameOffset);
		NameTable.Reserve(Summary.NameCount);
		for (int32 NameMapIdx = 0; NameMapIdx < Summary.NameCount; ++NameMapIdx)
		{
			FNameEntrySerialized NameEntry(ENAME_LinkerConstructor);
			MemAr << NameEntry;
			NameTable.Add(FName(NameEntry));
		}

		HeaderInformation.NameTableSize = MemAr.Tell() - HeaderInformation.SummarySize;

		// Initialize a mapping for Name to index in NameTable as we will use
		// this for patching in new names and to determine if multiple FNames share the same 
		// value but might not after patching (i.e. their use of the name differs based on context, and
		// post-patching the FNames in those contexts no longer match.
		NameToIndexMap.Empty(NameTable.Num());
		UnchangedNames.Reserve(NameTable.Num());
		RenameMap.Reserve(NameTable.Num());
		AddedNames.Empty();
		for (int32 i = 0; i < NameTable.Num(); ++i)
		{
			NameToIndexMap.Add(NameTable[i].GetDisplayIndex(), i);
		}
	}

	if (Summary.SoftObjectPathsCount > 0)
	{
		MemAr.Seek(Summary.SoftObjectPathsOffset);
		SoftObjectPathTable.Reserve(Summary.SoftObjectPathsCount);
		for (int32 Idx = 0; Idx < Summary.SoftObjectPathsCount; ++Idx)
		{
			// Note, a non IsPersistent() archive is used to preserve the original
			// FSoftObjectPaths found in the header, since those will refer to entries in the 
			// NameTable. The call to SerializePath below will redirect FNames when using an
			// IsPersistent archive which would make fixing up the nametable more difficult
			FSoftObjectPath& PathRef = SoftObjectPathTable.AddDefaulted_GetRef();
			PathRef.SerializePath(MemAr);
		}
		HeaderInformation.SoftObjectPathListSize = MemAr.Tell() - Summary.SoftObjectPathsOffset;
	}
	else if(Summary.GetFileVersionUE() >= EUnrealEngineObjectUE5Version::ADD_SOFTOBJECTPATH_LIST)
	{
		HeaderInformation.SoftObjectPathListSize = 0;
	}
	else
	{
		UE_LOGF(LogAssetHeaderPatcher, Error, "Asset '%ls' is too old to be used with AssetHeaderPatching. Please resave the file before trying to patch again.", *SrcAsset);
		return EResult::ErrorUnkownSection;
	}

	if (Summary.GatherableTextDataCount > 0)
	{
		MemAr.Seek(Summary.GatherableTextDataOffset);
		GatherableTextDataTable.Reserve(Summary.GatherableTextDataCount);
		for (int32 GatherableTextDataIndex = 0; GatherableTextDataIndex < Summary.GatherableTextDataCount; ++GatherableTextDataIndex)
		{
			FGatherableTextData& GatherableTextData = GatherableTextDataTable.Emplace_GetRef();
			MemAr << GatherableTextData;
		}

		HeaderInformation.GatherableTextDataSize = MemAr.Tell() - Summary.GatherableTextDataOffset;
	}
	else
	{
		HeaderInformation.GatherableTextDataSize = 0;
	}

#define UE_CHECK_AND_SET_ERROR_AND_RETURN(EXP)	\
	do											\
	{											\
		if (EXP)								\
		{										\
			UE_LOGF(LogAssetHeaderPatcher, Log, "Asset %ls fails %ls", *SrcAsset, TEXT(#EXP));	\
			return EResult::ErrorBadOffset;		\
		}										\
	}											\
	while(0)

	if (Summary.ImportCount > 0)
	{
		UE_CHECK_AND_SET_ERROR_AND_RETURN(Summary.ImportOffset >= Summary.TotalHeaderSize);
		UE_CHECK_AND_SET_ERROR_AND_RETURN(Summary.ImportOffset < 0);

		MemAr.Seek(Summary.ImportOffset);
		ImportTable.Reserve(Summary.ImportCount);
		ImportTablePatchedNames.Reserve(ImportTable.Num());
		ImportNameToImportTableIndexLookup.Reserve(ImportTable.Num());
		for (int32 ImportIndex = 0; ImportIndex < Summary.ImportCount; ++ImportIndex)
		{
			FObjectImport& Import = ImportTable.Emplace_GetRef();
			MemAr << Import;
		}

		HeaderInformation.ImportTableSize = MemAr.Tell() - Summary.ImportOffset;
	}
	else
	{
		HeaderInformation.ImportTableSize = 0;
	}

	if (Summary.ExportCount > 0)
	{
		UE_CHECK_AND_SET_ERROR_AND_RETURN(Summary.ExportOffset >= Summary.TotalHeaderSize);
		UE_CHECK_AND_SET_ERROR_AND_RETURN(Summary.ExportOffset < 0);

		MemAr.Seek(Summary.ExportOffset);
		ExportTable.Reserve(Summary.ExportCount);
		for (int32 ExportIndex = 0; ExportIndex < Summary.ExportCount; ++ExportIndex)
		{
			FObjectExport& Export = ExportTable.Emplace_GetRef();
			MemAr << Export;
		}

		HeaderInformation.ExportTableSize = MemAr.Tell() - Summary.ExportOffset;
	}
	else
	{
		HeaderInformation.ExportTableSize = 0;
	}

#undef UE_CHECK_AND_SET_ERROR_AND_RETURN

	if (Summary.SoftPackageReferencesCount)
	{
		MemAr.Seek(Summary.SoftPackageReferencesOffset);
		SoftPackageReferencesTable.Reserve(Summary.SoftPackageReferencesCount);
		for (int32 Idx = 0; Idx < Summary.SoftPackageReferencesCount; ++Idx)
		{
			FName& Reference = SoftPackageReferencesTable.Emplace_GetRef();
			MemAr << Reference;
		}

		HeaderInformation.SoftPackageReferencesListSize = MemAr.Tell() - Summary.SoftPackageReferencesOffset;
	}
	else
	{
		HeaderInformation.SoftPackageReferencesListSize = 0;
	}

	if (Summary.SearchableNamesOffset)
	{
		MemAr.Seek(Summary.SearchableNamesOffset);
		FLinkerTables LinkerTables;
		LinkerTables.SerializeSearchableNamesMap(MemAr);
		SearchableNamesMap = MoveTemp(LinkerTables.SearchableNamesMap);

		HeaderInformation.SearchableNamesMapSize = MemAr.Tell() - Summary.SearchableNamesOffset;
	}

	if (Summary.ImportTypeHierarchiesOffset)
	{
		MemAr.Seek(Summary.ImportTypeHierarchiesOffset);

		MemAr << ImportTypeHierarchyMap;

		HeaderInformation.ImportTypeHierarchiesMapSize = MemAr.Tell() - Summary.ImportTypeHierarchiesOffset;
	}

	if (Summary.ThumbnailTableOffset)
	{
		MemAr.Seek(Summary.ThumbnailTableOffset);

		int32 ThumbnailCount = 0;
		MemAr << ThumbnailCount;

		ThumbnailTable.Reserve(ThumbnailCount);
		for (int32 Index = 0; Index < ThumbnailCount; ++Index)
		{
			FThumbnailEntry& Entry = ThumbnailTable.Emplace_GetRef();
			MemAr << Entry.ObjectShortClassName;
			MemAr << Entry.ObjectPathWithoutPackageName;
			MemAr << Entry.FileOffset;
		}

		HeaderInformation.ThumbnailTableSize = MemAr.Tell() - Summary.ThumbnailTableOffset;
	}

	// Load AR data
	if (Summary.AssetRegistryDataOffset)
	{
		MemAr.Seek(Summary.AssetRegistryDataOffset);

		UE::AssetRegistry::EReadPackageDataMainErrorCode ErrorCode;
		if (!AssetRegistryData.PkgData.DoSerialize(MemAr, Summary, ErrorCode))
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to deserialize asset registry data for %ls", *SrcAsset);
			return EResult::ErrorFailedToDeserializeSourceAsset;
		}

		AssetRegistryData.ObjectData.Reserve(AssetRegistryData.PkgData.ObjectCount);
		for (int32 i = 0; i < AssetRegistryData.PkgData.ObjectCount; ++i)
		{
			FAssetRegistryObjectData& ObjData = AssetRegistryData.ObjectData.Emplace_GetRef();
			if (!ObjData.ObjectData.DoSerialize(MemAr, ErrorCode))
			{
				UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to deserialize asset registry data for %ls", *SrcAsset);
				return EResult::ErrorFailedToDeserializeSourceAsset;
			}

			ObjData.TagData.Reserve(ObjData.ObjectData.TagCount);
			for (int32 j = 0; j < ObjData.ObjectData.TagCount; ++j)
			{
				if (!ObjData.TagData.Emplace_GetRef().DoSerialize(MemAr, ErrorCode))
				{
					UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to deserialize asset registry data for %ls", *SrcAsset);
					return EResult::ErrorFailedToDeserializeSourceAsset;
				}
			}
		}

		AssetRegistryData.SectionSize = MemAr.Tell() - Summary.AssetRegistryDataOffset;

		UE::AssetRegistry::FReadPackageDataDependenciesArgs DependenciesArgs;
		DependenciesArgs.BinaryNameAwareArchive = &MemAr;
		DependenciesArgs.AssetRegistryDependencyDataOffset = AssetRegistryData.PkgData.DependencyDataOffset;
		DependenciesArgs.NumImports = Summary.ImportCount;
		DependenciesArgs.NumSoftPackageReferences = Summary.SoftPackageReferencesCount;
		DependenciesArgs.PackageVersion = Summary.GetFileVersionUE();

		if (!UE::AssetRegistry::ReadPackageDataDependencies(DependenciesArgs))
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to deserialize asset registry data for %ls", *SrcAsset);
			return EResult::ErrorFailedToDeserializeSourceAsset;
		}

		if (DependenciesArgs.ImportUsedInGame.Num() != Summary.ImportCount ||
			DependenciesArgs.SoftPackageUsedInGame.Num() != Summary.SoftPackageReferencesCount)
		{
			UE_LOGF(LogAssetHeaderPatcher, Error,
				"Failed to deserialize asset registry data for %ls. ReadPackageDataDependencies internal error: (%d != %d || %d != %d).",
				*SrcAsset, DependenciesArgs.ImportUsedInGame.Num(), Summary.ImportCount,
				DependenciesArgs.SoftPackageUsedInGame.Num(), Summary.SoftPackageReferencesCount);
			return EResult::ErrorFailedToDeserializeSourceAsset;
		}

		if ((AssetRegistryData.PkgData.DependencyDataOffset != INDEX_NONE) != (DependenciesArgs.AssetRegistryDependencyDataSize != 0))
		{
			// When writing the file, we use validity of Size to decide whether we write or skip the
			// AssetRegistryDependencyData section, but in the earlier AssetRegistryData section we write the DependencyOffset
			// based on whether the offset is valid. To prevent corruption, we need the validity of each to match. 
			UE_LOG(LogAssetHeaderPatcher, Error,
				TEXT("Failed to deserialize asset registry data for %s. DependencyDataOffset (%" INT64_FMT ") != -1 does not match AssetRegistryDependencyDataSize (%" INT64_FMT ") != 0."),
				*DstAsset, AssetRegistryData.PkgData.DependencyDataOffset, DependenciesArgs.AssetRegistryDependencyDataSize);
			return EResult::ErrorFailedToOpenDestinationFile;
		}

		AssetRegistryData.DependencyDataSectionSize = DependenciesArgs.AssetRegistryDependencyDataSize;
		AssetRegistryData.ImportIndexUsedInGame.Reserve(Summary.ImportCount);
		for (int32 ImportIndex = 0; ImportIndex < Summary.ImportCount; ++ImportIndex)
		{
			AssetRegistryData.ImportIndexUsedInGame.Add(ImportIndex, DependenciesArgs.ImportUsedInGame[ImportIndex]);
		}
		check(SoftPackageReferencesTable.Num() == Summary.SoftPackageReferencesCount); // Constructed above
		AssetRegistryData.SoftPackageReferenceUsedInGame.Reserve(Summary.SoftPackageReferencesCount);
		for (int32 SoftPackageIndex = 0; SoftPackageIndex < Summary.SoftPackageReferencesCount; ++SoftPackageIndex)
		{
			AssetRegistryData.SoftPackageReferenceUsedInGame.Add(SoftPackageReferencesTable[SoftPackageIndex],
				DependenciesArgs.SoftPackageUsedInGame[SoftPackageIndex]);
		}

		AssetRegistryData.ExtraPackageDependencies = MoveTemp(DependenciesArgs.ExtraPackageDependencies);
	}

	return EResult::Success;
}

bool FAssetHeaderPatcherInner::ShouldReplaceMountPoint(const FStringView InPath, FStringView& OutSrcMountPoint, FStringView& OutDstMountPoint)
{
	for (auto& MountPair : StringMountPointReplacements)
	{
		const FStringView SrcMount(MountPair.Key);
		const FStringView DstMount(MountPair.Value);

		if (InPath.StartsWith(SrcMount))
		{
			OutSrcMountPoint = SrcMount;
			OutDstMountPoint = DstMount;
			return true;
		}
	}
	return false;
}

// Note, like DoPatch(FName&) we should strive to remove this method in favour of one that understands
// the context for which this string belongs to. Patching it based on search and replace, is going to be 
// error-prone and should be avoided.
bool FAssetHeaderPatcherInner::DoPatch(FString& InOutString)
{
	auto RedirectString = [this](const FStringView InMaybePath, FString& OutRedirectedString)
		{
			// Strings we will redirect can be made of Package, Outer and Object FNames so we can't handle anything larger than 3 x NAME_SIZE
			if (InMaybePath.Len() >= (NAME_SIZE*3))
			{
				return false;
			}

			FCoreRedirectObjectName SrcName = FString(InMaybePath);
			FCoreRedirectObjectName DstName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_AllMask, SrcName, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
			if (SrcName != DstName)
			{
				OutRedirectedString = DstName.ToString();
				return true;
			}
			else
			{
				FStringView MaybeReplacement = Find(VerseStringReplacements, InMaybePath);
				if (!MaybeReplacement.IsEmpty())
				{
					OutRedirectedString = MaybeReplacement;
					return true;
				}
			}
			return false;
		};

	// Attempt a direct replacement
	if (RedirectString(InOutString, InOutString))
	{
		return true;
	}

	// Direct replacement failed so now try substring replacements

	bool bDidPatch = false;
	TStringBuilder<NAME_SIZE> DstStringBuilder;
	{
		// Patch Object paths with sub-object (not-necessarily quoted)
		// Path occurs to the left of a ":"
		int32 ColonPos;
		FStringView PathView(InOutString);
		while (PathView.FindChar(SUBOBJECT_DELIMITER_CHAR, ColonPos))
		{
			if ((ColonPos + 1) < PathView.Len() && PathView[ColonPos + 1] == SUBOBJECT_DELIMITER_CHAR)
			{
				// "::" is not a path delim
				PathView.RightChopInline(ColonPos + 1);
				continue;
			}

			// Presumably we have found the start of a path's sub-object path. Create a new 
			// view for our possible ObjectPath and walk backwards confirming we are in a path
			// otherwise start over at the next ':'
			FStringView ObjectPathView(PathView.GetData(), ColonPos);

			int32 OuterDelimiterPos;
			if (!ObjectPathView.FindLastChar(TEXT('.'), OuterDelimiterPos))
			{
				// A ':' but '.' before it is not an object path
				PathView.RightChopInline(ColonPos + 1);
				continue;
			}

			int32 LastPathDelimiterPos = INDEX_NONE;
			int32 Index = OuterDelimiterPos;
			while (--Index >= 0)
			{
				if (ObjectPathView[Index] == TEXT('/'))
				{
					LastPathDelimiterPos = Index;
				}
				else
				{
					// Confirm we are still in a path
					int32 PosInvalidChar = 0;
					if (InvalidObjectPathCharacters.FindChar(ObjectPathView[Index], PosInvalidChar))
					{
						break;
					}
				}
			}

			if (LastPathDelimiterPos < 0)
			{
				// No '/' means we aren't in a path
				PathView.RightChopInline(ColonPos + 1);
				continue;
			}

			FStringView SrcMountPoint;
			FStringView DstMountPoint;
			FStringView MaybePath(PathView.GetData() + LastPathDelimiterPos, ColonPos - LastPathDelimiterPos);
			FString MaybeReplacement;
			if (RedirectString(MaybePath, MaybeReplacement))
			{
				FStringView LeftPart(*InOutString, int32(PathView.GetData() - *InOutString) + LastPathDelimiterPos);
				FStringView RightPart(PathView.GetData() + ColonPos);

				DstStringBuilder.Reset();
				DstStringBuilder.Append(LeftPart);
				DstStringBuilder.Append(MaybeReplacement);
				DstStringBuilder.Append(RightPart);

				InOutString = DstStringBuilder.ToString();
				bDidPatch = true;

				// Keep searching until the path is depleted since there might be more than one path to replace
				PathView = FStringView(*InOutString + LeftPart.Len() + MaybeReplacement.Len() + 1);
			}
			else if (ShouldReplaceMountPoint(MaybePath, SrcMountPoint, DstMountPoint))
			{
				FStringView LeftPart(*InOutString, int32(PathView.GetData() - *InOutString) + LastPathDelimiterPos);
				FStringView RightPart(PathView.GetData() + LastPathDelimiterPos + SrcMountPoint.Len());

				DstStringBuilder.Reset();
				DstStringBuilder.Append(LeftPart);
				DstStringBuilder.Append(DstMountPoint);
				DstStringBuilder.Append(RightPart);

				InOutString = DstStringBuilder.ToString();
				bDidPatch = true;

				// Keep searching until the path is depleted since there might be more than one path to replace
				// Skip to the colon since we know we didn't have any matches within the quotes beyond the mount
				PathView = FStringView(*InOutString + ColonPos + 1);
			}
			else
			{
				// No match but keep searching as there may be more than one ':'
				PathView.RightChopInline(ColonPos + 1);
			}
		}
	}

	{
		// Patch quoted paths.
		// Path occurs to the right of the first "'" or """ 
		auto PatchQuotedPath = [this, &DstStringBuilder, &RedirectString](FString& StringToPatch, FStringView Quote)
			{
				int32 FirstQuotePos = INDEX_NONE;
				bool bFoundReplacement = false;
				FStringView PathView(StringToPatch);
				while ((FirstQuotePos = PathView.Find(Quote, 0, ESearchCase::CaseSensitive)) != INDEX_NONE)
				{
					int32 SecondQuotePos = PathView.Find(Quote, FirstQuotePos + 1, ESearchCase::CaseSensitive);
					if (SecondQuotePos == INDEX_NONE)
					{
						// If there isn't a second quote we're done
						break;
					}

					FStringView SrcMountPoint;
					FStringView DstMountPoint;
					FStringView MaybePath = FStringView(PathView.GetData() + FirstQuotePos + 1, SecondQuotePos - FirstQuotePos - 1); // +1 and -1 are to skip the quotes
					FString MaybeReplacement;
					if (RedirectString(MaybePath, MaybeReplacement))
					{
						FStringView LeftPart(*StringToPatch, int32(PathView.GetData() - *StringToPatch) + FirstQuotePos + 1); // +1 to ensure we include the quote
						FStringView RightPart(PathView.GetData() + SecondQuotePos);

						DstStringBuilder.Reset();
						DstStringBuilder.Append(LeftPart);
						DstStringBuilder.Append(MaybeReplacement);
						DstStringBuilder.Append(RightPart);

						StringToPatch = DstStringBuilder.ToString();
						bFoundReplacement = true;

						// Keep searching until the path is depleted since there might be more than one path to replace
						PathView = FStringView(*StringToPatch + LeftPart.Len() + MaybeReplacement.Len() + 1);
					}
					else if (ShouldReplaceMountPoint(MaybePath, SrcMountPoint, DstMountPoint))
					{
						FStringView LeftPart(*StringToPatch, int32(PathView.GetData() - *StringToPatch) + FirstQuotePos + 1); // +1 to ensure we include the quote
						FStringView RightPart(PathView.GetData() + FirstQuotePos + SrcMountPoint.Len() + 1); // +1 to ensure we skip the first quote

						DstStringBuilder.Reset();
						DstStringBuilder.Append(LeftPart);
						DstStringBuilder.Append(DstMountPoint);
						DstStringBuilder.Append(RightPart);

						StringToPatch = DstStringBuilder.ToString();
						bFoundReplacement = true;

						// Keep searching until the path is depleted since there might be more than one path to replace
						// Skip to the end quote since we know we didn't have any matches within the quotes beyond the mount
						PathView = FStringView(*StringToPatch + SecondQuotePos + 1 + (DstMountPoint.Len() - SrcMountPoint.Len())); // Dst - Src to account for new SecondQuotePos after replacement
					}
					else
					{
						// No match but keep searching as there may be more than one quoted path
						PathView.RightChopInline(SecondQuotePos + 1);
					}
				}
				return bFoundReplacement;
			};
		bDidPatch |= PatchQuotedPath(InOutString, TEXT("'"));
		bDidPatch |= PatchQuotedPath(InOutString, TEXT("\""));
	}

	return bDidPatch;
}

bool FAssetHeaderPatcherInner::AddFName(FName DstName)
{
	if (DstName == NAME_None)
	{
		return false;
	}

	FNameEntryId DstComparisonId = DstName.GetDisplayIndex();
	FNameEntryId* RemappedFName = RenameMap.Find(DstComparisonId);
	if (RemappedFName)
	{
		// We hit a case where we thought we needed to change the name in the NameTable
		// but have now discovered some part of the header needs to use the old name. In such a case
		// remove the remap and make it an add instead.
		AddedNames.Add(*RemappedFName);
		RenameMap.Remove(DstComparisonId);
	}
	else
	{
		AddedNames.Add(DstComparisonId);
	}
	UnchangedNames.Add(DstComparisonId);
	return true;
}

bool FAssetHeaderPatcherInner::RemapFName(FName SrcName, FName DstName)
{
	// None won't appear in the NameTable so skip any None passed in here
	if (SrcName == NAME_None)
	{
		return false;
	}
	checkf(DstName != NAME_None, TEXT("There should never be a None FName in the NameTable"));

	// NameTable entries only care about the comparison form (no number) so 
	// only consider that for remapping purposes
	FNameEntryId SrcComparisonId = SrcName.GetDisplayIndex();
	FNameEntryId DstComparisonId = DstName.GetDisplayIndex();

	// Since we do fuzzy matching against AssetRegistryTag data, we might get SrcNames not in the FName table as input; ignore these.
	if (!NameToIndexMap.Contains(SrcComparisonId))
	{
		// If we haven't patched the nametable we still want to flag this name as remapped so 
		// we can ensure we patch header object instances with any changed numbers
		return SrcName.GetNumber() != DstName.GetNumber();
	}

	// Previously it was thought that any FName at the front of the NameTable is an FName used by export data and should not be patched
	// since we can't know the context in which the user uses the FName. This is true, however it's quite common for PropertyTags and other
	// common types to be "ExportData" that refers to paths that will be patched and must stay in sync. As such if we come across one of these
	// paths when patching exports normally, we must replace it. We still do not do a generic walk of the FNameTable for patching without context
	// as that will be error-prone however we allow patching the "ExportData" section of the NameTable when we know there is an overlap with
	// header FNames and ExportData FNames.
	constexpr bool bIsExportDataFName = false; // NameToIndexMap[SrcComparisonId] < Summary.NamesReferencedFromExportDataCount;

	bool bNeedRemap = SrcComparisonId != DstComparisonId;
	if (!bNeedRemap)
	{
		AddFName(SrcName);

		// If we haven't patched the nametable we still want to flag this name as remapped so 
		// we can ensure we patch header object instances with any changed numbers
		return SrcName.GetNumber() != DstName.GetNumber();
	}
	else
	{
		FNameEntryId* RemappedFName = RenameMap.Find(SrcComparisonId);

		bool bForceAddName = bIsExportDataFName || (RemappedFName && *RemappedFName != DstComparisonId) || UnchangedNames.Contains(SrcComparisonId);
		if (bForceAddName)
		{
			// We already have a _different_ remapping, a header entry is still relying on the original name or this is a name used by the Export payload
			// in which case we can't safely modify it. That is fine; we might have used the same FName in more than one place.
			// We need to be certain we are renaming only the names we care about in context. If not, shared names in the NameTable might change
			// incorrectly (e.g. A class FName may have matched a Package FName, thus sharing a NameTable entry, but after patching it's possible _only_ 
			// the Package name has changed. In such a case we don't want to rename the class name inadvertently by patching the shared NameTable entry. 
			// If we have a mismatch with the new patched name, record the new name and we will append it to the NameTable later.
			AddedNames.Add(DstComparisonId);
		}
		else
		{
			RenameMap.Add(SrcComparisonId, DstComparisonId);
		}
		return true;
	}
}

bool FAssetHeaderPatcherInner::DoPatch(FName& InOutName)
{
	// If we are given an FName to patch we have no real context as to what that FName is
	// so we conservatively assume it is a package path and attempt to patch that only
	FCoreRedirectObjectName SrcPackageName(NAME_None, NAME_None, InOutName);
	FCoreRedirectObjectName DstPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, SrcPackageName, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
	if (RemapFName(SrcPackageName.PackageName, DstPackageName.PackageName))
	{
		InOutName = DstPackageName.PackageName;
		return true;
	}
	return false;
}

void FAssetHeaderPatcherInner::GetExportTablePatches(TArray<FExportPatch>& OutExportPatches)
{
	OutExportPatches.Reserve(ExportTable.Num());
	for (int32 i = 0; i < ExportTable.Num(); ++i)
	{
		FObjectExport& Export = ExportTable[i];
		FCoreRedirectObjectName SrcResourceName = GetFullObjectNameFromObjectResource(Export, true /*bIsExport*/);
		FCoreRedirectObjectName DstResourceName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_AllMask, SrcResourceName, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
		if (RemapFName(SrcResourceName.ObjectName, DstResourceName.ObjectName))
		{
			FExportPatch ExportPatch;
			ExportPatch.TableIndex = i;
			ExportPatch.ObjectName = DstResourceName.ObjectName;
			OutExportPatches.Add(MoveTemp(ExportPatch));
		}
	}
}

static FName GetObjectResourceNameFromCoreRedirectObjectName(const FCoreRedirectObjectName& InName)
{
	return !InName.ObjectName.IsNone() ? InName.ObjectName : InName.PackageName;
}

FAssetHeaderPatcher::EResult FAssetHeaderPatcherInner::GetImportTablePatches(TArray<FImportPatch>& OutImportPatches, int32& OutNewImportCount)
{
	OutNewImportCount = 0;
	OutImportPatches.Reserve(ImportTable.Num());
	
	struct FPatchDataForImport
	{
		int32 PatchIndex = INDEX_NONE;
		FCoreRedirectObjectName SrcImportPath;
		FCoreRedirectObjectName DstImportPath;
		bool bPatched = false;
		bool bSkipImportTableWalkForRedirectedOuters = false;
	};
	TArray<FPatchDataForImport> ImportIndexToPatchData;
	ImportIndexToPatchData.SetNum(ImportTable.Num());

	for (int32 ImportIndex = 0; ImportIndex < ImportTable.Num(); ++ImportIndex)
	{
		FObjectImport& Import = ImportTable[ImportIndex];
		FPatchDataForImport& PatchDataForImport = ImportIndexToPatchData[ImportIndex];

		// For each FObjectResource, immediately patch the FNames that do not impact other imports or exports.
		const FCoreRedirectObjectName SrcImportClass(Import.ClassName, NAME_None, Import.ClassPackage);
		const FCoreRedirectObjectName DstImportClass = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Package, SrcImportClass, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);

		Import.ClassName = RemapFName(SrcImportClass.ObjectName, DstImportClass.ObjectName)
			? DstImportClass.ObjectName : SrcImportClass.ObjectName;

		Import.ClassPackage = RemapFName(SrcImportClass.PackageName, DstImportClass.PackageName)
			? DstImportClass.PackageName : SrcImportClass.PackageName;

#if WITH_EDITORONLY_DATA
		const FCoreRedirectObjectName SrcImportPackageName(NAME_None, NAME_None, Import.PackageName);
		const FCoreRedirectObjectName DstImportPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, SrcImportPackageName, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
		Import.PackageName = RemapFName(SrcImportPackageName.PackageName, DstImportPackageName.PackageName)
			? DstImportPackageName.PackageName : SrcImportPackageName.PackageName;
#endif

		// Look up whether there is a specific redirect for the full path of the import.
		PatchDataForImport.SrcImportPath = GetFullObjectNameFromObjectResource(Import, false /*bIsExport*/, false /*bWalkImportsOnly*/);
		PatchDataForImport.DstImportPath = FCoreRedirects::GetRedirectedName(FCoreRedirects::GetFlagsForTypeName(Import.ClassPackage, Import.ClassName), PatchDataForImport.SrcImportPath, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);

		// Record the remapping of the FName in the FObjectResource (whether or not it was redirected)
		FName SrcObjectResourceName = GetObjectResourceNameFromCoreRedirectObjectName(PatchDataForImport.SrcImportPath);
		FName DstObjectResourceName = GetObjectResourceNameFromCoreRedirectObjectName(PatchDataForImport.DstImportPath);
		RemapFName(SrcObjectResourceName, DstObjectResourceName);

		// All objects within a redirected outer are moved along with that outer, but the CoreRedirects system does not find the redirect of the outer unless
		// we look for it specifically. We handle this case in the loop below, after we have processed all imports looking for redirects specific to them.
		PatchDataForImport.bPatched = PatchDataForImport.SrcImportPath != PatchDataForImport.DstImportPath;
		if (PatchDataForImport.bPatched)
		{
			// If our outer was patched (i.e. we had a direct match) or there is no outer but our package changed, ensure we don't use the ImportTable to 
			// check for redirected outers that should apply to this Import as we may incorrectly change an OuterName/PackageName back to the SrcName 
			// if that outer name is still in use after redirection. (e.g. an object moved out of a package to a new package but both packages are still valid imports)
			PatchDataForImport.bSkipImportTableWalkForRedirectedOuters = (PatchDataForImport.SrcImportPath.OuterName != PatchDataForImport.DstImportPath.OuterName) ||
				(PatchDataForImport.DstImportPath.OuterName.IsNone() && PatchDataForImport.SrcImportPath.PackageName != PatchDataForImport.DstImportPath.PackageName);
			PatchDataForImport.PatchIndex = OutImportPatches.Num();

			FImportPatch& ImportPatch = OutImportPatches.Emplace_GetRef();
			ImportPatch.ClassName = Import.ClassName;
			ImportPatch.ClassPackage = Import.ClassPackage;
#if WITH_EDITORONLY_DATA
			ImportPatch.PackageName = Import.PackageName;
#endif
			bool* UsedInGame = AssetRegistryData.ImportIndexUsedInGame.Find(ImportIndex);
			ImportPatch.bUsedInGame = UsedInGame ? *UsedInGame : true;
			ImportPatch.TableIndex = ImportIndex;
			ImportPatch.ObjectName = DstObjectResourceName;
			ImportPatch.OuterIndex = Import.OuterIndex;
		}
	}

	// Now that redirects specifically for each import have been detected, go through the list of imports again and assign
	// DstImportPath based on their (possibly redirected) outer.
	for (int32 ImportIndex = 0; ImportIndex < ImportTable.Num(); ++ImportIndex)
	{
		FObjectImport& Import = ImportTable[ImportIndex];
		FPatchDataForImport& PatchDataForImport = ImportIndexToPatchData[ImportIndex];

		if (!PatchDataForImport.bSkipImportTableWalkForRedirectedOuters)
		{
			// Recursively evaluate all outers of the import, and then evaluate the import
			TArray<int32, TInlineAllocator<10>> ImportsToEvaluate;
			ImportsToEvaluate.Add(ImportIndex);
			for (FPackageIndex OuterIndex = Import.OuterIndex; !OuterIndex.IsNull(); )
			{
				if (OuterIndex.IsExport())
				{
					// We don't have data on the export, so stop.
					// TODO: This is incorrect, we need to get the remapped destination from the export to know
					// the correct destination for the import.
					break;
				}
				check(OuterIndex.IsImport());
				int32 OuterImportIndex = OuterIndex.ToImport();
				FObjectImport& OuterImport = ImportTable[OuterImportIndex];
				FPatchDataForImport& PatchDataForOuter = ImportIndexToPatchData[OuterImportIndex];
				if (PatchDataForOuter.bSkipImportTableWalkForRedirectedOuters)
				{
					break;
				}
				ImportsToEvaluate.Add(OuterImportIndex);
				OuterIndex = OuterImport.OuterIndex;
			}

			while (!ImportsToEvaluate.IsEmpty())
			{
				int32 ImportIndexToEvaluate = ImportsToEvaluate.Pop(EAllowShrinking::No);
				FObjectImport& ImportToEvaluate = ImportTable[ImportIndexToEvaluate];
				FPatchDataForImport& PatchDataToEvaluate = ImportIndexToPatchData[ImportIndexToEvaluate];
				if (PatchDataToEvaluate.bSkipImportTableWalkForRedirectedOuters)
				{
					// Shouldn't be possible, but somehow we already evaluated this, cycle in the outer chain?
					continue;
				}

				if (ImportToEvaluate.OuterIndex.IsNull())
				{
					// This import has no outer, it is a package. We did not find a redirect specifically for it,
					// and it has no outer to move along with, so it has no redirect. Mark it skipped and do nothing.
					PatchDataToEvaluate.bSkipImportTableWalkForRedirectedOuters = true;
				}
				else if (ImportToEvaluate.OuterIndex.IsExport())
				{
					// We don't have data on the export, so we cannot read the transformed path for the import under it.
					// TODO: This is incorrect, we need to get the remapped destination from the export to know
					// the correct destination for the import.
					PatchDataToEvaluate.bSkipImportTableWalkForRedirectedOuters = true;
				}
				else
				{
					int32 OuterImportIndex = ImportToEvaluate.OuterIndex.ToImport();
					FPatchDataForImport& PatchDataForOuter = ImportIndexToPatchData[OuterImportIndex];
					PatchDataToEvaluate.DstImportPath = FCoreRedirectObjectName::AppendObjectName(
						PatchDataForOuter.DstImportPath, PatchDataToEvaluate.DstImportPath.ObjectName);
					PatchDataToEvaluate.bSkipImportTableWalkForRedirectedOuters = true;
				}
			}
		}

		check(PatchDataForImport.bSkipImportTableWalkForRedirectedOuters);

		// Record that the (possibly redirected) full path of the destination of this import is in this import index
		ImportNameToImportTableIndexLookup.Add(PatchDataForImport.DstImportPath, ImportIndex);
		ImportTablePatchedNames.Add({ PatchDataForImport.SrcImportPath, PatchDataForImport.DstImportPath });
	}

	// Loop over all imports, and update their outerindex. If an import for their outer does not already exist, add it on to the end.
	// Keep iterating over the added elements that we add on to the end until we don't add any more.
	for (int32 ImportIndex = 0;
		ImportIndex < ImportIndexToPatchData.Num(); // Num can change during the loop
		++ImportIndex)
	{
		FCoreRedirectObjectName ImportPath = ImportIndexToPatchData[ImportIndex].DstImportPath;
		// Do not create a pointer into ImportIndexToPatchData after we have done any necessary adds into ImportIndexToPatchData
		FPatchDataForImport* PatchDataForImport = nullptr;

		FPackageIndex DstOuterIndex;
		if (ImportPath.ObjectName.IsNone())
		{
			// A package, no outer
			DstOuterIndex = FPackageIndex();
		}
		else
		{
			FCoreRedirectObjectName OuterPath = FCoreRedirectObjectName::GetParent(ImportPath);
			if (OuterPath.PackageName == this->DstPackagePath)
			{
				// The outer of the import is an export (this happens when the import is an external actor package,
				// and the external actor is a child of a ULevel in this map package).
				// Keep the old OuterIndex. TODO: Read the correct package path from a map that we construct from remapped export name to export index,
				// so that we can handle replacing the outerindex for new outer imports
				PatchDataForImport = &ImportIndexToPatchData[ImportIndex];
				if (PatchDataForImport->bPatched)
				{
					DstOuterIndex = OutImportPatches[PatchDataForImport->PatchIndex].OuterIndex;
				}
				else
				{
					// The only unpatched imports have to be ones from the ImportTable of the pretransformed package
					check(ImportIndex < ImportTable.Num());
					DstOuterIndex = ImportTable[ImportIndex].OuterIndex;
				}
			}
			else
			{
				// The outer of an import is an import, and we need to find or add it in our importtable
				int32& ExistingOuterIndex = ImportNameToImportTableIndexLookup.FindOrAdd(OuterPath, INDEX_NONE);
				if (ExistingOuterIndex == INDEX_NONE)
				{
					// If the outer is not already in the list of imports, add on another FImportPatch to hold the outer
					int32 OuterPatchIndex = OutImportPatches.Num();
					FImportPatch& OuterPatch = OutImportPatches.Emplace_GetRef();
					int32 OuterImportIndex = ImportIndexToPatchData.Num();
					FPatchDataForImport& OuterPatchData = ImportIndexToPatchData.Emplace_GetRef();
					OuterPatchData.bPatched = true;
					OuterPatchData.DstImportPath = OuterPath;
					OuterPatchData.PatchIndex = OuterPatchIndex;
					ExistingOuterIndex = OuterImportIndex;

					if (OuterPath.ObjectName.IsNone())
					{
						// Outer is a package
						OuterPatch.ClassName = FName(TEXT("Package"));
						OuterPatch.ClassPackage = FName(TEXT("/Script/CoreUObject"));
					}
					else
					{
						// Outer is an object inside a package.
						// We don't know the class name and package of the outer, so set it to /Script/CoreUObject.Object.
						// TODO: Is there anyway to find this out? A better guess is to copy it from the previous outer.
						OuterPatch.ClassName = FName(TEXT("Object"));
						OuterPatch.ClassPackage = FName(TEXT("/Script/CoreUObject"));
					}
					AddFName(OuterPatch.ClassName);
					AddFName(OuterPatch.ClassPackage);
#if WITH_EDITORONLY_DATA
					// We don't have any information about whether the outer object has an external package. For now we
					// just don't support that case for redirects, and assume the outer is not in an external package.
					OuterPatch.PackageName = NAME_None;
#endif
					// Set the Outer to UsedInGame if the import that has it as the outer is UsedInGame.
					PatchDataForImport = &ImportIndexToPatchData[ImportIndex];
					if (PatchDataForImport->bPatched)
					{
						OuterPatch.bUsedInGame = OutImportPatches[PatchDataForImport->PatchIndex].bUsedInGame;
					}
					else
					{
						bool* UsedInGame = AssetRegistryData.ImportIndexUsedInGame.Find(ImportIndex);
						OuterPatch.bUsedInGame = UsedInGame ? *UsedInGame : true;
					}
					OuterPatch.TableIndex = OuterImportIndex;
					OuterPatch.ObjectName = GetObjectResourceNameFromCoreRedirectObjectName(OuterPath);
					AddFName(OuterPatch.ObjectName);
					// OuterIndex of the outer is not yet known. Set to null. If the outer is a package this is correct,
					// otherwise it is incorrect and we have to overwrite it later when we reach the outer in our iteration over
					// ImportIndexToPatchData.
					OuterPatch.OuterIndex = FPackageIndex();
				}
				DstOuterIndex = FPackageIndex::FromImport(ExistingOuterIndex);
			}
		}

		// If we have already patched, just assign the outer index.
		PatchDataForImport = &ImportIndexToPatchData[ImportIndex];
		if (PatchDataForImport->bPatched)
		{
			OutImportPatches[PatchDataForImport->PatchIndex].OuterIndex = DstOuterIndex;
		}
		else
		{
			// The only unpatched imports have to be ones from the ImportTable of the pretransformed package
			check(ImportIndex < ImportTable.Num());
			FObjectImport& Import = ImportTable[ImportIndex];

			// If the outer has changed and we have not already patched, turn this into a patch.
			if (Import.OuterIndex != DstOuterIndex)
			{
				PatchDataForImport->bPatched = true;
				PatchDataForImport->PatchIndex = OutImportPatches.Num();
				FImportPatch& ImportPatch = OutImportPatches.Emplace_GetRef();
				ImportPatch.ClassName = Import.ClassName;
				ImportPatch.ClassPackage = Import.ClassPackage;
#if WITH_EDITORONLY_DATA
				ImportPatch.PackageName = Import.PackageName;
#endif
				bool* UsedInGame = AssetRegistryData.ImportIndexUsedInGame.Find(ImportIndex);
				ImportPatch.bUsedInGame = UsedInGame ? *UsedInGame : true;
				ImportPatch.TableIndex = ImportIndex;
				ImportPatch.ObjectName = Import.ObjectName;
				ImportPatch.OuterIndex = DstOuterIndex;
			}
		}
	}

	OutNewImportCount = ImportIndexToPatchData.Num() - ImportTable.Num();
	return FAssetHeaderPatcher::EResult::Success;
}

void FAssetHeaderPatcherInner::PatchExportAndImportTables(const TArray<FExportPatch>& InExportPatches, const TArray<FImportPatch>& InImportPatches, const int32 InNewImportCount)
{
	// Any FPackageIndex that IsExport() will not change since we do not add or remove entries from the ExportTable.
	// For Imports we may have changed import names such that we resulted in new imports and now imports that are no longer
	// in use and should be removed (to avoid unnecessary overhead when loading). However, as far as I can tell we can't 
	// determine what to remove based on what is used since we append extra imports during serialization that don't have any
	// Exports or Imports referring to them (e.g. CDO subobjects are like this). As such we append our import patches as new imports
	// if we can't stomp over existing entries in the table. We then fixup any FPackageIndex members that might be using old indices that we
	// know we have explicitly changed due to additions.

	ImportTable.SetNum(ImportTable.Num() + InNewImportCount);
	for (const FImportPatch& ImportPatch : InImportPatches)
	{
		const int32 Index = ImportPatch.TableIndex;
		check(Index < ImportTable.Num());

		FObjectImport& Import = ImportTable[Index];
		Import.ObjectName = ImportPatch.ObjectName;
		Import.OuterIndex = ImportPatch.OuterIndex;
		Import.ClassName = ImportPatch.ClassName;
		Import.ClassPackage = ImportPatch.ClassPackage;
#if WITH_EDITORONLY_DATA
		Import.OldClassName = NAME_None;
		Import.PackageName = ImportPatch.PackageName;
#endif
		AssetRegistryData.ImportIndexUsedInGame.Add(Index, ImportPatch.bUsedInGame);
	}

	for (const FExportPatch& ExportPatch : InExportPatches)
	{
		// Patching cannot result in new Exports
		const int32 Index = ExportPatch.TableIndex;
		check(Index < ExportTable.Num());

		FObjectExport& Export = ExportTable[Index];
		Export.ObjectName = ExportPatch.ObjectName;
#if WITH_EDITORONLY_DATA
		Export.OldClassName = NAME_None;
#endif
	}

	// Walk through our Exports and ensure any FPackageIndex they have is pointing to the correct location
	auto RemapIndex = [this](FPackageIndex& Index)
		{
			if (Index.IsImport())
			{
				FCoreRedirectObjectName PatchedName = ImportTablePatchedNames[Index.ToImport()].Value;
				Index = FPackageIndex::FromImport(ImportNameToImportTableIndexLookup[PatchedName]);
			}
		};

	for (FObjectExport& Export : ExportTable)
	{
		RemapIndex(Export.ClassIndex);
		RemapIndex(Export.SuperIndex);
		RemapIndex(Export.TemplateIndex);
		RemapIndex(Export.OuterIndex);
	}

	// We may have added new Imports so ensure the Summary is accurate
	Summary.ImportCount = ImportTable.Num();
}

void FAssetHeaderPatcherInner::PatchNameTable()
{
	// Note, no number is assigned when replacing FNames as the NameTable only tracks unnumbered names

	// Update the NameTable with the known patched values and add our new patched names to the NameToIndex
	// map so we can validate that we always have a FName mapping to an entry in the name table when writing
	for (auto& Pair : RenameMap)
	{
		FNameEntryId SrcName = Pair.Key;
		FNameEntryId DstName = Pair.Value;
		int32* pSrcIndex = NameToIndexMap.Find(SrcName);
		checkf(pSrcIndex && *pSrcIndex < NameTable.Num(), TEXT("An FName remapping was done for a name (%s) not in the NameTable."),
			*FName::CreateFromDisplayId(DstName, NAME_NO_NUMBER_INTERNAL).ToString());
		int32 SrcIndex = *pSrcIndex;

		NameTable[SrcIndex] = FName::CreateFromDisplayId(DstName, NAME_NO_NUMBER_INTERNAL);
		NameToIndexMap.Remove(SrcName);
		NameToIndexMap.Add(DstName, SrcIndex);
	}

	for (FNameEntryId NewName : AddedNames)
	{
		if (NameToIndexMap.Contains(NewName))
		{
			// Definition of an AddedName is that an original name was remapped to a name, and we couldn't remove the
			// original name for one of several reasons, so we want to add the remapped name to the nametable.
			// But it is possible that remapped name already exists, so we don't need to add it.
			continue;
		}
		FName NewFName = FName::CreateFromDisplayId(NewName, NAME_NO_NUMBER_INTERNAL);
		int32 NameTableIndex = NameTable.Num();

		NameTable.Add(NewFName);
		NameToIndexMap.Add(NewName, NameTableIndex);
	}

	Summary.NameCount = NameTable.Num();
}

bool FAssetHeaderPatcherInner::DoPatch(FSoftObjectPath& InOutSoft)
{
	// FSoftObjectPaths are special in that while we may have an explict remapping 
	// provided to the patcher, there may also be redirects stored in the global GRedirectCollector 
	// reflecting on-disk object redirectors.
	// Redirectors will be applied during FSoftObjectPath serialization so we need to account for
	// that when patching so the nametable can be updated appropriately.

	// Honour explictly remapped paths first. Even if this succeeds we need to handle redirectors
	// because FSoftObjectPath serialization will always apply them.

	// Special care is needed to check to see if a patch is done via remapping or via redirects before we assume the 
	// name should be marked as unchanging (causing any future patching to the name to be an add in the NameTable). If we have a redirect 
	// but not remapping, the failed remapping could marked the srcname as unchanging erroneously.
	FTopLevelAssetPath SrcTopLevelAssetPath = InOutSoft.GetAssetPath();
	const FCoreRedirectObjectName DstTopLevelAssetPathObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_AllMask, SrcTopLevelAssetPath, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
	FTopLevelAssetPath DstTopLevelAssetPath(DstTopLevelAssetPathObjectName.ToString());
	bool bPatched = SrcTopLevelAssetPath != DstTopLevelAssetPath;
	if (bPatched)
	{
		InOutSoft.SetPath(DstTopLevelAssetPath, InOutSoft.GetSubPathString());
	}

// Only run in Editor builds as FSoftObjectPath only runs PreSavePath during SerializePath in editor builds
#if WITH_EDITOR
	if (InOutSoft.PreSavePath(nullptr))
	{
		bPatched = true;
		DstTopLevelAssetPath = InOutSoft.GetAssetPath();
	}
#endif

	if (bPatched)
	{
		RemapFName(SrcTopLevelAssetPath.GetAssetName(), DstTopLevelAssetPath.GetAssetName());
		RemapFName(SrcTopLevelAssetPath.GetPackageName(), DstTopLevelAssetPath.GetPackageName());
	}

	return bPatched;
}

FCoreRedirectObjectName FAssetHeaderPatcherInner::GetFullObjectNameFromObjectResource(const FObjectResource& InResource, bool bIsExport, bool bWalkImportsOnly)
{
	bool bOutermostIsExport = bIsExport;
	FPackageIndex OuterIndex = InResource.OuterIndex;
	TArray<FName, TInlineAllocator<8>> OuterStack;
	while (!OuterIndex.IsNull())
	{
		const FObjectResource* OuterResource;
		if (OuterIndex.IsImport())
		{
			bOutermostIsExport = false;
			OuterResource = &ImportTable[OuterIndex.ToImport()];
		}
		else if (bWalkImportsOnly)
		{
			break;
		}
		else
		{
			bOutermostIsExport = true;
			OuterResource = &ExportTable[OuterIndex.ToExport()];
		}

		OuterStack.Push(OuterResource->ObjectName);
		OuterIndex = OuterResource->OuterIndex;
	}

	FName SrcObjectName;
	FName SrcOuterName;
	FName SrcPackageName;
	bool bRemapByPackage = false;
	if (OuterStack.Num() == 0)
	{
		if (bOutermostIsExport)
		{
			SrcPackageName = OriginalPackagePath;	// /Package/Package
			SrcOuterName = NAME_None;
			SrcObjectName = InResource.ObjectName;	// MyObject
		}
		else
		{
			// The ObjectName is a package
			SrcPackageName = InResource.ObjectName;	// /Package/Package
			SrcOuterName = NAME_None;
			SrcObjectName = NAME_None;
			bRemapByPackage = true;
		}
	}
	else
	{
		SrcPackageName = bOutermostIsExport ? OriginalPackagePath : OuterStack.Pop();

		TStringBuilder<NAME_SIZE> OuterString;
		while (!OuterStack.IsEmpty())
		{
			FName Outer = OuterStack.Pop();
			Outer.AppendString(OuterString);
			OuterString.AppendChar(TEXT('.'));
		}
		if (OuterString.Len())
		{
			OuterString.RemoveSuffix(1);
		}
		SrcOuterName = FName(OuterString);
		SrcObjectName = InResource.ObjectName;
	}
	
	return FCoreRedirectObjectName(SrcObjectName, SrcOuterName, SrcPackageName);
}

bool FAssetHeaderPatcherInner::DoPatch(FTopLevelAssetPath& InOutPath)
{
	const FCoreRedirectObjectName SrcTopLevelAssetPath = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_AllMask, InOutPath, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
	const FTopLevelAssetPath DstTopLevelAssetPath(SrcTopLevelAssetPath.ToString());

	bool bPatched = RemapFName(InOutPath.GetAssetName(), DstTopLevelAssetPath.GetAssetName());
	bPatched |= RemapFName(InOutPath.GetPackageName(), DstTopLevelAssetPath.GetPackageName());

	if (bPatched)
	{
		InOutPath = DstTopLevelAssetPath;
	}

	return bPatched;
}

bool FAssetHeaderPatcherInner::DoPatch(FGatherableTextData& InOutGatherableTextData)
{
	// There are various fields in FGatherableTextData however only one pertains to 
	// asset paths and types, SourceSiteContexts.SiteDescription. The rest are contextual
	// key-value pairs of text which are not references to assets/types and thus do not need patching
	// (at least we can't understand the context a priori to know if specialized code
	// may try to load from these strings)

	bool bDidPatch = false;
	for (FTextSourceSiteContext& SourceSiteContext : InOutGatherableTextData.SourceSiteContexts)
	{
		FStringView ClassName;
		FStringView PackagePath;
		FStringView ObjectName;
		FStringView SubObjectName;
		FPackageName::SplitFullObjectPath(SourceSiteContext.SiteDescription, ClassName, PackagePath, ObjectName, SubObjectName, true /*bDetectClassName*/);

		// Todo to use StringView logic above to reduce string copies
		FSoftObjectPath SiteDescriptionPath(SourceSiteContext.SiteDescription);
		if (!SiteDescriptionPath.IsValid())
		{
			continue;
		}

		FTopLevelAssetPath TopLevelAssetPath = SiteDescriptionPath.GetAssetPath();
		const FCoreRedirectObjectName RedirectedTopLevelAssetPath = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_AllMask, TopLevelAssetPath, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
		const FTopLevelAssetPath PatchedTopLevelAssetPath(RedirectedTopLevelAssetPath.ToString());
		if (TopLevelAssetPath == PatchedTopLevelAssetPath)
		{
			continue;
		}
		bDidPatch = true;
		SiteDescriptionPath.SetPath(PatchedTopLevelAssetPath, SiteDescriptionPath.GetSubPathString());
		SourceSiteContext.SiteDescription = SiteDescriptionPath.ToString();
	}
	
	return bDidPatch;
}

bool FAssetHeaderPatcherInner::DoPatch(FThumbnailEntry& InThumbnailEntry)
{
	// These objects can potentially be paths to sub-objects. For renaming purposes we 
	// want to drop the sub-object path and grab the AssetName
	FStringView SrcObjectPathWithoutPackageName(InThumbnailEntry.ObjectPathWithoutPackageName);
	int32 ColonPos = INDEX_NONE;
	if (SrcObjectPathWithoutPackageName.FindChar(TEXT(':'), ColonPos))
	{
		SrcObjectPathWithoutPackageName.LeftChopInline(SrcObjectPathWithoutPackageName.Len() - ColonPos);
	}

	FName PackageFName = OriginalPackagePath;
	if (bIsNonOneFilePerActorPackage)
	{
		PackageFName = OriginalNonOneFilePerActorPackagePath;
	}

	const FCoreRedirectObjectName SrcTopLevelAssetName(FName(SrcObjectPathWithoutPackageName), NAME_None, PackageFName);
	const FCoreRedirectObjectName RedirectedTopLevelAssetName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Object, SrcTopLevelAssetName, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
	bool bPatched = RemapFName(SrcTopLevelAssetName.ObjectName, RedirectedTopLevelAssetName.ObjectName);

	const FCoreRedirectObjectName SrcClassName(FName(InThumbnailEntry.ObjectShortClassName), NAME_None, NAME_None);
	const FCoreRedirectObjectName RedirectedClassName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, SrcClassName, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
	bPatched |= RemapFName(SrcClassName.ObjectName, RedirectedClassName.ObjectName);

	if (bPatched)
	{
		InThumbnailEntry.ObjectShortClassName = RedirectedClassName.ObjectName.ToString();
		InThumbnailEntry.ObjectPathWithoutPackageName = RedirectedTopLevelAssetName.ObjectName.ToString();
	}

	return bPatched;
}

bool FAssetHeaderPatcherInner::DoPatch(UE::Serialization::Private::FTypeResource& InOutTypeResource)
{
	auto PatchObjectName = [](FName& Name, const FCoreRedirectObjectName& Src, const FCoreRedirectObjectName& Dst, FAssetHeaderPatcherInner& Patcher) -> bool
	{
		if (Patcher.RemapFName(Src.ObjectName, Dst.ObjectName))
		{
			Name = Dst.ObjectName;
			return true;
		}
		else
		{
			Name = Src.ObjectName;
			return false;
		}
	};

	auto PatchPackageName = [](FName& Name, const FCoreRedirectObjectName& Src, const FCoreRedirectObjectName& Dst, FAssetHeaderPatcherInner& Patcher) -> bool
	{
		if (Patcher.RemapFName(Src.PackageName, Dst.PackageName))
		{
			Name = Dst.PackageName;
			return true;
		}
		else
		{
			Name = Src.PackageName;
			return false;
		}
	};

	bool bDidPatch = false;

	// Note that even though we are patching the super types stored in the type hierarchy, the *actual* type hierarchy
	// might be different (because e.g., types have moved). Note however, that the type hierarchy itself is a snapshot
	// of what the import types looked liked at the point the asset was saved. For this reason, we don't attempt to
	// create the type hierarchy again here. This will be done the next type the patched asset is saved.

	const FCoreRedirectObjectName SrcClass(InOutTypeResource.ClassName, NAME_None, InOutTypeResource.ClassPackageName);
	const FCoreRedirectObjectName DstClass = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Package, SrcClass, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);

	bDidPatch |= PatchObjectName(InOutTypeResource.ClassName, SrcClass, DstClass, *this);
	bDidPatch |= PatchPackageName(InOutTypeResource.ClassPackageName, SrcClass, DstClass, *this);

	const FCoreRedirectObjectName SrcObj(InOutTypeResource.TypeName, NAME_None, InOutTypeResource.PackageName);
	const FCoreRedirectObjectName DstObj = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Object | ECoreRedirectFlags::Type_Package, SrcObj, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);

	bDidPatch |= PatchObjectName(InOutTypeResource.TypeName, SrcObj, DstObj, *this);
	bDidPatch |= PatchPackageName(InOutTypeResource.PackageName, SrcObj, DstObj, *this);

	return bDidPatch;
}

FAssetHeaderPatcher::EResult FAssetHeaderPatcherInner::PatchHeader_PatchSections()
{
	// Package Summary
	{
		const FCoreRedirectObjectName DstPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package,
			FCoreRedirectObjectName(NAME_None, NAME_None, OriginalPackagePath), ECoreRedirectMatchFlags::DisallowPartialLHSMatch);

		// This is a string, so we do not want to Remap the patched name unless it's a non-OFPA
		// package, in which case there will be an FName entry for this path
		Summary.PackageName = DstPackageName.PackageName.ToString();
		DstPackagePath = DstPackageName.PackageName;
		
		// It seems that non-OFPA packages tend to have the package name in the nametable, 
		// however it isn't a guarantee, so we confirm the name is there before remapping and
		// extend this special case of NameTable patching to all packages, OFPA or not.
		if (NameToIndexMap.Find(OriginalPackagePath.GetDisplayIndex()))
		{
			bIsPackagePathInNametable = true;
			RemapFName(OriginalPackagePath, DstPackageName.PackageName);
		}
	}

	// For Import and Export Tables we need to generate patches for both and apply them afterwards. This is due to the ObjectPaths
	// being patched being split across multiple Export and Import entries that refer to one another. Patching them as we go would
	// change the ObjectPaths of Exports/Imports we may need to patch but won't be able to deduce the original, unpatched ObjectPath.
	TArray<FExportPatch> ExportPatches;
	int32 NewImportCount = 0;
	TArray<FImportPatch> ImportPatches;
	GetExportTablePatches(ExportPatches);
	FAssetHeaderPatcher::EResult Result = GetImportTablePatches(ImportPatches, NewImportCount);
	if (Result != FAssetHeaderPatcher::EResult::Success)
	{
		return Result;
	}
	PatchExportAndImportTables(ExportPatches, ImportPatches, NewImportCount);

	// Soft paths
	for (FSoftObjectPath& SoftObjectPath : SoftObjectPathTable)
	{
		DoPatch(SoftObjectPath);
	}

	// GatherableTextData table
	for (FGatherableTextData& GatherableTextData : GatherableTextDataTable)
	{
		DoPatch(GatherableTextData);
	}

	// Soft Package References
	for (FName& Reference : SoftPackageReferencesTable)
	{
		FCoreRedirectObjectName SrcPackagePath(NAME_None, NAME_None, Reference);
		FCoreRedirectObjectName DstPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, SrcPackagePath, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
		bool* UsedInGame = AssetRegistryData.SoftPackageReferenceUsedInGame.Find(SrcPackagePath.PackageName);
		bool bUsedInGame = UsedInGame ? *UsedInGame : true;
		AssetRegistryData.SoftPackageReferenceUsedInGame.Add(DstPackageName.PackageName, bUsedInGame);
		if (RemapFName(SrcPackagePath.PackageName, DstPackageName.PackageName))
		{
			Reference = DstPackageName.PackageName;
		}
	}

	// SearchableNamesMap
	for (auto& Pair : SearchableNamesMap)
	{
		TArray<FName>& Names = Pair.Value;
		for (FName& Name : Names)
		{
			DoPatch(Name);
		}
	}

	// ImportTypeHierarchy Map
	for (auto& Pair : ImportTypeHierarchyMap)
	{
		UE::Serialization::Private::FImportTypeHierarchy& Entry = Pair.Value;

		const TArray<UE::Serialization::Private::FTypeResource>& SuperTypes = Entry.GetSuperTypes();

		for (const UE::Serialization::Private::FTypeResource& SuperType : SuperTypes)
		{
			UE::Serialization::Private::FTypeResource& WritableSuperType = const_cast<UE::Serialization::Private::FTypeResource&>(SuperType);

			DoPatch(WritableSuperType);
		}
	}

	// Thumbnail Table
	for (FThumbnailEntry& ThumbnailEntry : ThumbnailTable)
	{
		DoPatch(ThumbnailEntry);
	}

	// Asset Registry Data
	for (FAssetRegistryObjectData& ObjData : AssetRegistryData.ObjectData)
	{
		// ObjectPath is a toss-up. 
		// Sometimes it's a FTopLevelAssetPath with an implied PackageName (this package's name) and AssetName.
		// Sometimes it's a full FSoftPath (e.g. when dealing with ExternalObjects)
		FSoftObjectPath SrcObjectPath(ObjData.ObjectData.ObjectPath);
		{
			if (SrcObjectPath.IsValid())
			{
				FSoftObjectPath SrdDstObjectPath = SrcObjectPath;

				if (DoPatch(SrdDstObjectPath))
				{
					ObjData.ObjectData.ObjectPath = SrdDstObjectPath.ToString();
				}
			}
			// Only use OriginalPackagePath if it was in the nametable to begin with, 
			// since if it wasn't we can't have a remapping for the ObjectPath
			else if(bIsPackagePathInNametable)
			{
				FTopLevelAssetPath SrcDstTopLevelAssetPath(OriginalPackagePath, FName(ObjData.ObjectData.ObjectPath));
				SrcObjectPath.SetPath(SrcDstTopLevelAssetPath, SrcObjectPath.GetSubPathString());

				if (DoPatch(SrcDstTopLevelAssetPath))
				{
					ObjData.ObjectData.ObjectPath = SrcDstTopLevelAssetPath.GetAssetName().ToString();
				}
			}
		}

		// ObjData.ObjectData.ObjectClassName is a FTopLevelAssetPath stored as a string
		FTopLevelAssetPath SrcObjectClassName(ObjData.ObjectData.ObjectClassName);
		{
			FTopLevelAssetPath SrcDstObjectClassName = SrcObjectClassName;

			if (DoPatch(SrcDstObjectClassName))
			{
				ObjData.ObjectData.ObjectClassName = SrcDstObjectClassName.ToString();
			}
		}

		for (UE::AssetRegistry::FDeserializeTagData& TagData : ObjData.TagData)
		{
			if (IgnoredTags.Contains(TagData.Key))
			{
				continue;
			}

			// WorldPartitionActor metadata is special. It's an encoded string blob which needs
			// handling internally, so we make use of a custom patcher to let us intercept
			// various elements that might need patching.
			if (TagData.Key == FWorldPartitionActorDescUtils::ActorMetaDataTagName())
			{
				const FString LongPackageName(SrcAsset);
				const FString ObjectPath(ObjData.ObjectData.ObjectPath);
				const FTopLevelAssetPath AssetClassPathName(ObjData.ObjectData.ObjectClassName);
				const FAssetDataTagMap Tags(MakeTagMap(ObjData.TagData));
				const FAssetData AssetData(LongPackageName, ObjectPath, AssetClassPathName, Tags);

				struct FWorldPartitionAssetDataPatcherInner : FWorldPartitionAssetDataPatcher
				{
					FWorldPartitionAssetDataPatcherInner(FAssetHeaderPatcherInner* InInner) : Inner(InInner) {}
					virtual bool DoPatch(FString& InOutString) override 
					{ 
						return Inner->DoPatch(InOutString); 
					}
					virtual bool DoPatch(FName& InOutName) override 
					{ 
						// FNames are actually strings inside WorldPartitionActor metadata, and since a lone
						// FName has no context for how to patch it, convert it to a string to perform a 
						// best-effort search.
						FString NameString;
						InOutName.ToString(NameString);
						if (Inner->DoPatch(NameString))
						{
							InOutName = FName(NameString);
							return true;
						}
						return false;
					}
					virtual bool DoPatch(FSoftObjectPath& InOutSoft) override 
					{
						return Inner->DoPatch(InOutSoft);
					}
					virtual bool DoPatch(FTopLevelAssetPath& InOutPath) override 
					{ 
						return Inner->DoPatch(InOutPath);
					}
					FAssetHeaderPatcherInner* Inner;
				};

				FString PatchedAssetData;
				FWorldPartitionAssetDataPatcherInner Patcher(this);
				if (FWorldPartitionActorDescUtils::GetPatchedAssetDataFromAssetData(AssetData, PatchedAssetData, &Patcher))
				{
					TagData.Value = PatchedAssetData;
				}
			}
			// Special case for common Tag
			else if (bPatchPrimaryAssetTag && TagData.Key == TEXT("PrimaryAssetName"))
			{
				if (TagData.Value == OriginalPrimaryAssetName)
				{
					const FCoreRedirectObjectName DstPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package,
						FCoreRedirectObjectName(NAME_None, NAME_None, OriginalPackagePath), ECoreRedirectMatchFlags::DisallowPartialLHSMatch);

					TStringBuilder<256> Builder;
					DstPackageName.PackageName.ToString(Builder);
					FStringView PrimaryAssetView = Builder.ToView();
					ensure(PrimaryAssetView.Len() && PrimaryAssetView[0] == TEXT('/'));
					PrimaryAssetView.RemovePrefix(1);

					int32 SlashPos = INDEX_NONE;
					if (PrimaryAssetView.FindChar(TEXT('/'), SlashPos))
					{
						TagData.Value.Empty();
						TagData.Value.Append(PrimaryAssetView.GetData(), SlashPos);
					}
				}
			}
			else
			{
				DoPatch(TagData.Value);
			}
		}
	}

	// AssetRegistryDependencyData
	{
		using namespace UE::AssetRegistry;

		// Move into a TMap to avoid O(n^2) insertions/removals
		TMap<FName, EExtraDependencyFlags> Dependencies;
		Dependencies.Reserve(AssetRegistryData.ExtraPackageDependencies.Num());
		for (const TPair<FName, EExtraDependencyFlags>& Pair : AssetRegistryData.ExtraPackageDependencies)
		{
			Dependencies.Add(Pair.Key, Pair.Value);
		}

		TArray<TPair<FName, EExtraDependencyFlags>> AddedKeys;
		TSet<FName> RemovedKeys;
		for (const TPair<FName, EExtraDependencyFlags>& Pair : Dependencies)
		{
			FCoreRedirectObjectName SrcPackagePath(NAME_None, NAME_None, Pair.Key);
			FCoreRedirectObjectName DstPackageName =
				FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, SrcPackagePath, ECoreRedirectMatchFlags::DisallowPartialLHSMatch);
			if (RemapFName(SrcPackagePath.PackageName, DstPackageName.PackageName))
			{
				AddedKeys.Add({ DstPackageName.PackageName, Pair.Value});
				RemovedKeys.Add(SrcPackagePath.PackageName);
			}
		}
		if (!AddedKeys.IsEmpty() || !RemovedKeys.IsEmpty())
		{
			for (TPair<FName, EExtraDependencyFlags>& Pair : AddedKeys)
			{
				// If the added name already existed, take the union of the old and new values
				EExtraDependencyFlags& Existing = Dependencies.FindOrAdd(Pair.Key, EExtraDependencyFlags::None);
				Existing |= Pair.Value;

				// If an added key adds back a key that was removed, then no longer remove the key
				RemovedKeys.Remove(Pair.Key);
			}
			for (FName RemoveKey : RemovedKeys)
			{
				Dependencies.Remove(RemoveKey);
			}

			// Return the remapped values to the array and restore sortedness
			AssetRegistryData.ExtraPackageDependencies = Dependencies.Array();
			Algo::Sort(AssetRegistryData.ExtraPackageDependencies,
				[](const TPair<FName, EExtraDependencyFlags>& A, const TPair<FName, EExtraDependencyFlags>& B)
				{
					return A.Key.LexicalLess(B.Key);
				});
		}
	}

	// Do nametable patching last since we want to ensure we have determined all the remappings necessary
	PatchNameTable();

	return FAssetHeaderPatcher::EResult::Success;
}

FAssetHeaderPatcher::EResult FAssetHeaderPatcherInner::PatchHeader_WriteDestinationFile()
{
	// Serialize modified sections and reconstruct the file	
	// Original offsets and sizes of any sections that will be patched
	//	  Tag											Offset									Size												bRequired
	const FSectionData SourceSections[] = {
		{ EPatchedSection::Summary,						0,										HeaderInformation.SummarySize,						true	},
		{ EPatchedSection::NameTable,					Summary.NameOffset,						HeaderInformation.NameTableSize,					true	},
		{ EPatchedSection::SoftPathTable,				Summary.SoftObjectPathsOffset,			HeaderInformation.SoftObjectPathListSize,			false	},
		{ EPatchedSection::GatherableTextDataTable,		Summary.GatherableTextDataOffset,		HeaderInformation.GatherableTextDataSize,			false	},
		{ EPatchedSection::ImportTable,					Summary.ImportOffset,					HeaderInformation.ImportTableSize,					true	},
		{ EPatchedSection::ExportTable,					Summary.ExportOffset,					HeaderInformation.ExportTableSize,					true	},
		{ EPatchedSection::SoftPackageReferencesTable,	Summary.SoftPackageReferencesOffset,	HeaderInformation.SoftPackageReferencesListSize,	false	},
		{ EPatchedSection::SearchableNamesMap,			Summary.SearchableNamesOffset,			HeaderInformation.SearchableNamesMapSize,			false	},
		{ EPatchedSection::ImportTypeHierarchyMap,		Summary.ImportTypeHierarchiesOffset,	HeaderInformation.ImportTypeHierarchiesMapSize,		false	},
		{ EPatchedSection::ThumbnailTable,				Summary.ThumbnailTableOffset,			HeaderInformation.ThumbnailTableSize,				false	},
		{ EPatchedSection::AssetRegistryData,			Summary.AssetRegistryDataOffset,		AssetRegistryData.SectionSize,						true	},
		{ EPatchedSection::AssetRegistryDependencyData,	AssetRegistryData.PkgData.DependencyDataOffset,	AssetRegistryData.DependencyDataSectionSize,		false	},
	};

	const int32 SourceTotalHeaderSize = Summary.TotalHeaderSize;

	// Ensure the sections are in the expected order.
	for (int32 SectionIdx = 1; SectionIdx < UE_ARRAY_COUNT(SourceSections); ++SectionIdx)
	{
		const FSectionData& SourceSection = SourceSections[SectionIdx];
		const FSectionData& PrevSection = SourceSections[SectionIdx - 1];

		// Verify sections are ordered as expected
		if (SourceSection.Offset < 0 || (SourceSection.bRequired && (SourceSection.Offset < PrevSection.Offset)))
		{
			UE_LOG(LogAssetHeaderPatcher, Error, TEXT("Unexpected section order for %s (%d %" INT64_FMT " < %" INT64_FMT ") "),
				*SrcAsset, SectionIdx, SourceSection.Offset, PrevSection.Offset);
			return EResult::ErrorUnexpectedSectionOrder;
		}
	}

	// Ensure the required sections have data
	for (int32 SectionIdx = 0; SectionIdx < UE_ARRAY_COUNT(SourceSections); ++SectionIdx)
	{
		const FSectionData& SourceSection = SourceSections[SectionIdx];
		if (SourceSection.bRequired && SourceSection.Size <= 0)
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Unexpected missing required section for %ls: %d is required but has zero size.",
				*SrcAsset, SectionIdx);
			return EResult::ErrorEmptyRequireSection;
		}
	}

	// Create the destination file if not open already
	if (!DstArchive)
	{
		TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*DstAsset, FILEWRITE_EvenIfReadOnly));
		if (!FileWriter)
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to open %ls for write", *DstAsset);
			return EResult::ErrorFailedToOpenDestinationFile;
		}
		DstArchiveOwner = MoveTemp(FileWriter);
		DstArchive = DstArchiveOwner.Get();
	}

	FNamePatchingWriter Writer(*DstArchive, NameToIndexMap);
	Writer.SetShouldSkipUpdateCustomVersion(true);
	
	// set version numbers so components branch correctly
	Writer.SetUEVer(Summary.GetFileVersionUE());
	Writer.SetLicenseeUEVer(Summary.GetFileVersionLicenseeUE());
	Writer.SetEngineVer(Summary.SavedByEngineVersion);
	Writer.SetCustomVersions(Summary.GetCustomVersionContainer());
	if (Summary.GetPackageFlags() & PKG_FilterEditorOnly)
	{
		Writer.SetFilterEditorOnly(true);
	}

	if (Writer.GetCustomVersions().GetVersion(FFortniteMainBranchObjectVersion::GUID) == nullptr)
	{
		const_cast<FCustomVersionContainer&>(Writer.GetCustomVersions()).SetVersion(FFortniteMainBranchObjectVersion::GUID, -1, "");
	}

	int64 LastSectionEndedAt = 0;

	for (int32 SectionIdx = 0; SectionIdx < UE_ARRAY_COUNT(SourceSections); ++SectionIdx)
	{
		const FSectionData& SourceSection = SourceSections[SectionIdx];

		// skip processing empty non required chunks.
		if (!SourceSection.bRequired && SourceSection.Size <= 0)
		{
			continue;
		}

		// copy the blob from the end of the last section, to the start of this one
		if (LastSectionEndedAt)
		{
			int64 SizeToCopy = SourceSection.Offset - LastSectionEndedAt;
			checkf(SizeToCopy >= 0, TEXT("Section %d of %s\n%" INT64_FMT " -> %" INT64_FMT" %" INT64_FMT),
				SectionIdx, *SrcAsset, SourceSection.Offset, LastSectionEndedAt, SizeToCopy);
			Writer.Serialize(SrcBuffer.GetData() + LastSectionEndedAt, SizeToCopy);
		}
		LastSectionEndedAt = SourceSection.Offset + SourceSection.Size;

		// Serialize the current patched section and patch summary offsets
		switch (SourceSection.Section)
		{
		case EPatchedSection::Summary:
		{
			// We will write the Summary twice.
			// The first is so we get its new size (if the name was changed in patching)
			// The second is done after the loop, to patch up all the offsets.
			check(Writer.Tell() == 0);
			Writer << Summary;
			const int64 SummarySize = Writer.Tell();
			const int64 Delta = SummarySize - SourceSection.Size;
			PatchSummaryOffsets(Summary, 0, Delta);
			Summary.TotalHeaderSize += (int32)Delta;

			break;
		}

		case EPatchedSection::NameTable:
		{
			const int64 NameTableStartOffset = Writer.Tell();
			for (FName& Name : NameTable)
			{
				const FNameEntry* Entry = FName::GetEntry(Name.GetDisplayIndex());
				check(Entry);
				Entry->Write(Writer);
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 NameTableSize = Writer.Tell() - NameTableStartOffset;
			const int64 Delta = NameTableSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, NameTableStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;
			check(Summary.NameCount == NameTable.Num());
			check(Summary.NameOffset == NameTableStartOffset);

			break;
		}

		case EPatchedSection::SoftPathTable:
		{
			const int64 TableStartOffset = Writer.Tell();
			for (FSoftObjectPath& PathRef : SoftObjectPathTable)
			{
				PathRef.SerializePath(Writer);
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 TableSize = Writer.Tell() - TableStartOffset;
			const int64 Delta = TableSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, TableStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;
			check(Summary.SoftObjectPathsCount == SoftObjectPathTable.Num());
			check(Summary.SoftObjectPathsOffset == TableStartOffset);

			break;
		}

		case EPatchedSection::GatherableTextDataTable:
		{
			const int64 GatherableTableStartOffset = Writer.Tell();
			for (FGatherableTextData& GatherableTextData : GatherableTextDataTable)
			{
				Writer << GatherableTextData;
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 TableSize = Writer.Tell() - GatherableTableStartOffset;
			const int64 Delta = TableSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, GatherableTableStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;
			check(Summary.GatherableTextDataCount == GatherableTextDataTable.Num());
			check(Summary.GatherableTextDataOffset == GatherableTableStartOffset);

			break;
		}

		case EPatchedSection::SearchableNamesMap:
		{
			const int64 TableStartOffset = Writer.Tell();
			
			FLinkerTables LinkerTables;
			LinkerTables.SearchableNamesMap = SearchableNamesMap;
			LinkerTables.SerializeSearchableNamesMap(Writer);

			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 TableSize = Writer.Tell() - TableStartOffset;
			const int64 Delta = TableSize - SourceSection.Size;
			checkf(Delta == 0, TEXT("Delta should be Zero. is %d"), (int)Delta);
			check(Summary.SearchableNamesOffset == TableStartOffset);

			break;
		}

		case EPatchedSection::ImportTable:
		{
			const int64 ImportTableStartOffset = Writer.Tell();
			for (FObjectImport& Import : ImportTable)
			{
				Writer << Import;
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 ImportTableSize = Writer.Tell() - ImportTableStartOffset;
			const int64 Delta = ImportTableSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, ImportTableStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;
			checkf(Summary.ImportCount == ImportTable.Num(), TEXT("%d == %d"), Summary.ImportCount, ImportTable.Num());
			checkf(Summary.ImportOffset == ImportTableStartOffset, TEXT("%d == %" INT64_FMT), Summary.ImportOffset, ImportTableStartOffset);

			break;
		}

		case EPatchedSection::ExportTable:
		{
			// The export table offsets aren't correct yet.
			// Once we know them, we will seek back and write it a second time.
			const int64 ExportTableStartOffset = Writer.Tell();
			for (FObjectExport& Export : ExportTable)
			{
				Writer << Export;
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 ExportTableSize = Writer.Tell() - ExportTableStartOffset;
			const int64 Delta = ExportTableSize - SourceSection.Size;
			check(Delta == 0);
			checkf(ExportTableSize == SourceSection.Size, TEXT("%d == %d"), (int)ExportTableSize, (int)SourceSection.Size); // We only patch export table offsets, we should not be patching size
			checkf(Summary.ExportCount == ExportTable.Num(), TEXT("%d == %d"), Summary.ExportCount, ExportTable.Num());
			checkf(Summary.ExportOffset == ExportTableStartOffset, TEXT("%d == %" INT64_FMT), Summary.ExportOffset, ExportTableStartOffset);

			break;
		}

		case EPatchedSection::SoftPackageReferencesTable:
		{
			const int64 TableStartOffset = Writer.Tell();
			for (FName& Reference : SoftPackageReferencesTable)
			{
				Writer << Reference;
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 TableSize = Writer.Tell() - TableStartOffset;
			const int64 Delta = TableSize - SourceSection.Size;
			checkf(Delta == 0, TEXT("Delta should be Zero. is %d"), (int)Delta);
			check(Summary.SoftPackageReferencesCount == SoftPackageReferencesTable.Num());
			check(Summary.SoftPackageReferencesOffset == TableStartOffset);

			break;
		}

		case EPatchedSection::ImportTypeHierarchyMap:
		{
			const int64 MapStartOffset = Writer.Tell();
			Writer << ImportTypeHierarchyMap;

			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 MapSize = Writer.Tell() - MapStartOffset;
			const int64 Delta = MapSize - SourceSection.Size;
			checkf(Delta == 0, TEXT("Delta should be Zero. is %d"), (int)Delta);
			check(Summary.ImportTypeHierarchiesCount == ImportTypeHierarchyMap.Num());
			check(Summary.ImportTypeHierarchiesOffset == MapStartOffset);

			break;
		}

		case EPatchedSection::ThumbnailTable:
		{
			const int64 ThumbnailTableStartOffset = Writer.Tell();
			const int64 ThumbnailTableDeltaOffset = ThumbnailTableStartOffset - SourceSection.Offset;
			int32 ThumbnailCount = ThumbnailTable.Num();
			Writer << ThumbnailCount;
			for (FThumbnailEntry& Entry : ThumbnailTable)
			{
				Writer << Entry.ObjectShortClassName;
				Writer << Entry.ObjectPathWithoutPackageName;
				Entry.FileOffset += (int32)ThumbnailTableDeltaOffset;
				Writer << Entry.FileOffset;
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 ThumbnailTableSize = Writer.Tell() - ThumbnailTableStartOffset;
			const int64 Delta = ThumbnailTableSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, ThumbnailTableStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;
			checkf(ThumbnailTableStartOffset == Summary.ThumbnailTableOffset, TEXT("%" INT64_FMT " == %d"),
				ThumbnailTableStartOffset, Summary.ThumbnailTableOffset);

			break;
		}

		case EPatchedSection::AssetRegistryData:
		{
			const int64 AssetRegistryDataStartOffset = Writer.Tell();
			checkf(AssetRegistryDataStartOffset == Summary.AssetRegistryDataOffset, TEXT("%" INT64_FMT " == %d"),
				AssetRegistryDataStartOffset, Summary.AssetRegistryDataOffset);

			// Code to write this section copied and modified from UE::AssetRegistry::WritePackageData.
			// TODO: Factor this into a public function in SavePackageUtilities and call that.

			if (AssetRegistryData.PkgData.DependencyDataOffset != INDEX_NONE)
			{
				// This field is conditionally written depending on package version and cookedness. If it is written it
				// is guaranteed to != INDEX_NONE. We are not changing those properties for the package, so we write
				// the field if and only if we found it present in the package originally.
				Writer << AssetRegistryData.PkgData.DependencyDataOffset;
			}
			Writer << AssetRegistryData.PkgData.ObjectCount;

			check(AssetRegistryData.PkgData.ObjectCount == AssetRegistryData.ObjectData.Num());
			for (FAssetRegistryObjectData& ObjData : AssetRegistryData.ObjectData)
			{
				Writer << ObjData.ObjectData.ObjectPath;
				Writer << ObjData.ObjectData.ObjectClassName;
				Writer << ObjData.ObjectData.TagCount;

				check(ObjData.ObjectData.TagCount == ObjData.TagData.Num());
				for (UE::AssetRegistry::FDeserializeTagData& TagData : ObjData.TagData)
				{
					Writer << TagData.Key;
					Writer << TagData.Value;
				}
			}
			checkf(!Writer.IsCriticalError(), TEXT("Issue writing %s"), *Writer.GetErrorMessage());

			const int64 AssetRegistryDataSize = Writer.Tell() - AssetRegistryDataStartOffset;
			const int64 Delta = AssetRegistryDataSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, AssetRegistryDataStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;

			break;
		}

		case EPatchedSection::AssetRegistryDependencyData:
		{
			int64 AssetRegistryDependencyDataStartOffset = Writer.Tell();

			// Re-write the offset of this section into AssetRegistryData.PkgData.DependencyDataOffset in the
			// earlier EPatchedSection::AssetRegistryData section.
			Writer.Seek(Summary.AssetRegistryDataOffset);
			Writer << AssetRegistryDependencyDataStartOffset;
			if (Writer.IsError())
			{
				UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to write to %ls", *DstAsset);
				return EResult::ErrorFailedToWriteToDestinationFile;
			}
			Writer.Seek(AssetRegistryDependencyDataStartOffset);
			AssetRegistryData.PkgData.DependencyDataOffset = AssetRegistryDependencyDataStartOffset;

			// Code to write this section copied and modified from UE::AssetRegistry::WritePackageData.
			// TODO: Factor this into a public function in SavePackageUtilities and call that.

			TBitArray<> ImportUsedInGameBits;
			TBitArray<> SoftPackageUsedInGameBits;
			ImportUsedInGameBits.Reserve(ImportTable.Num());
			for (int32 ImportIndex = 0; ImportIndex < ImportTable.Num(); ++ImportIndex)
			{
				bool* UsedInGame = AssetRegistryData.ImportIndexUsedInGame.Find(ImportIndex);
				bool bUsedInGame = UsedInGame ? *UsedInGame : true;
				ImportUsedInGameBits.Add(bUsedInGame);
			}
			SoftPackageUsedInGameBits.Reserve(SoftPackageReferencesTable.Num());
			for (int32 SoftPackageIndex = 0; SoftPackageIndex < SoftPackageReferencesTable.Num(); ++SoftPackageIndex)
			{
				bool* UsedInGame = AssetRegistryData.SoftPackageReferenceUsedInGame.Find(SoftPackageReferencesTable[SoftPackageIndex]);
				bool bUsedInGame = UsedInGame ? *UsedInGame : true;
				SoftPackageUsedInGameBits.Add(bUsedInGame);
			}

			Writer << ImportUsedInGameBits;
			Writer << SoftPackageUsedInGameBits;

			TArray<TPair<FName, uint32>> ExtraPackageDependenciesInts;
			ExtraPackageDependenciesInts.Reserve(AssetRegistryData.ExtraPackageDependencies.Num());
			for (const TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>& DependencyPair : AssetRegistryData.ExtraPackageDependencies)
			{
				ExtraPackageDependenciesInts.Add({ DependencyPair.Key, static_cast<uint32>(DependencyPair.Value) });
			}
			Writer << ExtraPackageDependenciesInts;

			const int64 AssetRegistryDependencyDataSize = Writer.Tell() - AssetRegistryDependencyDataStartOffset;
			const int64 Delta = AssetRegistryDependencyDataSize - SourceSection.Size;
			PatchSummaryOffsets(Summary, AssetRegistryDependencyDataStartOffset, Delta);
			Summary.TotalHeaderSize += (int32)Delta;

			break;
		}

		default:
			UE_LOGF(LogAssetHeaderPatcher, Error, "Unexpected section for %ls", *SrcAsset);
			return EResult::ErrorUnkownSection;
		}

		if (Writer.IsError())
		{
			UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to write to %ls", *DstAsset);
			return EResult::ErrorFailedToWriteToDestinationFile;
		}
	}

	{   // copy the last blob
		int64 SizeToCopy = SrcBuffer.Num() - LastSectionEndedAt;
		checkf(SizeToCopy >= 0, TEXT("Section last of %s\n%" INT64_FMT " -> %" INT64_FMT " %" INT64_FMT),
			*SrcAsset, SrcBuffer.Num(), LastSectionEndedAt, SizeToCopy);
		Writer.Serialize(SrcBuffer.GetData() + LastSectionEndedAt, SizeToCopy);
	}

	if (Writer.IsError())
	{
		UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to write to %ls", *DstAsset);
		return EResult::ErrorFailedToWriteToDestinationFile;
	}

	// Re-write summary with patched offsets
	Writer.Seek(0);
	Writer << Summary;

	{
		// Re-write export table with patched offsets
		// Patch Export table offsets now that we have patched all the header sections
		Writer.Seek(Summary.ExportOffset);
		const int64 ExportOffsetDelta = static_cast<int64>(Summary.TotalHeaderSize) - SourceTotalHeaderSize;
		for (FObjectExport& Export : ExportTable)
		{
			Export.SerialOffset += ExportOffsetDelta;
			Writer << Export;
		}
	}

	if (Writer.IsError())
	{
		UE_LOGF(LogAssetHeaderPatcher, Error, "Failed to write to %ls", *DstAsset);
		return EResult::ErrorFailedToWriteToDestinationFile;
	}

	return EResult::Success;
}

void FAssetHeaderPatcherInner::DumpState(FStringView OutputDirectory)
{
	TStringBuilder<1024> Builder;
	auto GetDebugFNameString = [this](FName Name)
		{
			int32* Index = NameToIndexMap.Find(Name.GetDisplayIndex());
			if (Index)
			{
				FName NameTableName = NameTable[*Index];
				return FString::Printf(TEXT("%s (nametable index: %d, fname:{'%s', %d})"), *NameTableName.ToString(), *Index, *Name.GetPlainNameString(), Name.GetNumber());
			}
			else
			{
				return FString(TEXT("None (nametable index: -1, fname {'None', 0})"));
			}
		};

	Builder.Append(TEXT("{\n"));

	Builder.Append(TEXT("\t\"Summary\":{ "));
	{
		Builder.Appendf(TEXT("\n\t\t\"PackageName\": \"%s\""), *Summary.PackageName);
		Builder.Appendf(TEXT(",\n\t\t\"NamesReferencedFromExportDataCount\": \"%d\""), Summary.NamesReferencedFromExportDataCount);
		Builder.Appendf(TEXT(",\n\t\t\"ExportCount\": \"%d\""), Summary.ExportCount);
		Builder.Appendf(TEXT(",\n\t\t\"ImportCount\": \"%d\""), Summary.ImportCount);
	}
	Builder.Append(TEXT("\n\t},\n"));

	Builder.Append(TEXT("\t\"NameTable\":[ "));
	for (const FName& Name : NameTable)
	{
		Builder.Append(TEXT("\n\t\t\""));
		Builder.Append(GetDebugFNameString(Name));
		Builder.Append(TEXT("\","));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"ExportTable\":[ "));
	int ExportIndex = 0;
	for (const auto& Export : ExportTable)
	{
		Builder.Append(TEXT("\n\t\t{\n"));

		Builder.Appendf(TEXT("\t\t\t\"Index\": \"%d\""), ExportIndex++);
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"ObjectName\": \""));
		Builder.Append(GetDebugFNameString(Export.ObjectName));
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"Outer\": \""));
		if (Export.OuterIndex.IsNull())
		{
			Builder.Append(TEXT("None"));
		}
		else if (Export.OuterIndex.IsExport())
		{
			Builder.Appendf(TEXT("Export(%d) - %s"), Export.OuterIndex.ToExport(), *GetDebugFNameString(ExportTable[Export.OuterIndex.ToExport()].ObjectName));
		}
		else if (Export.OuterIndex.IsImport())
		{
			Builder.Appendf(TEXT("Import(%d) - %s"), Export.OuterIndex.ToImport(), *GetDebugFNameString(ImportTable[Export.OuterIndex.ToImport()].ObjectName));
		}
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"ClassIndex\": \""));
		if (Export.ClassIndex.IsNull())
		{
			Builder.Append(TEXT("None"));
		}
		else if (Export.ClassIndex.IsExport())
		{
			Builder.Appendf(TEXT("Export(%d) - %s"), Export.ClassIndex.ToExport(), *GetDebugFNameString(ExportTable[Export.ClassIndex.ToExport()].ObjectName));
		}
		else if (Export.ClassIndex.IsImport())
		{
			Builder.Appendf(TEXT("Import(%d) - %s"), Export.ClassIndex.ToImport(), *GetDebugFNameString(ImportTable[Export.ClassIndex.ToImport()].ObjectName));
		}
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"SuperIndex\": \""));
		if (Export.SuperIndex.IsNull())
		{
			Builder.Append(TEXT("None"));
		}
		else if (Export.SuperIndex.IsExport())
		{
			Builder.Appendf(TEXT("Export(%d) - %s"), Export.SuperIndex.ToExport(), *GetDebugFNameString(ExportTable[Export.SuperIndex.ToExport()].ObjectName));
		}
		else if (Export.SuperIndex.IsImport())
		{
			Builder.Appendf(TEXT("Import(%d) - %s"), Export.SuperIndex.ToImport(), *GetDebugFNameString(ImportTable[Export.SuperIndex.ToImport()].ObjectName));
		}
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"TemplateIndex\": \""));
		if (Export.TemplateIndex.IsNull())
		{
			Builder.Append(TEXT("None"));
		}
		else if (Export.TemplateIndex.IsExport())
		{
			Builder.Appendf(TEXT("Export(%d) - %s"), Export.TemplateIndex.ToExport(), *GetDebugFNameString(ExportTable[Export.TemplateIndex.ToExport()].ObjectName));
		}
		else if (Export.TemplateIndex.IsImport())
		{
			Builder.Appendf(TEXT("Import(%d) - %s"), Export.TemplateIndex.ToImport(), *GetDebugFNameString(ImportTable[Export.TemplateIndex.ToImport()].ObjectName));
		}
		Builder.Append(TEXT("\",\n"));

#if WITH_EDITORONLY_DATA
		Builder.Append(TEXT("\t\t\t\"OldClassName\": \""));
		Builder.Append(GetDebugFNameString(Export.OldClassName));
		Builder.Append(TEXT("\""));
#endif
		Builder.Append(TEXT("\n\t\t},"));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"ImportTable\":[ "));
	int ImportIndex = 0;
	for (const auto& Import : ImportTable)
	{
		Builder.Append(TEXT("\n\t\t{\n"));

		Builder.Appendf(TEXT("\t\t\t\"Index\": \"%d\""), ImportIndex++);
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"ObjectName\": \""));
		Builder.Append(GetDebugFNameString(Import.ObjectName));
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"Outer\": \""));
		if (Import.OuterIndex.IsNull())
		{
			Builder.Append(TEXT("None"));
		}
		else if (Import.OuterIndex.IsExport())
		{
			Builder.Appendf(TEXT("Export(%d) - %s"), Import.OuterIndex.ToExport(), *GetDebugFNameString(ExportTable[Import.OuterIndex.ToExport()].ObjectName));
		}
		else if (Import.OuterIndex.IsImport())
		{
			Builder.Appendf(TEXT("Import(%d) - %s"), Import.OuterIndex.ToImport(), *GetDebugFNameString(ImportTable[Import.OuterIndex.ToImport()].ObjectName));
		}
		Builder.Append(TEXT("\",\n"));

#if WITH_EDITORONLY_DATA
		Builder.Append(TEXT("\t\t\t\"OldClassName\": \""));
		Builder.Append(GetDebugFNameString(Import.OldClassName));
		Builder.Append(TEXT("\",\n"));
#endif

		Builder.Append(TEXT("\t\t\t\"ClassPackage\": \""));
		Builder.Append(GetDebugFNameString(Import.ClassPackage));
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"ClassName\": \""));
		Builder.Append(GetDebugFNameString(Import.ClassName));
		Builder.Append(TEXT("\""));

#if WITH_EDITORONLY_DATA
		Builder.Append(TEXT(",\n\t\t\t\"PackageName\": \""));
		Builder.Append(GetDebugFNameString(Import.PackageName));
		Builder.Append(TEXT("\""));
#endif
		Builder.Append(TEXT("\n\t\t},"));

	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"SoftObjectPathTable\":[ "));
	for (const FSoftObjectPath& SoftObjectPath : SoftObjectPathTable)
	{
		Builder.Append(TEXT("\n\t\t{\n"));

		FTopLevelAssetPath TLAP = SoftObjectPath.GetAssetPath();
		FString Subpath = SoftObjectPath.GetSubPathString();

		Builder.Append(TEXT("\t\t\t\"AssetPath\": {\n\""));
		Builder.Append(TEXT("\t\t\t\t\"PackageName\": \""));
		Builder.Append(GetDebugFNameString(TLAP.GetPackageName()));
		Builder.Append(TEXT("\",\n"));
		Builder.Append(TEXT("\t\t\t\t\"AssetName\": \""));
		Builder.Append(GetDebugFNameString(TLAP.GetAssetName()));
		Builder.Append(TEXT("\"\n"));
		Builder.Append(TEXT("\t\t\t},\n"));

		Builder.Append(TEXT("\t\t\t\"Subpath (string)\": \""));
		Builder.Append(Subpath);
		Builder.Append(TEXT("\""));

		Builder.Append(TEXT("\n\t\t},"));

	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"SoftPackageReferencesTable\":[ "));
	for (const FName SoftPackageRef : SoftPackageReferencesTable)
	{
		Builder.Append(TEXT("\n\t\t\""));
		Builder.Append(GetDebugFNameString(SoftPackageRef));
		Builder.Append(TEXT("\","));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"GatherableTextDataTable\":[ "));
	for (const FGatherableTextData& GatherableTextData : GatherableTextDataTable)
	{
		Builder.Append(TEXT("\n\t\t{\n"));
		Builder.Append(TEXT("\t\t\t\"SourceSiteContexts.SiteDescription (string)\": ["));
		for (auto& SiteContext : GatherableTextData.SourceSiteContexts)
		{
			Builder.Append(TEXT("\n\t\t\t\t\""));
			Builder.Append(SiteContext.SiteDescription);
			Builder.Append(TEXT("\","));
		}
		Builder.RemoveSuffix(1);
		Builder.Append(TEXT("\n\t\t\t]"));
		Builder.Append(TEXT("\n\t\t},"));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"ImportTypeHierarchyMap\":[ "));
	for (TPair<FPackageIndex, UE::Serialization::Private::FImportTypeHierarchy> Pair : ImportTypeHierarchyMap)
	{
		Builder.Append(TEXT("\n\t\t{\n"));
		Builder.Append(TEXT("\t\t\t\"Super types\": ["));

		const TArray<UE::Serialization::Private::FTypeResource>& SuperTypes = Pair.Value.GetSuperTypes();

		for (const UE::Serialization::Private::FTypeResource& SuperType : SuperTypes)
		{
			Builder.Append(TEXT("\n\t\t\t\t{"));

			Builder.Append(TEXT("\n\t\t\t\t\t\"TypeName\": \""));
			Builder.Append(GetDebugFNameString(SuperType.TypeName));
			Builder.Append(TEXT("\","));

			Builder.Append(TEXT("\n\t\t\t\t\t\"PackageName\": \""));
			Builder.Append(GetDebugFNameString(SuperType.PackageName));
			Builder.Append(TEXT("\","));

			Builder.Append(TEXT("\n\t\t\t\t\t\"ClassName\": \""));
			Builder.Append(GetDebugFNameString(SuperType.ClassName));
			Builder.Append(TEXT("\","));

			Builder.Append(TEXT("\n\t\t\t\t\t\"ClassPackageName\": \""));
			Builder.Append(GetDebugFNameString(SuperType.ClassPackageName));
			Builder.Append(TEXT("\","));

			Builder.Append(TEXT("\n\t\t\t\t},"));
		}

		Builder.RemoveSuffix(1);
		Builder.Append(TEXT("\n\t\t\t]"));
		Builder.Append(TEXT("\n\t\t},"));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"ThumbnailTable\":[ "));
	for (const FThumbnailEntry& ThumbnailEntry : ThumbnailTable)
	{
		Builder.Append(TEXT("\n\t\t{\n"));

		Builder.Append(TEXT("\t\t\t\"ObjectPathWithoutPackageName (string)\": \""));
		Builder.Append(ThumbnailEntry.ObjectPathWithoutPackageName);
		Builder.Append(TEXT("\",\n"));

		Builder.Append(TEXT("\t\t\t\"ObjectShortClassName (string)\": \""));
		Builder.Append(ThumbnailEntry.ObjectShortClassName);
		Builder.Append(TEXT("\""));

		Builder.Append(TEXT("\n\t\t},"));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t],\n"));

	Builder.Append(TEXT("\t\"AssetRegistryData\":[ "));
	for (const FAssetRegistryObjectData& ObjData : AssetRegistryData.ObjectData)
	{
		Builder.Append(TEXT("\n\t\t{\n"));

		Builder.Append(TEXT("\t\t\t\"ObjectData\": {\n"));
		Builder.Append(TEXT("\t\t\t\t\"ObjectPath (string)\": \""));
		Builder.Append(ObjData.ObjectData.ObjectPath);
		Builder.Append(TEXT("\",\n"));
		Builder.Append(TEXT("\t\t\t\t\"ObjectClassName (string)\": \""));
		Builder.Append(ObjData.ObjectData.ObjectClassName);
		Builder.Append(TEXT("\"\n"));
		Builder.Append(TEXT("\t\t\t},\n"));

		Builder.Append(TEXT("\t\t\t\"TagData\": [\n"));
		for (const auto& TagData : ObjData.TagData)
		{
			FString Value = TagData.Value;
			bool bNeedDecode = TagData.Key == FWorldPartitionActorDescUtils::ActorMetaDataTagName();
			if (bNeedDecode)
			{
				const FString LongPackageName(SrcAsset);
				const FString ObjectPath(ObjData.ObjectData.ObjectPath);
				const FTopLevelAssetPath AssetClassPathName(ObjData.ObjectData.ObjectClassName);
				const FAssetDataTagMap Tags(MakeTagMap(ObjData.TagData));
				const FAssetData AssetData(LongPackageName, ObjectPath, AssetClassPathName, Tags);

				struct FWorldPartitionAssetDataPrinter : FWorldPartitionAssetDataPatcher
				{
					FWorldPartitionAssetDataPrinter(int32 InIndentDepth)
						: IndentDepth(InIndentDepth) 
					{
					}

					virtual bool DoPatch(FString& InOutString) override
					{
						Builder.Append(TEXT("\n"));
						Indent();
						Builder.Append(TEXT("string=\""));
						Builder.Append(InOutString);
						Builder.Append(TEXT("\""));
						return false;
					}
					virtual bool DoPatch(FName& InOutName) override
					{
						Builder.Append(TEXT("\n"));
						Indent();
						Builder.Append(TEXT("FName=\""));
						Builder.Append(InOutName.ToString());
						Builder.Append(TEXT("\""));
						return false;
					}
					virtual bool DoPatch(FSoftObjectPath& InOutSoft) override
					{
						Builder.Append(TEXT("\n"));
						Indent();
						Builder.Append(TEXT("FSoftObjectPath="));
						FTopLevelAssetPath TLAP = InOutSoft.GetAssetPath();
						Builder.Append(TEXT("{{PackageName=\""));
						Builder.Append(TLAP.GetPackageName().ToString());
						Builder.Append(TEXT("\", AssetName=\""));
						Builder.Append(TLAP.GetAssetName().ToString());
						Builder.Append(TEXT("\"}, SubPath (string)=\""));
						Builder.Append(InOutSoft.GetSubPathString());
						Builder.Append(TEXT("\"}"));
						return false;
					}
					virtual bool DoPatch(FTopLevelAssetPath& InOutPath) override
					{
						Builder.Append(TEXT("\n"));
						Indent();
						Builder.Append(TEXT("FTopLevelAssetPath="));
						Builder.Append(TEXT("{PackageName=\""));
						Builder.Append(InOutPath.GetPackageName().ToString());
						Builder.Append(TEXT("\", AssetName=\""));
						Builder.Append(InOutPath.GetAssetName().ToString());
						Builder.Append(TEXT("\"}"));
						return false;
					}
					void Indent()
					{
						for (int32 i = 0; i < IndentDepth; ++i)
						{
							Builder.Append(TEXT("\t"));
						}
					}
					const TCHAR* ToString()
					{
						return Builder.ToString();
					}
					int32 IndentDepth;
					TStringBuilder<1024> Builder;
				};

				FString PatchedAssetData;
				FWorldPartitionAssetDataPrinter Patcher(5);
				FWorldPartitionActorDescUtils::GetPatchedAssetDataFromAssetData(AssetData, PatchedAssetData, &Patcher);
				Value = Patcher.ToString();
			}

			Builder.Append(TEXT("\n\t\t\t\t{\n"));

			Builder.Append(TEXT("\t\t\t\t\t\"Key (string)\": \""));
			Builder.Append(TagData.Key);
			Builder.Append(TEXT("\",\n"));
			Builder.Append(TEXT("\t\t\t\t\t\"Value"));
			if (bNeedDecode)
			{
				Builder.Append(TEXT(" (decoded string)"));
			}
			else
			{
				Builder.Append(TEXT("(string)"));
			}
			Builder.Append(TEXT("\": \""));
			Builder.Append(Value);
			Builder.Append(TEXT("\"\n"));

			Builder.Append(TEXT("\t\t\t\t},"));
		}
		Builder.RemoveSuffix(1);
		Builder.Append(TEXT("\n\t\t\t]\n"));

		Builder.Append(TEXT("\n\t\t},"));
	}
	Builder.RemoveSuffix(1);
	Builder.Append(TEXT("\n\t]\n"));

	Builder.Append(TEXT("\t\"AssetRegistryDependencyData\":{ "));
	{
		Builder.Append(TEXT("\n\t\t\"ImportIndexUsedInGame\":{ "));
		for (ImportIndex = 0; ImportIndex < ImportTable.Num(); ++ImportIndex)
		{
			bool* UsedInGame = AssetRegistryData.ImportIndexUsedInGame.Find(ImportIndex);
			bool bUsedInGame = UsedInGame ? *UsedInGame : true;
			Builder.Appendf(TEXT("\n\t\t\t%d : %s,"), ImportIndex, bUsedInGame ? TEXT("true") : TEXT("false"));
		}
		Builder.RemoveSuffix(1);
		Builder.Append(TEXT("\n\t\t}"));

		Builder.Append(TEXT(",\n\t\t\"SoftPackageReferenceUsedInGame\":{ "));
		for (FName SoftPackageReference : SoftPackageReferencesTable)
		{
			bool* UsedInGame = AssetRegistryData.SoftPackageReferenceUsedInGame.Find(SoftPackageReference);
			bool bUsedInGame = UsedInGame ? *UsedInGame : true;
			Builder.Append(TEXT("\n\t\t\t"));
			Builder << SoftPackageReference;
			Builder.Appendf(TEXT(" : %s,"), bUsedInGame ? TEXT("true") : TEXT("false"));
		}
		Builder.RemoveSuffix(1);
		Builder.Append(TEXT("\n\t\t}"));

		Builder.Append(TEXT(",\n\t\t\"ExtraPackageDependencies\":[ "));
		for (const TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>& Pair : AssetRegistryData.ExtraPackageDependencies)
		{
			Builder.Append(TEXT("\n\t\t\t[ \""));
			Builder << Pair.Key;
			Builder.Appendf(TEXT("\", 0x%x],"), (uint32) Pair.Value);
		}
		Builder.RemoveSuffix(1);
		Builder.Append(TEXT("\n\t\t]"));
	}
	Builder.Append(TEXT("\n\t},\n"));

	Builder.Append(TEXT("}"));

	// Write to disk
	TStringBuilder<256> OutPath;
	OutPath.Append(OutputDirectory);
	FString SubPath = SrcAsset;
	FPaths::CollapseRelativeDirectories(SubPath);
	if (SubPath.StartsWith(TEXT("../")))
	{
		int32 Pos = SubPath.Find(TEXT("../"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (Pos >= 0)
		{
			SubPath.RightChopInline(Pos + 3);
		}
	}
	else if (SubPath.Len() > 2 && SubPath[1] == TEXT(':'))
	{
		SubPath.RightChopInline(2); // Drop the drive
	}
	OutPath = FPaths::Combine(OutPath, SubPath);
	OutPath.Append(TEXT(".txt"));
	FFileHelper::SaveStringToFile(Builder.ToString(), *OutPath);
}

///////////////////////////////////////////////////////////////////////////

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FAssetHeaderPatcherTests, "AssetHeaderPatcher", "[AssetHeaderPatcher][EngineFilter]")
{
// Useful when iterating so you halt if something fails while a debugger is attached
#if DEBUG_ASSET_HEADER_PATCHING
#undef CHECK
#undef CHECK_EQUALS
#undef CHECK_NOT_EQUALS
#define CHECK(...) if (!(__VA_ARGS__)) { UE_DEBUG_BREAK(); FAutomationTestFramework::Get().GetCurrentTest()->AddError(TEXT("Condition failed")); }
#define CHECK_EQUALS(What, X, Y) if(!FAutomationTestFramework::Get().GetCurrentTest()->TestEqual(What, X, Y)) { UE_DEBUG_BREAK(); };
#define CHECK_NOT_EQUALS(What, X, Y) if(!FAutomationTestFramework::Get().GetCurrentTest()->TestNotEqual(What, X, Y)) { UE_DEBUG_BREAK(); };
#endif

	struct FTestPatcherContext : FAssetHeaderPatcher::FContext
	{
		FTestPatcherContext(TMap<FString, FString> PackageRenameMap, bool bGatherDependentPackages = true) : FAssetHeaderPatcher::FContext(PackageRenameMap, bGatherDependentPackages) {}
		const TMap<FString, FString>& GetVerseStringReplacements()
		{
			return VerseStringReplacements;
		}

		void GenerateRemappings()
		{
			GenerateAdditionalRemappings();
		}

		const TArray<FCoreRedirect>& GetRedirects()
		{
			return Redirects;
		}

		const TArray<FString>& GetVerseMountPoints()
		{
			return VerseMountPoints;
		}
	};

	// To avoid having to deal with serialization, we mock some data and inject it directly
	// into the patcher as if done via serialization
	const FString DummySrcDstAsset			 = TEXT("/SrcMount/SomePath/SrcPackage");
	const TCHAR*  SrcPackagePath			 = TEXT("/SrcMount/SomePath/SrcPackage");
	const TCHAR*  DstPackagePath			 = TEXT("/DstMount/SomePath/DstPackage");
	const TCHAR*  SrcPackageObjectPath		 = TEXT("/SrcMount/SomePath/SrcPackage.SrcPackage");
	const TCHAR*  DstPackageObjectPath		 = TEXT("/DstMount/SomePath/DstPackage.DstPackage");
	const TCHAR*  SrcNumberChangedPath		 = TEXT("/SrcMount/SomePath/UnchangingName_1");
	const TCHAR*  DstNumberChangedPath		 = TEXT("/DstMount/SomePath/UnchangingName_2");
	const TCHAR*  SrcNumberChangedObjectPath = TEXT("/SrcMount/SomePath/UnchangingName_1.UnchangingName_1");
	const TCHAR*  DstNumberChangedObjectPath = TEXT("/DstMount/SomePath/UnchangingName_2.UnchangingName_2");
	const TCHAR*  SrcMountName				 = TEXT("/SourceSpecialMount/");
	const TCHAR*  DstMountName				 = TEXT("/DestinationSpecialMount/");
	const FSoftObjectPath SoftObjectPathToRedirect(TEXT("/ToBeRedirectedMount/SomePath/ToBeRedirectedPackage.ToBeRedirectedPackage:Some.ToBeRedirectedPackage.Subobject"));
	const FSoftObjectPath RedirectedSoftObjectPath(TEXT("/RedirectedMount/SomePath/RedirectedPackage.RedirectedPackage:Some.RedirectedPackage.Subobject"));
	const FName SrcPackagePathFName(SrcPackagePath);
	const FName DstPackagePathFName(DstPackagePath);
	const FName SrcNumberChangedPathFName(SrcNumberChangedPath);
	const FName DstNumberChangedPathFName(DstNumberChangedPath);
	const FName SrcAssetFName(TEXT("SrcPackage"));
	const FName DstAssetFName(TEXT("DstPackage"));
	const FName SrcNumberAssetFName(TEXT("UnchangingName_1"));
	const FName DstNumberAssetFName(TEXT("UnchangingName_2"));
	const FName SrcExportObjectFName = SrcAssetFName;
	const FName DstExportObjectFName = DstAssetFName;
	const FName SrcExportSubObjectFName(TEXT("SrcSubObject"));
	const FName DstExportSubObjectFName(TEXT("DstSubObject"));
	const FName SrcExportOnlyNumberChangesFName(TEXT("OnlyNumberChange_1"));
	const FName DstExportOnlyNumberChangesFName(TEXT("OnlyNumberChange_2"));
	const FName DummyImportPackagePathFName(TEXT("/DummyMount/DummyPackage"));
	// Import FNames
	const FName SrcEngineModuleImportObjectName(TEXT("/Script/SrcEngineModule"));
	const FName SrcTypeAImportObjectName(TEXT("SrcTypeA"));
	const FName SrcTypeBImportObjectName(TEXT("SrcTypeB"));
	const FName OnlySubTypeChangedImportObjectName(TEXT("OnlySubTypeChanged"));
	const FName MovedButNotRenamedTypeImportObjectName(TEXT("MovedButNotRenamedType"));
	const FName SrcPropertyToChangeAImportObjectName(TEXT("SrcPropertyToChangeA_2"));
	const FName SrcPropertyToChangeBImportObjectName(TEXT("SrcPropertyToChangeB"));
	const FName MovedButNotRenamedPropertyImportObjectName(TEXT("MovedButNotRenamedProperty"));
	const FName NewOuterImportObjectName(TEXT("NewOuter"));
	const FName MovedToNewOuterImportObjectName(TEXT("MovedToNewOuter"));
	const FName InnerMovedButNotRenamedPropertyImportObjectName(TEXT("InnerMovedButNotRenamedProperty"));
	const FName InnerInnerMovedButNotRenamedPropertyImportObjectName(TEXT("InnerInnerMovedButNotRenamedProperty"));
	const FName UnchangedPropertyImportObjectName(TEXT("UnchangedProperty"));
	const FName SrcImportClassName(TEXT("SrcClass"));
	const FName SrcImportClassPackage(TEXT("/Engine/SrcClassPackage"));
	const FName SrcVerseAssetName(TEXT("/Module/_Verse/VerseAsset"));
	const FName SrcVerseImportObjectName(TEXT("some_verse_class"));
	const FName UnchangedVerseImportSubObject1Name(TEXT("__verse_0x7A8CDEBC_VerseObject1"));
	const FName UnchangedVerseImportSubObject2Name(TEXT("__verse_0x5614AC82_VerseObject2"));

	const FName DstEngineModuleImportObjectName(TEXT("/Script/DstEngineModule"));
	const FName DstTypeAImportObjectName(TEXT("DstTypeA"));
	const FName DstTypeBImportObjectName(TEXT("DstTypeB"));
	const FName DstPropertyToChangeAImportObjectName(TEXT("DstPropertyToChangeA_4"));
	const FName DstPropertyToChangeBImportObjectName(TEXT("DstPropertyToChangeB"));
	const FName DstVerseAssetName(TEXT("/Module/_Verse"));
	const FName DstVerseImportObjectName(TEXT("VerseAsset-some_verse_class"));
	const FName DstImportClassName(TEXT("DstClass"));
	const FName DstImportClassPackage(TEXT("/Engine/DstClassPackage"));


	TMap<FString, FString> MountPointReplacementMap =
	{
		{ SrcMountName, DstMountName },
	};

	TMap<FString, FString> PackageRenameMap =
	{
		{ SrcPackagePath, DstPackagePath },
		{ SrcNumberChangedPath, DstNumberChangedPath},
	};

	auto MakeImport = [](const FName ObjectName, const FPackageIndex OuterIndex, const FName ClassPackage, const FName ClassName, const FName PackageName)
		{
			FObjectImport Import;
			Import.ObjectName = ObjectName;
			Import.OuterIndex = OuterIndex;
#if WITH_EDITORONLY_DATA
			Import.OldClassName = NAME_None;
			Import.PackageName = PackageName;
#endif
			Import.ClassPackage = ClassPackage;
			Import.ClassName = ClassName;
			return Import;
		};

	auto MakeExport = [](const FName ObjectName, const FPackageIndex ThisIndex, const FPackageIndex OuterIndex,
		const FPackageIndex SuperIndex, const FPackageIndex ClassIndex, const FPackageIndex TemplateIndex)
		{
			FObjectExport Export;
			Export.ObjectName = ObjectName;
			Export.ThisIndex = ThisIndex;
			Export.OuterIndex = OuterIndex;
			Export.SuperIndex = SuperIndex;
			Export.ClassIndex = ClassIndex;
			Export.TemplateIndex = TemplateIndex;
#if WITH_EDITORONLY_DATA
			Export.OldClassName = NAME_None;
#endif
			return Export;
		};

	struct FImportTestCase
	{
		FObjectImport Src;
		FObjectImport Dst;
		bool bExistingImport;
	};
	// Note the order of these cases defines the ImportTable entry order before/after patching
	TArray<FImportTestCase> ImportTestCases
	{
		// /Script/SrcEngineModule -> (remains unchanged)
		{	
			.Src = MakeImport(SrcEngineModuleImportObjectName, FPackageIndex(), GLongCoreUObjectPackageName, NAME_Package, SrcPackagePathFName),
			.Dst = MakeImport(SrcEngineModuleImportObjectName, FPackageIndex(), GLongCoreUObjectPackageName, NAME_Package, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeA	-> /Script/DstEngineModule.DstTypeA
		{ 
			.Src = MakeImport(SrcTypeAImportObjectName, FPackageIndex::FromImport(0), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(DstTypeAImportObjectName, FPackageIndex::FromImport(18), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeA.SrcPropertyToChangeA_2 -> /Script/DstEngineModule.DstTypeA.DstPropertyToChangeA_4
		{ 
			.Src = MakeImport(SrcPropertyToChangeAImportObjectName, FPackageIndex::FromImport(1), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(DstPropertyToChangeAImportObjectName, FPackageIndex::FromImport(1), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty -> /Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty
		{ 
			.Src = MakeImport(MovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(1), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(MovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(1), DstImportClassPackage, DstImportClassName, DstPackagePathFName), 
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty.InnerMovedButNotRenamedProperty
		// -> 
		// /Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty.InnerMovedButNotRenamedProperty
		{
			.Src = MakeImport(InnerMovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(3), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName),
			.Dst = MakeImport(InnerMovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(3), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty.InnerMovedButNotRenamedProperty.InnerInnerMovedButNotRenamedProperty 
		// -> 
		// /Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty.InnerMovedButNotRenamedProperty.InnerInnerMovedButNotRenamedProperty 
		{
			.Src = MakeImport(InnerInnerMovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(4), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName),
			.Dst = MakeImport(InnerInnerMovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(4), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.OnlySubTypeChanged -> (remains unchanged)
		{ 
			.Src = MakeImport(OnlySubTypeChangedImportObjectName, FPackageIndex::FromImport(0), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(OnlySubTypeChangedImportObjectName, FPackageIndex::FromImport(0), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.OnlySubTypeChanged.SrcPropertyToChangeB -> /Script/SrcEngineModule.OnlySubTypeChanged.DstPropertyToChangeB
		{ 
			.Src = MakeImport(SrcPropertyToChangeBImportObjectName, FPackageIndex::FromImport(6), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(DstPropertyToChangeBImportObjectName, FPackageIndex::FromImport(6), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.OnlySubTypeChanged.UnchangedProperty -> (remains unchanged)
		{ 
			.Src = MakeImport(UnchangedPropertyImportObjectName, FPackageIndex::FromImport(6), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(UnchangedPropertyImportObjectName, FPackageIndex::FromImport(6), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeB	-> /Script/SrcEngineModule.DstTypeB
		{ 
			.Src = MakeImport(SrcTypeBImportObjectName, FPackageIndex::FromImport(0), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(DstTypeBImportObjectName, FPackageIndex::FromImport(0), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeB.MovedButNotRenamedProperty -> /Script/SrcEngineModule.DstTypeB.MovedButNotRenamedProperty
		{ 
			.Src = MakeImport(MovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(9), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(MovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(9), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.SrcTypeB.MovedToNewOuter	-> /Script/SrcEngineModule.NewOuter.MovedToNewOuter
		{
			.Src = MakeImport(MovedToNewOuterImportObjectName, FPackageIndex::FromImport(9), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName),
			.Dst = MakeImport(MovedToNewOuterImportObjectName, FPackageIndex::FromImport(19), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.MovedButNotRenamedType -> /Script/DstEngineModule.MovedButNotRenamedType
		{ 
			.Src = MakeImport(MovedButNotRenamedTypeImportObjectName, FPackageIndex::FromImport(0), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(MovedButNotRenamedTypeImportObjectName, FPackageIndex::FromImport(18), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Script/SrcEngineModule.MovedButNotRenamedType.MovedButNotRenamedProperty -> /Script/DstEngineModule.MovedButNotRenamedType.MovedButNotRenamedProperty
		{ 
			.Src = MakeImport(MovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(12), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName), 
			.Dst = MakeImport(MovedButNotRenamedPropertyImportObjectName, FPackageIndex::FromImport(12), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// Note, the verse transformations test a case where package is moved into a new package with a different path, while at the same time renaming top-level assets 
		// in the new path. We want to ensure we don't keep using the old top-level asset name. These redirections are done using CoreRedirects from the context
		// rather than explicit remappings in the patcher.
		
		// /Module/_Verse/VerseAsset -> /Module/_Verse
		{
			.Src = MakeImport(SrcVerseAssetName, FPackageIndex(), GLongCoreUObjectPackageName, NAME_Package, SrcPackagePathFName),
			.Dst = MakeImport(DstVerseAssetName, FPackageIndex(), GLongCoreUObjectPackageName, NAME_Package, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Module/_Verse/VerseAsset.some_verse_class -> /Module/_Verse.VerseAsset-some_verse_class
		{
			.Src = MakeImport(SrcVerseImportObjectName, FPackageIndex::FromImport(14), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName),
			.Dst = MakeImport(DstVerseImportObjectName, FPackageIndex::FromImport(14), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Module/_Verse/VerseAsset.some_verse_class.__verse_0x7A8CDEBC_VerseObject1 
		// -> 
		// /Module/_Verse.VerseAsset-some_verse_class.__verse_0x7A8CDEBC_VerseObject1
		{
			.Src = MakeImport(UnchangedVerseImportSubObject1Name, FPackageIndex::FromImport(15), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName),
			.Dst = MakeImport(UnchangedVerseImportSubObject1Name, FPackageIndex::FromImport(15), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// /Module/_Verse/VerseAsset.some_verse_class.__verse_0x7A8CDEBC_VerseObject1.__verse_0x5614AC82_VerseObject2
		// ->
		// /Module/_Verse.VerseAsset-some_verse_class.__verse_0x7A8CDEBC_VerseObject1.__verse_0x5614AC82_VerseObject2
		{
			.Src = MakeImport(UnchangedVerseImportSubObject2Name, FPackageIndex::FromImport(16), SrcImportClassPackage, SrcImportClassName, SrcPackagePathFName),
			.Dst = MakeImport(UnchangedVerseImportSubObject2Name, FPackageIndex::FromImport(16), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = true,
		},
		// <none> -> /Script/DstEngineModule
		{ 
			.Src = FObjectImport(), 
			.Dst = MakeImport(DstEngineModuleImportObjectName, FPackageIndex(), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = false,
		},
		// <none> -> /Script/SrcEngineModule.NewOuter
		{
			.Src = FObjectImport(),
			.Dst = MakeImport(NewOuterImportObjectName, FPackageIndex::FromImport(0), DstImportClassPackage, DstImportClassName, DstPackagePathFName),
			.bExistingImport = false,
		},
	};

	struct FExportTestCase
	{
		FObjectExport Src;
		FObjectExport Dst;
	};
	TArray<FExportTestCase> ExportTestCases
	{
		// SrcPackage																-> DstPackage
		// Super: /Script/SrcEngineModule.Package									-> /Script/DstEngineModule.Package
		// Class: /Script/SrcEngineModule.SrcTypeA.SrcPropertyToChangeA_2			-> /Script/DstEngineModule.DstTypeA.DstPropertyToChangeA_4
		// Template: /Script/SrcEngineModule.OnlySubTypeChanged						-> /Script/SrcEngineModule.OnlySubTypeChanged
		{
			.Src = MakeExport(SrcExportObjectFName, FPackageIndex::FromExport(0), FPackageIndex(), FPackageIndex::FromImport(0), FPackageIndex::FromImport(2), FPackageIndex::FromImport(4)),
			.Dst = MakeExport(DstExportObjectFName, FPackageIndex::FromExport(0), FPackageIndex(), FPackageIndex::FromImport(11), FPackageIndex::FromImport(2), FPackageIndex::FromImport(4))
		},
		// SrcPackage.SrcSubObject														-> DstPackage.DstSubObject
		// Super: /Script/SrcEngineModule.SrcTypeA										-> /Script/DstEngineModule.DstTypeA
		// Class: /Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty			-> /Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty
		// Template: /Script/SrcEngineModule.OnlySubTypeChanged.SrcPropertyToChangeB	-> /Script/SrcEngineModule.OnlySubTypeChanged.DstPropertyToChangeB
		{
			.Src = MakeExport(SrcExportSubObjectFName, FPackageIndex::FromExport(1), FPackageIndex::FromExport(0), FPackageIndex::FromImport(1), FPackageIndex::FromImport(3), FPackageIndex::FromImport(5)),
			.Dst = MakeExport(DstExportSubObjectFName, FPackageIndex::FromExport(1), FPackageIndex::FromExport(0), FPackageIndex::FromImport(1), FPackageIndex::FromImport(3), FPackageIndex::FromImport(5))
		},
		// SrcPackage.OnlyNumberChange_1												-> DstPackage.OnlyNumberChange_2
		// Super: /Script/SrcEngineModule.SrcTypeA										-> /Script/DstEngineModule.DstTypeA
		// Class: /Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty			-> /Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty
		// Template: /Script/SrcEngineModule.OnlySubTypeChanged.SrcPropertyToChangeB	-> /Script/SrcEngineModule.OnlySubTypeChanged.DstPropertyToChangeB
		{
			.Src = MakeExport(SrcExportOnlyNumberChangesFName, FPackageIndex::FromExport(1), FPackageIndex::FromExport(0), FPackageIndex::FromImport(1), FPackageIndex::FromImport(3), FPackageIndex::FromImport(5)),
			.Dst = MakeExport(DstExportOnlyNumberChangesFName, FPackageIndex::FromExport(1), FPackageIndex::FromExport(0), FPackageIndex::FromImport(1), FPackageIndex::FromImport(3), FPackageIndex::FromImport(5))
		}
	};

	FCoreRedirectsContext TestRedirectContext;
	TestRedirectContext.InitializeContext();
	CHECK(TestRedirectContext.IsInitialized());
	FCoreRedirectsContext& OriginalContext = FCoreRedirectsContext::GetThreadContext();
	FCoreRedirectsContext::SetThreadContext(TestRedirectContext);
	ON_SCOPE_EXIT{ FCoreRedirectsContext::SetThreadContext(OriginalContext); };

	FTestPatcherContext Context(PackageRenameMap, false /*bGatherDependentPackages*/);
	const TMap<FString, FString>& VerseStringReplacements = Context.GetVerseStringReplacements();
	CHECK(VerseStringReplacements.Num() > PackageRenameMap.Num()); // Ensure we generated more mappings off of the PackageRenameMap
	CHECK(FCoreRedirects::AddRedirectList(Context.GetRedirects(), TEXT("Asset Header Patcher Tests")));

	// Add Global CoreRedirects (scoped within the test context)
	{
		FCoreRedirect TagClassRedirect(ECoreRedirectFlags::Type_Class, TEXT("/Script/VerseGameplayTags.VerseTagMarkupComponent"), TEXT("/Script/VerseTags.VerseTagMarkupComponent"));
		FCoreRedirects::AddRedirectList({ TagClassRedirect }, TEXT("Asset Header Patcher Tests - Additional Redirects"));
	}

	FAssetHeaderPatcherInner Patcher(DummySrcDstAsset, DummySrcDstAsset, VerseStringReplacements, MountPointReplacementMap);

	auto AddToNameTable = [&Patcher](FName Name)
		{
			Patcher.NameToIndexMap.Add(Name.GetDisplayIndex(), Patcher.NameTable.Num());
			Patcher.NameTable.Add(Name);
		};
	int32 OriginalNameTableCount = 0;
	auto ResetPatcher = [&]()
		{
			// Reset Patcher
			Patcher.ResetInternalState();

			// Repopulate patcher state with our test data normally set through deserialization
			///////////////////////////////////////////////////////////////////////////////////
			
			// NameTable
			AddToNameTable(SrcPackagePathFName);
			AddToNameTable(SrcNumberChangedPathFName);
			AddToNameTable(SrcAssetFName);
			AddToNameTable(DummyImportPackagePathFName);
			// Redirected names (softpaths don't include the subpath in the name table
			AddToNameTable(SoftObjectPathToRedirect.GetAssetPath().GetPackageName());
			AddToNameTable(SoftObjectPathToRedirect.GetAssetPath().GetAssetName());
			// Export Table Names
			AddToNameTable(SrcExportObjectFName);
			AddToNameTable(SrcExportSubObjectFName);
			AddToNameTable(SrcExportOnlyNumberChangesFName);
			// Import Table Names
			AddToNameTable(SrcEngineModuleImportObjectName);
			AddToNameTable(SrcTypeAImportObjectName);
			AddToNameTable(SrcTypeBImportObjectName);
			AddToNameTable(OnlySubTypeChangedImportObjectName);
			AddToNameTable(MovedButNotRenamedTypeImportObjectName);
			AddToNameTable(SrcPropertyToChangeAImportObjectName);
			AddToNameTable(SrcPropertyToChangeBImportObjectName);
			AddToNameTable(MovedButNotRenamedPropertyImportObjectName);
			AddToNameTable(NewOuterImportObjectName);
			AddToNameTable(MovedToNewOuterImportObjectName);
			AddToNameTable(InnerMovedButNotRenamedPropertyImportObjectName);
			AddToNameTable(InnerInnerMovedButNotRenamedPropertyImportObjectName);
			AddToNameTable(UnchangedPropertyImportObjectName);
			AddToNameTable(SrcVerseAssetName);
			AddToNameTable(SrcVerseImportObjectName);
			AddToNameTable(UnchangedVerseImportSubObject1Name);
			AddToNameTable(UnchangedVerseImportSubObject2Name);
			AddToNameTable(SrcImportClassName);
			AddToNameTable(SrcImportClassPackage);
			AddToNameTable(GLongCoreUObjectPackageName);
			AddToNameTable(NAME_Package);

			auto CheckNameTableInit = [&Patcher](FName Name)
				{
					// Skip lookups for None, as we already verified the NameTable 
					// does not / should not contains None
					if (Name == NAME_None)
					{
						return;
					}
					CHECK(Patcher.NameToIndexMap.Contains(Name.GetDisplayIndex()));
					CHECK(Patcher.NameTable[Patcher.NameToIndexMap[Name.GetDisplayIndex()]] == Name);
				};
			CHECK(!Patcher.NameToIndexMap.Contains(FName(NAME_None).GetDisplayIndex()));
			CHECK(!Patcher.NameTable.Contains(NAME_None));
			// Import/Export Table (see breakdown of import names for the intended tests in the FObjectResource test section below)
			for (const FImportTestCase& TestCase : ImportTestCases)
			{
				const FObjectImport& Import = TestCase.Src;
				if (Import.ObjectName == NAME_None)
				{
					// We have more cases than initial starting states so if we see an empty
					// name we can stop adding to the initial state
					break;
				}
				CheckNameTableInit(Import.ObjectName);
				CheckNameTableInit(Import.ClassName);
				CheckNameTableInit(Import.ClassPackage);
#if WITH_EDITORONLY_DATA
				CheckNameTableInit(Import.PackageName);
				CheckNameTableInit(Import.OldClassName);
#endif
				Patcher.ImportTable.Add(Import);
			}

			for (const FExportTestCase& TestCase : ExportTestCases)
			{
				const FObjectExport& Export = TestCase.Src;
				CheckNameTableInit(Export.ObjectName);
#if WITH_EDITORONLY_DATA
				CheckNameTableInit(Export.OldClassName);
#endif
				Patcher.ExportTable.Add(Export);
			}		

			// Summary
			Patcher.Summary.NameCount = Patcher.NameTable.Num();
			Patcher.OriginalPackagePath = SrcPackagePathFName;
			OriginalNameTableCount = Patcher.NameTable.Num();
		};

	SECTION("FContext Additional Remappings")
	{
		{
			FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage)"));
			const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage)"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("Patch string with direct match"), Actual, Expected);
		}

		{
			FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage.SrcPackage)"));
			const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage.DstPackage)"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("Generated Top-Level Asset mapping"), Actual, Expected);
		}

		{
			FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage.SrcPackage_C)"));
			const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage.DstPackage_C)"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("Generated Blueprint Generated Class mapping"), Actual, Expected);
		}

		{
			FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage.Default__SrcPackage_C)"));
			const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage.Default__DstPackage_C)"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("Generated Blueprint Generated Class Default Object mapping"), Actual, Expected);
		}

		{
			FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage.SrcPackageEditorOnlyData)"));
			const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage.DstPackageEditorOnlyData)"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("Generated MaterialFunctionInterface Editor Only Data mapping"), Actual, Expected);
		}
		
		{
			FString Actual(TEXT(R"(/SrcMount/SomePath/UnchangingName_1)"));
			const FString Expected(TEXT(R"(/DstMount/SomePath/UnchangingName_2)"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("Patch string with direct match"), Actual, Expected);
		}

		SECTION("Verse Mountpoints")
		{
			for (const FString& VerseMount : Context.GetVerseMountPoints())
			{
				// We only generate verse paths for objects, so this package path will not have a mapping
				{
					FString Actual = FString::Printf(TEXT(R"(/%s/SrcMount/SomePath/SrcPackage)"), *VerseMount);
					const FString Expected = FString::Printf(TEXT(R"(/%s/DstMount/SomePath/DstPackage)"), *VerseMount);
					CHECK(!Patcher.DoPatch(Actual));
					CHECK_NOT_EQUALS(TEXT("Patch string with direct match"), Actual, Expected);
				}

				{
					FString Actual = FString::Printf(TEXT(R"(/%s/SrcMount/SomePath/SrcPackage/SrcPackage)"), *VerseMount);
					const FString Expected = FString::Printf(TEXT(R"(/%s/DstMount/SomePath/DstPackage/DstPackage)"), *VerseMount);
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch string with direct match"), Actual, Expected);
				}

				{
					FString Actual = FString::Printf(TEXT(R"(/%s/SrcMount/SomePath/SrcPackage/SrcPackage)"), *VerseMount);
					const FString Expected = FString::Printf(TEXT(R"(/%s/DstMount/SomePath/DstPackage/DstPackage)"), *VerseMount);
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Generated Top-Level Asset mapping"), Actual, Expected);
				}

				{
					FString Actual = FString::Printf(TEXT(R"(/%s/SrcMount/SomePath/SrcPackage/SrcPackage_C)"), *VerseMount);
					const FString Expected = FString::Printf(TEXT(R"(/%s/DstMount/SomePath/DstPackage/DstPackage_C)"), *VerseMount);
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Generated Blueprint Generated Class mapping"), Actual, Expected);
				}

				{
					FString Actual = FString::Printf(TEXT(R"(/%s/SrcMount/SomePath/SrcPackage/Default__SrcPackage_C)"), *VerseMount);
					const FString Expected = FString::Printf(TEXT(R"(/%s/DstMount/SomePath/DstPackage/Default__DstPackage_C)"), *VerseMount);
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Generated Blueprint Generated Class Default Object mapping"), Actual, Expected);
				}

				{
					FString Actual = FString::Printf(TEXT(R"(/%s/SrcMount/SomePath/SrcPackage/SrcPackageEditorOnlyData)"), *VerseMount);
					const FString Expected = FString::Printf(TEXT(R"(/%s/DstMount/SomePath/DstPackage/DstPackageEditorOnlyData)"), *VerseMount);
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Generated MaterialFunctionInterface Editor Only Data mapping"), Actual, Expected);
				}
			}
		}
	}

	SECTION("DoPatch(FString)")
	{
		SECTION("Direct match")
		{
			{
				FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage)"));
				const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch string with direct match"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"(/SrcMount/SomePath/UnchangingName_1)"));
				const FString Expected(TEXT(R"(/DstMount/SomePath/UnchangingName_2)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch string with direct match"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage2)"));
				const FString Expected = Actual; // Must be a copy
				CHECK(!Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch string with no direct match"), Actual, Expected);
			}

			// Do not replace strings that happen to overlap with ObjectNames that are remapped
			{
				FString Actual(TEXT(R"(SrcPackage)"));
				const FString Expected(TEXT(R"(SrcPackage)"));
				CHECK(!Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Do not remap a string that matches a non-fully-qualified ObjectName"), Actual, Expected);
			}
		}

		SECTION("Sub-Object Paths")
		{
			{
				FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage.SrcPackage:AnOuter.To.A.SubObject)"));
				const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage.DstPackage:AnOuter.To.A.SubObject)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch sub-object path"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"(/SrcMount/SomePath/UnchangingName_1.UnchangingName_1:AnOuter.To.A.SubObject)"));
				const FString Expected(TEXT(R"(/DstMount/SomePath/UnchangingName_2.UnchangingName_2:AnOuter.To.A.SubObject)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch sub-object path"), Actual, Expected);
			}

			// Worth adding support for in the future, but at the moment we cannot patch various parts of unquoted 
			// sub-object paths (that are specifically strings in the header, FNames are fine). In this case we 
			// can't patch the package path because the top-level asset (UnmappedObject) has no mapping for patching
			{
				FString Actual(TEXT(R"(/SrcMount/SomePath/SrcPackage.UnmappedObject:AnOuter.To.A.SubObject)"));
				const FString Expected(TEXT(R"(/DstMount/SomePath/DstPackage.UnmappedObject:AnOuter.To.A.SubObject)"));
				CHECK(!Patcher.DoPatch(Actual));
				CHECK_NOT_EQUALS(TEXT("Can't patch sub-object paths, for "), Actual, Expected);
			}
		}

		SECTION("Quoted match")
		{
			SECTION("Single Quote")
			{
				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/SrcPackage')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/DstPackage')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch package path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/SrcPackage.SrcPackage')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/DstPackage.DstPackage')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch object path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/SrcPackage.SrcPackage_C')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/DstPackage.DstPackage_C')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch blueprint generated class with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/SrcPackage.Default__SrcPackage_C')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/DstPackage.Default__DstPackage_C')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch default blueprint generated class object path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/UnchangingName_1')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/UnchangingName_2')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch package path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/UnchangingName_1.UnchangingName_1')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/UnchangingName_2.UnchangingName_2')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch object path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/UnchangingName_1.UnchangingName_1_C')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/UnchangingName_2.UnchangingName_2_C')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch blueprint generated class with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"('/SrcMount/SomePath/UnchangingName_1.Default__UnchangingName_1_C')"));
					const FString Expected(TEXT(R"('/DstMount/SomePath/UnchangingName_2.Default__UnchangingName_2_C')"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch default blueprint generated class object path with quotes"), Actual, Expected);
				}

				// Do not replace strings that happen to overlap with ObjectNames that are remapped
				{
					FString Actual(TEXT(R"('SrcPackage')"));
					const FString Expected(TEXT(R"('SrcPackage')"));
					CHECK(!Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Do not remap a string that matches a non-fully-qualified ObjectName"), Actual, Expected);
				}
			}

			SECTION("Double Quote")
			{
				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/SrcPackage")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/DstPackage")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch package path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/SrcPackage.SrcPackage")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/DstPackage.DstPackage")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch object path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/SrcPackage.SrcPackage_C")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/DstPackage.DstPackage_C")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch blueprint generated class with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/SrcPackage.Default__SrcPackage_C")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/DstPackage.Default__DstPackage_C")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch default blueprint generated class object path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/UnchangingName_1")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/UnchangingName_2")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch package path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/UnchangingName_1.UnchangingName_1")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/UnchangingName_2.UnchangingName_2")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch object path with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/UnchangingName_1.UnchangingName_1_C")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/UnchangingName_2.UnchangingName_2_C")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch blueprint generated class with quotes"), Actual, Expected);
				}

				{
					FString Actual(TEXT(R"("/SrcMount/SomePath/UnchangingName_1.Default__UnchangingName_1_C")"));
					const FString Expected(TEXT(R"("/DstMount/SomePath/UnchangingName_2.Default__UnchangingName_2_C")"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch default blueprint generated class object path with quotes"), Actual, Expected);
				}

				// Do not replace strings that happen to overlap with ObjectNames that are remapped
				{
					FString Actual(TEXT(R"("SrcPackage")"));
					const FString Expected(TEXT(R"("SrcPackage")"));
					CHECK(!Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Do not remap a string that matches a non-fully-qualified ObjectName"), Actual, Expected);
				}
			}

			SECTION("Substring match")
			{
				{
					FString Actual(
						TEXT(R"(((ReferenceNodePath="/SrcMount/SomePath/SrcPackage.SrcPackage:RigVMModel.Setup Arm",)")
						TEXT(R"(((Package="/SrcMount/SomePath/SrcPackage",)")
						TEXT(R"(HostObject="/SrcMount/SomePath/SrcPackage.SrcPackage_C")))"));
					FString Expected(
						TEXT(R"(((ReferenceNodePath="/DstMount/SomePath/DstPackage.DstPackage:RigVMModel.Setup Arm",)")
						TEXT(R"(((Package="/DstMount/SomePath/DstPackage",)")
						TEXT(R"(HostObject="/DstMount/SomePath/DstPackage.DstPackage_C")))"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch substring with quoted package, object and sub-object paths"), Actual, Expected);
				}

				{
					FString Actual(
						TEXT(R"(((ReferenceNodePath="/SrcMount/SomePath/UnchangingName_1.UnchangingName_1:RigVMModel.Setup Arm",)")
						TEXT(R"(((Package="/SrcMount/SomePath/UnchangingName_1",)")
						TEXT(R"(HostObject="/SrcMount/SomePath/UnchangingName_1.UnchangingName_1_C")))"));
					FString Expected(
						TEXT(R"(((ReferenceNodePath="/DstMount/SomePath/UnchangingName_2.UnchangingName_2:RigVMModel.Setup Arm",)")
						TEXT(R"(((Package="/DstMount/SomePath/UnchangingName_2",)")
						TEXT(R"(HostObject="/DstMount/SomePath/UnchangingName_2.UnchangingName_2_C")))"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch substring with quoted package, object and sub-object paths"), Actual, Expected);
				}

				// Do not replace strings that happen to overlap with ObjectNames that are remapped(i.e "SrcPackage=" is not transformed to "DstPackage=") 
				{
					FString Actual(
						TEXT(R"(((SrcPackage="/SrcMount/SomePath/SrcPackage.SrcPackage:RigVMModel.Setup Arm",)")
						TEXT(R"(((SrcPackage="/SrcMount/SomePath/SrcPackage",)")
						TEXT(R"(SrcPackage="/SrcMount/SomePath/SrcPackage.SrcPackage_C")))"));
					FString Expected(
						TEXT(R"(((SrcPackage="/DstMount/SomePath/DstPackage.DstPackage:RigVMModel.Setup Arm",)")
						TEXT(R"(((SrcPackage="/DstMount/SomePath/DstPackage",)")
						TEXT(R"(SrcPackage="/DstMount/SomePath/DstPackage.DstPackage_C")))"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch substring with quoted package, object and sub-object paths. No non-fully-qualified ObjectNames are patched."), Actual, Expected);
				}

				// Relies on the TagClassRedirect defined above
				{
					FString Actual(TEXT(R"(CacheVersion=1,AllowedClassReferences=("/Script/VerseGameplayTags.VerseTagMarkupComponent"))"));
					FString Expected(TEXT(R"(CacheVersion=1,AllowedClassReferences=("/Script/VerseTags.VerseTagMarkupComponent"))"));
					CHECK(Patcher.DoPatch(Actual));
					CHECK_EQUALS(TEXT("Patch substring with quoted package, object and sub-object paths. No non-fully-qualified ObjectNames are patched."), Actual, Expected);
					
				}
			}
		}

		SECTION("Mountpoint match")
		{
			/*
				We currently don't support mount point replacement _for strings_ that don't
				provide some kind of delimiter for us to scan for. As such package paths
				and top-level asset paths are not supported unless they are quoted. Sub-object
				paths are supported.
			*/
			/*
			{
				FString Actual(TEXT(R"(/SourceSpecialMount/SomePath/SomePackage)"));
				const FString Expected(TEXT(R"(/DestinationSpecialMount/SomePath/SomePackage)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch package path replaces only mount"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"(/SourceSpecialMount/SomePath/SomePackage.SomePackage)"));
				const FString Expected(TEXT(R"(/DestinationSpecialMount/SomePath/SomePackage.SomePackage)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch package object path replaces only mount"), Actual, Expected);
			}
			*/

			{
				FString Actual(TEXT(R"(/SourceSpecialMount/SomePath/SomePackage.TopLevel:SubObject1.SubObject2)"));
				const FString Expected(TEXT(R"(/DestinationSpecialMount/SomePath/SomePackage.TopLevel:SubObject1.SubObject2)"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch package sub-object path replaces only mount"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"("/SourceSpecialMount/SomePath/SomePackage")"));
				const FString Expected(TEXT(R"("/DestinationSpecialMount/SomePath/SomePackage")"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch double quoted path replaces only mount"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"('/SourceSpecialMount/SomePath/SomePackage')"));
				const FString Expected(TEXT(R"('/DestinationSpecialMount/SomePath/SomePackage')"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Patch single quoted path replaces only mount"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"(SomePrefix="/SourceSpecialMount/SomePath/SomePackage")"));
				const FString Expected(TEXT(R"(SomePrefix="/DestinationSpecialMount/SomePath/SomePackage")"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Substring patch replaces only mount when double quoted"), Actual, Expected);
			}

			{
				FString Actual(TEXT(R"(SomePrefix='/SourceSpecialMount/SomePath/SomePackage')"));
				const FString Expected(TEXT(R"(SomePrefix='/DestinationSpecialMount/SomePath/SomePackage')"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Substring patch replaces only mount when single quoted"), Actual, Expected);
			}

			{
				FString Actual(
					TEXT(R"("/SourceSpecialMount/SomePath/SomePackage1",)")
					TEXT(R"("/SourceSpecialMount/SomePath/SomePackage2")"));
				const FString Expected(
					TEXT(R"("/DestinationSpecialMount/SomePath/SomePackage1",)")
					TEXT(R"("/DestinationSpecialMount/SomePath/SomePackage2")"));
				CHECK(Patcher.DoPatch(Actual));
				CHECK_EQUALS(TEXT("Substring patch replaces mount in multiple double quoted paths"), Actual, Expected);
			}
		}
	}

	SECTION("DoPatch(FSoftObjectPath)")
	{
		{
			ResetPatcher();

			FSoftObjectPath Actual(TEXT("/SrcMount/SomePath/SrcPackage.SrcPackage"));
			FSoftObjectPath Expected(TEXT("/DstMount/SomePath/DstPackage.DstPackage"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("SoftObjectPath patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			CHECK_EQUALS(TEXT("SoftObject patching updates NameTable entry"), Patcher.NameTable[0], DstPackagePathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		{
			ResetPatcher();

			FSoftObjectPath Actual(TEXT("/SrcMount/SomePath/SrcPackage.SrcPackage:Some.SrcPackage.Subobject"));
			// Note we do not replace the sub-object "SrcPackage" despite it matching the original package and object name
			FSoftObjectPath Expected(TEXT("/DstMount/SomePath/DstPackage.DstPackage:Some.SrcPackage.Subobject"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("SoftObjectPath with sub-object path patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			CHECK_EQUALS(TEXT("SoftObject patching updates NameTable entry"), Patcher.NameTable[0], DstPackagePathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		{
			ResetPatcher();

			FSoftObjectPath Actual(TEXT("/SrcMount/SomePath/UnchangingName_1.UnchangingName_1"));
			FSoftObjectPath Expected(TEXT("/DstMount/SomePath/UnchangingName_2.UnchangingName_2"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("SoftObjectPath patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the NameTable"), Patcher.NameTable[1], SrcNumberChangedPathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			// Numbered FNames have no number in the NameTable
			CHECK_EQUALS(TEXT("SoftObject patching updates NameTable entry"), Patcher.NameTable[1].GetDisplayIndex(), DstNumberChangedPathFName.GetDisplayIndex());
			CHECK_EQUALS(TEXT("SoftObject patching updates NameTable entry"), Patcher.NameTable[1].GetNumber(), 0);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		{
			ResetPatcher();

			FSoftObjectPath Actual(TEXT("/SrcMount/SomePath/UnchangingName_1.UnchangingName_1:Some.UnchangingName_1.Subobject"));
			// Note we do not replace the sub-object "UnchangingName_1" despite it matching the original package and object name
			FSoftObjectPath Expected(TEXT("/DstMount/SomePath/UnchangingName_2.UnchangingName_2:Some.UnchangingName_1.Subobject"));
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("SoftObjectPath with sub-object path patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the NameTable"), Patcher.NameTable[1], SrcNumberChangedPathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			// Numbered FNames have no number in the NameTable
			CHECK_EQUALS(TEXT("SoftObject patching updates NameTable entry"), Patcher.NameTable[1].GetDisplayIndex(), DstNumberChangedPathFName.GetDisplayIndex());
			CHECK_EQUALS(TEXT("SoftObject patching updates NameTable entry"), Patcher.NameTable[1].GetNumber(), 0);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

#if WITH_EDITOR
		SECTION("DoPatch(FSoftObjectPath)-WithRedirector")
		{
			ResetPatcher();

			GRedirectCollector.AddAssetPathRedirection(SoftObjectPathToRedirect, RedirectedSoftObjectPath);
			ON_SCOPE_EXIT
			{
				GRedirectCollector.RemoveAssetPathRedirection(SoftObjectPathToRedirect);
			};

			FSoftObjectPath Actual = SoftObjectPathToRedirect;
			FSoftObjectPath Expected = RedirectedSoftObjectPath;
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("SoftObjectPath with sub-object path patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		SECTION("DoPatch(FSoftObjectPath)-WithRedirectorAndExplicitPatch")
		{
			ResetPatcher();

			// Note SrcPackageObjectPath is NOT being redirected to DstPackageObjectPath (that mapping 
			// provided to the patcher explicitly already)
			FSoftObjectPath NameToRedirect(SrcPackageObjectPath);
			GRedirectCollector.AddAssetPathRedirection(NameToRedirect, RedirectedSoftObjectPath);
			ON_SCOPE_EXIT
			{
				GRedirectCollector.RemoveAssetPathRedirection(NameToRedirect);
			};

			// Even though we have a redirector, we also have a mapping specified to the patcher. 
			// We give the patcher priority in such cases
			FSoftObjectPath Actual = NameToRedirect;
			FSoftObjectPath Expected(DstPackageObjectPath); 
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("SoftObjectPath with sub-object path patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			CHECK_EQUALS(TEXT("SoftObject patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}
#endif
	}

	SECTION("DoPatch(FTopLevelAssetPath)")
	{
		{
			ResetPatcher();

			FTopLevelAssetPath Actual(SrcPackagePath, SrcAssetFName);
			FTopLevelAssetPath Expected(DstPackagePath, DstAssetFName);
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching updates NameTable entry"), Patcher.NameTable[0], DstPackagePathFName);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		{
			ResetPatcher();

			FTopLevelAssetPath Actual(SrcPackageObjectPath);
			FTopLevelAssetPath Expected(DstPackageObjectPath);
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching updates NameTable entry"), Patcher.NameTable[0], DstPackagePathFName);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		{
			ResetPatcher();

			FTopLevelAssetPath Actual(SrcNumberChangedPath, SrcNumberAssetFName);
			FTopLevelAssetPath Expected(DstNumberChangedPath, DstNumberAssetFName);
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching doesn't implicitly update the NameTable"), Patcher.NameTable[1], SrcNumberChangedPathFName);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			// Numbered FNames have no number in the NameTable
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching updates NameTable entry"), Patcher.NameTable[1].GetDisplayIndex(), DstNumberChangedPathFName.GetDisplayIndex());
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching updates NameTable entry"), Patcher.NameTable[1].GetNumber(), 0);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(FName,FName) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}

		{
			ResetPatcher();

			FTopLevelAssetPath Actual(SrcNumberChangedObjectPath);
			FTopLevelAssetPath Expected(DstNumberChangedObjectPath);
			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching"), Actual, Expected);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching doesn't implicitly update the NameTable"), Patcher.NameTable[1], SrcNumberChangedPathFName);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			Patcher.PatchNameTable();
			// Numbered FNames have no number in the NameTable
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching updates NameTable entry"), Patcher.NameTable[1].GetDisplayIndex(), DstNumberChangedPathFName.GetDisplayIndex());
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching updates NameTable entry"), Patcher.NameTable[1].GetNumber(), 0);
			CHECK_EQUALS(TEXT("TopLevelAssetPatch(string) patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
		}
	}

	SECTION("DoPatch(FGatherableTextData)")
	{
		{
			ResetPatcher();

			FGatherableTextData Actual;
			Actual.NamespaceName = SrcPackagePath;
			Actual.SourceData.SourceString = SrcPackagePath;
			FTextSourceSiteContext SrcSiteContext;
			SrcSiteContext.KeyName = SrcPackagePath;
			SrcSiteContext.SiteDescription = SrcPackagePath;
			Actual.SourceSiteContexts.Add(SrcSiteContext);

			FGatherableTextData Expected = Actual;
			Expected.SourceSiteContexts = TArray<FTextSourceSiteContext>();
			FTextSourceSiteContext DstSiteContext;
			DstSiteContext.KeyName = SrcPackagePath;
			DstSiteContext.SiteDescription = DstPackagePath;
			Expected.SourceSiteContexts.Add(DstSiteContext);

			CHECK(Patcher.DoPatch(Actual));
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't update NamespaceName"), Actual.NamespaceName, Expected.NamespaceName);
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't update SourceData.SourceString"), Actual.SourceData.SourceString, Expected.SourceData.SourceString);
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't update SourceSiteContexts[].KeyName"), Actual.SourceSiteContexts[0].KeyName, Expected.SourceSiteContexts[0].KeyName);
			CHECK_EQUALS(TEXT("FGatherableTextData patching does update SourceData.SourceString[].SiteDescription"), Actual.SourceSiteContexts[0].SiteDescription, Expected.SourceSiteContexts[0].SiteDescription);
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't implicitly update the NameTable"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't implicitly update the NameTable"), Patcher.NameTable[2], SrcAssetFName);
			Patcher.PatchNameTable();
			// FGatherableTexData doesn't contain FNames so we shouldn't have updated the NameTable at all
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't implicitly update the PackageFileSummary"), Patcher.Summary.NameCount, OriginalNameTableCount);
			CHECK_EQUALS(TEXT("FGatherableTextData patching updates NameTable entry"), Patcher.NameTable[0], SrcPackagePathFName);
			CHECK_EQUALS(TEXT("FGatherableTextData patching doesn't implicitly update the NameTable"), Patcher.NameTable[2], SrcAssetFName);
		}
	}

	SECTION("DoPatch(FTypeResource)")
	{
		ResetPatcher();
		TArray<FCoreRedirect> ImportTypeHierarchyRedirects =
		{
			{ ECoreRedirectFlags::Type_Package, TEXT("/Engine/SrcClassPackage"),                           TEXT("/Engine/DstClassPackage")},
			{ ECoreRedirectFlags::Type_Class,   TEXT("/Engine/SrcClassPackage.SrcClass"),                  TEXT("/Engine/DstClassPackage.DstClass")},
			{ ECoreRedirectFlags::Type_Package, TEXT("/Module/_Verse/VerseAsset"),                         TEXT("/Module/_Verse") },
			{ ECoreRedirectFlags::Type_Object,  TEXT("/Module/_Verse/VerseAsset.some_verse_class"),        TEXT("/Module/_Verse.VerseAsset-some_verse_class") },
		};
		CHECK(FCoreRedirects::AddRedirectList(ImportTypeHierarchyRedirects, TEXT("Asset Header Patcher Tests - FTypeResource Patching")));

		UE::Serialization::Private::FTypeResource Actual;
		Actual.TypeName = SrcVerseImportObjectName;
		Actual.PackageName = SrcVerseAssetName;
		Actual.ClassName = SrcImportClassName;
		Actual.ClassPackageName = SrcImportClassPackage;

		UE::Serialization::Private::FTypeResource Expected;
		Expected.TypeName = DstVerseImportObjectName;
		Expected.PackageName = DstVerseAssetName;
		Expected.ClassName = DstImportClassName;
		Expected.ClassPackageName = DstImportClassPackage;

		CHECK(Patcher.DoPatch(Actual));
		CHECK_EQUALS(TEXT("FTypeResource patching doesn't update TypeName"), Actual.TypeName, Expected.TypeName);
		CHECK_EQUALS(TEXT("FTypeResource patching doesn't update PackageName"), Actual.PackageName, Expected.PackageName);
		CHECK_EQUALS(TEXT("FTypeResource patching doesn't update ClassName"), Actual.ClassName, Expected.ClassName);
		CHECK_EQUALS(TEXT("FTypeResource patching doesn't update ClassPackageName"), Actual.ClassPackageName, Expected.ClassPackageName);
	}

	// Import Test Breakdown
	// 
	// Assume that SrcEngineModule and DstEngineModule both exist and types are moving between them and/or possibly being renamed while moving
	// 
	// Coreredirects are added to make the following transformations:
	// 
	// /Engine/SrcClassPackage                                                   -> /Engine/DstClassPackage
	// /Engine/SrcClassPackage.SrcClass                                          -> /Engine/DstClassPackage.DstClass
	// /Script/SrcEngineModule.SrcTypeA                                          -> /Script/DstEngineModule.DstTypeA
	// /Script/SrcEngineModule.SrcTypeA.SrcPropertyToChangeA_2                   -> /Script/DstEngineModule.DstTypeA.DstPropertyToChangeA_4
	// /Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty               -> /Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty
	// /Script/SrcEngineModule.OnlySubTypeChanged.SrcPropertyToChangeB           -> /Script/SrcEngineModule.OnlySubTypeChanged.DstPropertyToChangeB
	// /Script/SrcEngineModule.SrcTypeB                                          -> /Script/SrcEngineModule.DstTypeB
	// /Script/SrcEngineModule.SrcTypeB.MovedButNotRenamedProperty               -> /Script/SrcEngineModule.DstTypeB.MovedButNotRenamedProperty
	// /Script/SrcEngineModule.SrcTypeB.MovedToNewOuter                          -> /Script/SrcEngineModule.NewOuter.MovedToNewOuter
	// /Script/SrcEngineModule.MovedButNotRenamedType                            -> /Script/DstEngineModule.MovedButNotRenamedType
	// /Script/SrcEngineModule.MovedButNotRenamedType.MovedButNotRenamedProperty -> /Script/DstEngineModule.MovedButNotRenamedType.MovedButNotRenamedProperty
	// /Module/_Verse/VerseAsset                                                 -> /Module/_Verse
	// /Module/_Verse/VerseAsset.some_verse_class                                -> /Module/_Verse.VerseAsset-some_verse_class
	// 
	// Our ImportTable will be transformed from the left side to the right where each line is its own ImportEntry outered to the import "up and to the left" of it:
	// 
    // /Script/SrcEngineModule                                                                                          /Script/SrcEngineModule
    //                        .SrcTypeA                                                                                                       .OnlySubTypeChanged                                 
    //                                 .SrcPropertyToChangeA_2                                                                                                   .DstPropertyToChangeB             
    //                                  .MovedButNotRenamedProperty                                                                                                                   .UnchangedProperty    
    //                        .OnlySubTypeChanged                                                                                             .DstTypeB                                           
    //                                           .SrcPropertyToChangeB                                                                                 .MovedButNotRenamedProperty              
    //                                           .UnchangedProperty                                               -->                         .NewOuter
    //                        .SrcTypeB                                                                                                                .MovedToNewOuter
    //                                 .MovedButNotRenamedProperty                                                      /Script/DstEngineModule                       
	//                                 .MovedToNewOuter                                        							                       .DstTypeA
    //                        .MovedButNotRenamedType                                                                  	                                .DstPropertyToChangeA_4
    //                                               .MovedButNotRenamedProperty                                                                        .MovedButNotRenamedProperty                                               
    // /Module/_Verse/VerseAsset                                                                                       	                                .MovedButNotRenamedType                             
    //                          .some_verse_class                                                                      	                                                       .MovedButNotRenamedProperty
    //                                           .__verse_0x7A8CDEBC_VerseObject1                                      	/Module/_Verse         
    //                                                                           .__verse_0x5614AC82_VerseObject2      	              .VerseAsset-some_verse_class         
    //                                                                                                                 	                                          .__verse_0x7A8CDEBC_VerseObject1
    //                                                                                                                 	                                                                          .__verse_0x5614AC82_VerseObject2
	SECTION("FObjectResource Patching")
	{
		SECTION("Redirect object to new package keeps the original package name if still in use")
		{
			auto CheckFNames = [&Patcher](FName Expected, FName Actual)
				{
					CHECK(Expected == Actual);
					// It's fine to compare Expected == Actual when None but the NameTable
					// should not contain None, so don't look for it
					if (!Actual.IsNone())
					{
						FNameEntryId ActualNameEntryId = Actual.GetDisplayIndex();
						CHECK(Patcher.NameTable.ContainsByPredicate([ActualNameEntryId](const FName& Name)
							{
								return Name.GetDisplayIndex() == ActualNameEntryId;
							}));
					}
				};

			ResetPatcher();
			TArray<FCoreRedirect> ImportTableRedirects =
			{
				{ ECoreRedirectFlags::Type_Package, TEXT("/Engine/SrcClassPackage"),                                                   TEXT("/Engine/DstClassPackage")},
				{ ECoreRedirectFlags::Type_Class,   TEXT("/Engine/SrcClassPackage.SrcClass"),                                          TEXT("/Engine/DstClassPackage.DstClass")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.SrcTypeA"),                                          TEXT("/Script/DstEngineModule.DstTypeA")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.SrcTypeA.SrcPropertyToChangeA_2"),                   TEXT("/Script/DstEngineModule.DstTypeA.DstPropertyToChangeA_4")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.SrcTypeA.MovedButNotRenamedProperty"),               TEXT("/Script/DstEngineModule.DstTypeA.MovedButNotRenamedProperty")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.OnlySubTypeChanged.SrcPropertyToChangeB"),           TEXT("/Script/SrcEngineModule.OnlySubTypeChanged.DstPropertyToChangeB")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.SrcTypeB"),                                          TEXT("/Script/SrcEngineModule.DstTypeB")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.SrcTypeB.MovedButNotRenamedProperty"),               TEXT("/Script/SrcEngineModule.DstTypeB.MovedButNotRenamedProperty")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.SrcTypeB.MovedToNewOuter"),                          TEXT("/Script/SrcEngineModule.NewOuter.MovedToNewOuter")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.MovedButNotRenamedType"),                            TEXT("/Script/DstEngineModule.MovedButNotRenamedType")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Script/SrcEngineModule.MovedButNotRenamedType.MovedButNotRenamedProperty"), TEXT("/Script/DstEngineModule.MovedButNotRenamedType.MovedButNotRenamedProperty")},
				{ ECoreRedirectFlags::Type_Package, TEXT("/Module/_Verse/VerseAsset"),                                                 TEXT("/Module/_Verse") },
				{ ECoreRedirectFlags::Type_Object,  TEXT("/Module/_Verse/VerseAsset.some_verse_class"),                                TEXT("/Module/_Verse.VerseAsset-some_verse_class") },
				{ ECoreRedirectFlags::Type_Object,  TEXT("/SrcMount/SomePath/SrcPackage.SrcPackage:SrcSubObject"),                     TEXT("/DstMount/SomePath/DstPackage.DstPackage:DstSubObject")},
				{ ECoreRedirectFlags::Type_Object,  TEXT("/SrcMount/SomePath/SrcPackage.SrcPackage:OnlyNumberChange_1"),               TEXT("/SrcMount/SomePath/SrcPackage.SrcPackage:OnlyNumberChange_2")},
			};
			CHECK(FCoreRedirects::AddRedirectList(ImportTableRedirects, TEXT("Asset Header Patcher Tests - FObjectResource Patching")));

			// Confirm the initial state before patching is what we expect
			check(Patcher.ImportTable.Num() <= ImportTestCases.Num());
			for (int32 i = 0; i < Patcher.ImportTable.Num(); ++i)
			{
				const FObjectImport& Expected = ImportTestCases[i].Src;
				const FObjectImport& Actual = Patcher.ImportTable[i];

				CheckFNames(Expected.ObjectName, Actual.ObjectName);
				CHECK(Expected.OuterIndex == Actual.OuterIndex);
				CheckFNames(Expected.ClassName, Actual.ClassName);
				CheckFNames(Expected.ClassPackage, Actual.ClassPackage);
#if WITH_EDITORONLY_DATA
				//CheckFNames(Expected.PackageName, Actual.PackageName);
#endif
			}
			check(Patcher.ExportTable.Num() <= ExportTestCases.Num());
			for (int32 i = 0; i < Patcher.ExportTable.Num(); ++i)
			{
				const FObjectExport& Expected = ExportTestCases[i].Src;
				const FObjectExport& Actual = Patcher.ExportTable[i];

				CheckFNames(Expected.ObjectName, Actual.ObjectName);
				CHECK(Expected.OuterIndex == Actual.OuterIndex);
#if WITH_EDITORONLY_DATA
				CheckFNames(Expected.OldClassName, Actual.OldClassName);
				CheckFNames(Actual.OldClassName, NAME_None);
#endif
			}

			// Perform patching

			TArray<FAssetHeaderPatcherInner::FExportPatch> ExportPatches;
			int32 NewImportCount = 0;
			TArray<FAssetHeaderPatcherInner::FImportPatch> ImportPatches;
			Patcher.GetExportTablePatches(ExportPatches);
			CHECK(!ExportPatches.IsEmpty());
			FAssetHeaderPatcher::EResult Result = Patcher.GetImportTablePatches(ImportPatches, NewImportCount);
			CHECK(Result == FAssetHeaderPatcher::EResult::Success);
			CHECK(!ExportPatches.IsEmpty());
			Patcher.PatchExportAndImportTables(ExportPatches, ImportPatches, NewImportCount);
			Patcher.PatchNameTable();

			// Confirm the patched state is what is expected
			CHECK(Patcher.ImportTable.Num() == ImportTestCases.Num());
			for (int32 i = 0; i < ImportTestCases.Num(); ++i)
			{
				const FObjectImport& Expected = ImportTestCases[i].Dst;
				const FObjectImport& Actual = Patcher.ImportTable[i];

				CheckFNames(Expected.ObjectName, Actual.ObjectName);
				CHECK(Expected.OuterIndex == Actual.OuterIndex);
				if (ImportTestCases[i].bExistingImport)
				{
					CheckFNames(Expected.ClassName, Actual.ClassName);
					CheckFNames(Expected.ClassPackage, Actual.ClassPackage);
#if WITH_EDITORONLY_DATA
					CheckFNames(Expected.PackageName, Actual.PackageName);
					CheckFNames(Expected.OldClassName, Actual.OldClassName);
					CheckFNames(Actual.OldClassName, NAME_None);
#endif
				}
				else
				{
					// For new imports created by the patcher, we don't yet have a contract for what they
					// should report for Class of the import and external packagename, because it would have
					// to read that out of the target packages.
				}
			}
			
			CHECK(Patcher.ExportTable.Num() == ExportTestCases.Num());
			for (int32 i = 0; i < Patcher.ExportTable.Num(); ++i)
			{
				const FObjectExport& Expected = ExportTestCases[i].Dst;
				const FObjectExport& Actual = Patcher.ExportTable[i];

				CheckFNames(Expected.ObjectName, Actual.ObjectName);
				CHECK(Expected.OuterIndex == Actual.OuterIndex);
#if WITH_EDITORONLY_DATA
				CheckFNames(Expected.OldClassName, Actual.OldClassName);
				CheckFNames(Actual.OldClassName, NAME_None);
#endif
			}
		}
	}
}

#endif // WITH_TESTS
