// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRenameManager.h"
#include "AssetDefinitionRegistry.h"
#include "Containers/VersePath.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Class.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "SourceControlHelpers.h"
#include "FileHelpers.h"
#include "SDiscoveringAssetsDialog.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "ObjectTools.h"
#include "Interfaces/IMainFrameModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/RedirectCollector.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "Settings/EditorProjectSettings.h"
#include "AssetToolsLog.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/MapBuildDataRegistry.h"
#include "GameMapsSettings.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Internationalization/PackageLocalizationUtil.h"
#include "IAssetTools.h"
#include "ILocalizedAssetTools.h"
#include "SFileListReportDialog.h"

#define LOCTEXT_NAMESPACE "AssetRenameManager"

namespace AssetRenameManagerImpl
{
	// Same as CheckSubPath.IsEmpty() || SubPath == CheckSubPath || SubPath.StartsWith(CheckSubPath + TEXT("."))
	// but with early outs and without having to concatenate a string for comparison
	static bool IsSubPath(const FString& SubPath, const FString& CheckSubPath)
	{
		const int32 CheckSubPathLen = CheckSubPath.Len();
		if (CheckSubPathLen == 0)
		{
			return true;
		}

		const int32 SubPathLen = SubPath.Len();
		if (SubPathLen == CheckSubPathLen)
		{
			if (SubPathLen)
			{
				// Checking the last character first should skip most string compare since lots of paths might have the same beginning
				return (*SubPath)[SubPathLen - 1] == (*CheckSubPath)[SubPathLen - 1] && SubPath == CheckSubPath;
			}
			else
			{
				// Both strings are empty
				return true;
			}
		}
		else
		{
			//Checking for the . at the exact position first should eliminate most of the StartsWith comparison.
			return SubPathLen > CheckSubPathLen && (*SubPath)[CheckSubPathLen] == TEXT('.') && SubPath.StartsWith(CheckSubPath);
		}
	}
}

struct FAssetRenameDataWithReferencers : public FAssetRenameData
{
	TSet<FName> NotRenamedReferencingPackageNames;
	TMap<FName, FName> RenamedReferencingPackageNames;
	FText FailureReason;
	bool bCreateRedirector;
	bool bRenameFailed;

	FAssetRenameDataWithReferencers(const FAssetRenameData& InRenameData)
		: FAssetRenameData(InRenameData)
		, bCreateRedirector(false)
		, bRenameFailed(false)
	{
		if (Asset.IsValid() && !OldObjectPath.IsValid())
		{
			OldObjectPath = FSoftObjectPath(Asset.Get());
		}
		else if (OldObjectPath.IsValid() && !Asset.IsValid())
		{
			Asset = OldObjectPath.ResolveObject();
		}

		if (!NewName.IsEmpty() && !NewObjectPath.IsValid())
		{
			NewObjectPath = FSoftObjectPath(FString::Printf(TEXT("%s/%s.%s"), *NewPackagePath, *NewName, *NewName));
		}
		else if (NewObjectPath.IsValid() && NewName.IsEmpty())
		{
			NewName = NewObjectPath.GetAssetName();
			NewPackagePath = FPackageName::GetLongPackagePath(NewObjectPath.GetLongPackageName());
		}
	}
};

///////////////////////////
// FAssetRenameManager
///////////////////////////

bool FAssetRenameManager::RenameAssets(const TArray<FAssetRenameData>& AssetsAndNames) const
{
	const bool bAutoCheckout = true;
	const bool bWithDialog = false;

	// If the asset registry is still loading assets, we cant check for referencers and we cant open dialogs so we fail
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		UE_LOGF(LogAssetTools, Warning, "Unable To Rename While Discovering Assets");
		return false;
	}

	return RenameAssetsAndVariants(AssetsAndNames, bAutoCheckout, bWithDialog);
}

EAssetRenameResult FAssetRenameManager::RenameAssetsWithDialog(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout) const
{
	const bool bWithDialog = true;

	// If the asset registry is still loading assets, we cant check for referencers, so we must open the rename dialog
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		// Open a dialog asking the user to wait while assets are being discovered
		SDiscoveringAssetsDialog::OpenDiscoveringAssetsDialog(
			SDiscoveringAssetsDialog::FOnAssetsDiscovered::CreateSP(this, &FAssetRenameManager::RenameAssetsAndVariantsCallback, AssetsAndNames, bAutoCheckout, bWithDialog)
		);
		return EAssetRenameResult::Pending;
	}

	// No need to wait, check for variants and attempt to fix references now.
	return RenameAssetsAndVariants(AssetsAndNames, bAutoCheckout, bWithDialog)
		? EAssetRenameResult::Success
		: EAssetRenameResult::Failure;
}

void FAssetRenameManager::FindSoftReferencesToObject(FSoftObjectPath TargetObject, TArray<UObject*>& ReferencingObjects) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetRenameManager::FindSoftReferencesToObject);

	TArray<FAssetRenameDataWithReferencers> AssetsToRename;
	AssetsToRename.Emplace(FAssetRenameDataWithReferencers(FAssetRenameData(TargetObject, TargetObject, true)));

	// Fill out referencers from asset registry
	PopulateAssetReferencers(AssetsToRename);

	// Load all referencing objects and find for referencing objects
	TMap<FSoftObjectPath, TArray<UObject*>> ReferencingObjectsMap;

	GatherReferencingObjects(AssetsToRename, ReferencingObjectsMap);

	// Build an array out of the map results.
	for (const auto& It : ReferencingObjectsMap)
	{
		for (UObject* Obj : It.Value)
		{
			ReferencingObjects.AddUnique(Obj);
		}
	}
}

void FAssetRenameManager::FindSoftReferencesToObjects(const TArray<FSoftObjectPath>& TargetObjects, TMap<FSoftObjectPath, TArray<UObject*>>& ReferencingObjects) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetRenameManager::FindSoftReferencesToObjects);

	TArray<FAssetRenameDataWithReferencers> AssetsToRename;
	for (const FSoftObjectPath& TargetObject : TargetObjects)
	{
		AssetsToRename.Emplace(FAssetRenameDataWithReferencers(FAssetRenameData(TargetObject, TargetObject, true)));
	}

	// Fill out referencers from asset registry
	PopulateAssetReferencers(AssetsToRename);

	// Load all referencing objects and find for referencing objects
	GatherReferencingObjects(AssetsToRename, ReferencingObjects);
}

void FAssetRenameManager::RenameAssetsAndVariantsCallback(TArray<FAssetRenameData> InAssetsRenameData, bool bAutoCheckout, bool bWithDialog) const
{
	RenameAssetsAndVariants(InAssetsRenameData, bAutoCheckout, bWithDialog);
}

bool FAssetRenameManager::RenameAssetsAndVariants(const TArray<FAssetRenameData>& InAssetsRenameData, bool bAutoCheckout, bool bWithDialog) const
{
	FScopedSlowTask RenamingSlowTask(11.0f, LOCTEXT("RenamingSlowTask", "Renaming assets..."), bWithDialog);
	bool bShowCancelButton = false;
	bool bAllowInPIE = true;
	RenamingSlowTask.MakeDialog(bShowCancelButton, bAllowInPIE);
	RenamingSlowTask.EnterProgressFrame(2.0f, LOCTEXT("CheckingForLocalizedVariantsSlowTask", "Checking for localized variants... This might take a long time."));
	RenamingSlowTask.ForceRefresh();

	TArray<FAssetRenameData> AssetsAndVariants;

	TArray<FAssetRenameData> AssetsToCheckForVariants;
	if (RequiresCheckingForVariants(InAssetsRenameData, AssetsAndVariants, AssetsToCheckForVariants))
	{
		// Build a list of packages to check for variants (and build the map to remap the rename data later)
		TArray<FName> SourcePackagesToCheckForVariants;
		TMap<FName, const FAssetRenameData*> MapSourcePackageToRenameData;
		SourcePackagesToCheckForVariants.Reserve(AssetsToCheckForVariants.Num());
		MapSourcePackageToRenameData.Reserve(AssetsToCheckForVariants.Num());
		for (const FAssetRenameData& AssetToCheckForVariants : AssetsToCheckForVariants)
		{
			FString SourcePackageString;
			FPackageLocalizationUtil::ConvertToSource(AssetToCheckForVariants.Asset->GetOuter()->GetName(), SourcePackageString);
			FName SourcePackage(SourcePackageString);
			SourcePackagesToCheckForVariants.Add(SourcePackage);
			MapSourcePackageToRenameData.Add(SourcePackage, &AssetToCheckForVariants);
		}

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		TSharedPtr<ILocalizedAssetTools> LocalizedAssetTools = AssetTools.GetLocalizedAssetTools();

		TArray<FName> SourcePackagesWithoutVariants;
		TMap<FName, TArray<FName>> VariantsBySourcesOnDisk;
		TMap<FName, TArray<FName>> VariantsBySourcesOnlyInRevisionControl;
		bool bNeedToCheckInRevisionControl = USourceControlPreferences::RequiresRevisionControlToRenameLocalizableAssets();
		ELocalizedAssetsResult Result = LocalizedAssetTools->GetLocalizedVariants(SourcePackagesToCheckForVariants, VariantsBySourcesOnDisk, bNeedToCheckInRevisionControl, VariantsBySourcesOnlyInRevisionControl, &SourcePackagesWithoutVariants);
		bool bRevisionControlWasNeeded = (Result == ELocalizedAssetsResult::RevisionControlNotAvailable);

		// If all variants are accessible (not in Revision Control), we can now
		// rebuild all the renaming data from the localized variants information
		if (!bRevisionControlWasNeeded && VariantsBySourcesOnlyInRevisionControl.IsEmpty())
		{
			IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

			// Let's process each "VariantsBySourcesOnDisk" one-by-one and map it back to its rename data
			for (const TPair<FName, TArray<FName>>& VariantsBySourceOnDisk : VariantsBySourcesOnDisk)
			{
				// Find the rename data that match this source
				const FAssetRenameData& CurrentRenameData = *MapSourcePackageToRenameData[VariantsBySourceOnDisk.Key];

				// Get some rename data information (at this point, we don't know if it is a Source Asset or a Localized Variant rename data information)
				FSoftObjectPath OldAssetSoftObjectPath(CurrentRenameData.Asset.Get());
				FString OldAssetPackageName = OldAssetSoftObjectPath.GetLongPackageName();
				FString OldAssetPath = OldAssetSoftObjectPath.GetAssetPath().ToString();

				// Create and add the Source Asset Rename Data
				FString SourceAssetOldPath;
				FPackageLocalizationUtil::ConvertToSource(OldAssetPath, SourceAssetOldPath);
				UObject* SourceObjectPtr = AssetRegistry.GetAssetByObjectPath(SourceAssetOldPath).GetAsset();
				FString SourceNewPackagePath;
				FPackageLocalizationUtil::ConvertToSource(CurrentRenameData.NewPackagePath, SourceNewPackagePath);
				FAssetRenameData CurrentAssetRenameData(SourceObjectPtr,
					SourceNewPackagePath,
					CurrentRenameData.NewName,
					CurrentRenameData.bOnlyFixSoftReferences,
					CurrentRenameData.bAlsoRenameLocalizedVariants);
				AssetsAndVariants.Add(CurrentAssetRenameData);

				// Create and add all Variants Asset Rename Data
				for (const FName& Variant : VariantsBySourceOnDisk.Value)
				{
					FString Culture;
					FPackageLocalizationUtil::ExtractCultureFromLocalized(Variant.ToString(), Culture);

					FString OldLocalizedAssetPath;
					FString NewLocalizedPackagePath;
					FPackageLocalizationUtil::ConvertSourceToLocalized(SourceAssetOldPath, Culture, OldLocalizedAssetPath);
					FPackageLocalizationUtil::ConvertSourceToLocalized(SourceNewPackagePath, Culture, NewLocalizedPackagePath);

					UObject* LocalizedObjectPtr = AssetRegistry.GetAssetByObjectPath(OldLocalizedAssetPath).GetAsset();

					// Create the renaming data for the localized variant then add it to the list of files to be renamed
					CurrentAssetRenameData.Asset = LocalizedObjectPtr;
					CurrentAssetRenameData.NewPackagePath = NewLocalizedPackagePath;
					AssetsAndVariants.Add(CurrentAssetRenameData);
				}
			}

			// Finally, let's add back all the assets that don't have any variants as-is
			for (const FName& SourcePackageWithoutVariants : SourcePackagesWithoutVariants)
			{
				AssetsAndVariants.Add(*MapSourcePackageToRenameData[SourcePackageWithoutVariants]);
			}
		}

		// Process error and interrupt the renaming process if necessary
		if (bRevisionControlWasNeeded)
		{
			// Revision Control is not available. Renaming must fail.
			UE_LOGF(LogAssetTools, Warning, "%ls", *LocalizedAssetTools->GetRevisionControlIsNotAvailableWarningText().ToString());
			RenameInterrupted(InAssetsRenameData, LocalizedAssetTools->GetRevisionControlIsNotAvailableWarningText(), bWithDialog);
			if (bWithDialog)
			{
				LocalizedAssetTools->OpenRevisionControlRequiredDialog();
			}
			return false;
		}
		else if (!VariantsBySourcesOnlyInRevisionControl.IsEmpty())
		{
			TArray<FText> LocalizedVariantsPathsRequiredToSync;
			for (const TPair<FName, TArray<FName>>& VariantsBySourceOnlyInRevisionControl : VariantsBySourcesOnlyInRevisionControl)
			{
				for (const FName& VariantInRevisionControl : VariantsBySourceOnlyInRevisionControl.Value)
				{
					LocalizedVariantsPathsRequiredToSync.Add(FText::AsCultureInvariant(VariantInRevisionControl.ToString()));
				}
			}

			UE_LOGF(LogAssetTools, Warning, "A file that needs to be renamed was detected in Revision Control but not on your disk.");
			RenameInterrupted(InAssetsRenameData, LocalizedAssetTools->GetFilesNeedToBeOnDiskWarningText(), bWithDialog);
			if (bWithDialog)
			{
				LocalizedAssetTools->OpenFilesInRevisionControlRequiredDialog(LocalizedVariantsPathsRequiredToSync);
			}
			return false;
		}
		
		// Display newly added rename data if possible
		if (bWithDialog)
		{
			TArray<FText> NewAssetsAdded;
			for (const FAssetRenameData& AssetRenameData : AssetsAndVariants)
			{
				if (!InAssetsRenameData.ContainsByPredicate([&AssetRenameData](const FAssetRenameData& InAssetRenameData) { return AssetRenameData.Asset == InAssetRenameData.Asset; }))
				{
					const FString& AssetNameStr = AssetRenameData.Asset->GetOuter()->GetName();
					UE_LOGF(LogAssetTools, Display, "Also trying to rename: %ls", *AssetNameStr);
					NewAssetsAdded.Add(FText::AsCultureInvariant(AssetNameStr));
				}
			}
			if (!NewAssetsAdded.IsEmpty())
			{
				ELocalizedVariantsInclusion InclusionResult = LocalizedAssetTools->OpenIncludeLocalizedVariantsListDialog(NewAssetsAdded);
				switch (InclusionResult)
				{
				case ELocalizedVariantsInclusion::Exclude:
					AssetsAndVariants = InAssetsRenameData;
					break;
				case ELocalizedVariantsInclusion::Cancel:
					AssetsAndVariants = InAssetsRenameData;
					return false;
				case ELocalizedVariantsInclusion::Include:
				default:
					// The inclusion is already done at this point. Only the exclude/cancel options need to do some revert work.
					break;
				}
			}
		}
	}

	// We now have a full list of Assets to rename and their variants if applicable. Let's continue the renaming process.
	return FixReferencesAndRename(AssetsAndVariants, bAutoCheckout, bWithDialog, RenamingSlowTask);
}

bool FAssetRenameManager::FixReferencesAndRename(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout, bool bWithDialog, FScopedSlowTask& RenamingSlowTask) const
{
	RenamingSlowTask.EnterProgressFrame(0.9f, LOCTEXT("FixingReferencesSlowTask", "Finding references..."));

	bool bSoftReferencesOnly = true;
	// Prep a list of assets to rename with an extra boolean to determine if they should leave a redirector or not
	TArray<FAssetRenameDataWithReferencers> AssetsToRename;
	AssetsToRename.Reset(AssetsAndNames.Num());
	// Avoid duplicates when adding MapBuildData to list
	TSet<UObject*> AssetsToRenameLookup;
	for (const FAssetRenameData& AssetRenameData : AssetsAndNames)
	{
		AssetsToRenameLookup.Add(AssetRenameData.Asset.Get());
	}
	for (const FAssetRenameData& AssetRenameData : AssetsAndNames)
	{
		if (!AssetRenameData.OldObjectPath.IsValid() && !AssetRenameData.NewObjectPath.IsValid())
		{
			// Rename MapBuildData when renaming world
			UWorld* World = Cast<UWorld>(AssetRenameData.Asset.Get());
			if (World && World->PersistentLevel && World->PersistentLevel->MapBuildData && !AssetsToRenameLookup.Contains(World->PersistentLevel->MapBuildData))
			{
				// Leave MapBuildData inside the map's package
				if (World->PersistentLevel->MapBuildData->GetOutermost() != World->GetOutermost())
				{
					FString NewMapBuildDataName = AssetRenameData.NewName + TEXT("_BuiltData");
					// Perform rename of MapBuildData before world otherwise original files left behind
					AssetsToRename.EmplaceAt(0, FAssetRenameDataWithReferencers(FAssetRenameData(World->PersistentLevel->MapBuildData, AssetRenameData.NewPackagePath, NewMapBuildDataName)));
					AssetsToRename[0].bOnlyFixSoftReferences = AssetRenameData.bOnlyFixSoftReferences;
					AssetsToRenameLookup.Add(World->PersistentLevel->MapBuildData);
				}
			}
		}

		// Perform rename of MapBuildData before world otherwise original files left behind
		UMapBuildDataRegistry* MapBuildData = Cast<UMapBuildDataRegistry>(AssetRenameData.Asset.Get());
		if (MapBuildData)
		{
			AssetsToRename.EmplaceAt(0, FAssetRenameDataWithReferencers(AssetRenameData));
		}
		else
		{
			AssetsToRename.Emplace(FAssetRenameDataWithReferencers(AssetRenameData));
		}

		if (!AssetRenameData.bOnlyFixSoftReferences)
		{
			bSoftReferencesOnly = false;
		}
	}

	// Warn the user if they are about to rename an asset that is referenced by a CDO
	TArray<FAssetRenameDataWithReferencers*> CDOHardReferencedAssets, CDOSoftReferenceRenames;
	FindCDOReferences(AssetsToRename, CDOHardReferencedAssets, CDOSoftReferenceRenames, true);

	// Warn the user if there were any references
	if (CDOHardReferencedAssets.Num() || CDOSoftReferenceRenames.Num())
	{
		FString AssetNames;
		for (auto HardAssetIt = CDOHardReferencedAssets.CreateConstIterator(); HardAssetIt; ++HardAssetIt)
		{
			UObject* Asset = (*HardAssetIt)->Asset.Get();
			if (Asset)
			{
				AssetNames += FString("\n") + Asset->GetName();
			}
		}

		for (auto SoftRefIt = CDOSoftReferenceRenames.CreateConstIterator(); SoftRefIt; ++SoftRefIt)
		{
			UObject* Asset = (*SoftRefIt)->Asset.Get();
			if (Asset)
			{
				AssetNames += FString("\n") + Asset->GetName();
			}
		}

		const FText MessageText = FText::Format(LOCTEXT("RenameCDOReferences", "Source code, config INI, and text files may need Find/Replace for:\n\n{0}\n\nOtherwise assets can be missing from cooked builds. Continue with rename?"), FText::FromString(AssetNames));
		if (FMessageDialog::Open(EAppMsgType::OkCancel, EAppReturnType::Cancel, MessageText) == EAppReturnType::Cancel)
		{
			return false;
		}
	}

	// Fill out the referencers for the assets we are renaming
	PopulateAssetReferencers(AssetsToRename);

	// Update the source control state for the packages containing the assets we are renaming if source control is enabled. If source control is enabled and this fails we can not continue.
	if (bSoftReferencesOnly || UpdatePackageStatus(AssetsToRename))
	{
		// Detect whether the assets are being referenced by a collection. Assets within a collection must leave a redirector to avoid the collection losing its references.
		DetectReferencingCollections(AssetsToRename);

		// Load all referencing packages and mark any assets that must have redirectors.
		TArray<UPackage*> ReferencingPackagesToSave;
		TArray<UObject*> SoftReferencingObjects;
		LoadReferencingPackages(AssetsToRename, bSoftReferencesOnly, true, ReferencingPackagesToSave, SoftReferencingObjects);

		// Prompt to check out source package and all referencing packages, leave redirectors for assets referenced by packages that are not checked out and remove those packages from the save list.
		// If revision control is disabled, this still checks if the packages are not read-only.
		const FText& CheckOutSlowTask = ISourceControlModule::Get().IsEnabled()
			? LOCTEXT("CheckOutPackagesSlowTask", "Check out packages...")
			: LOCTEXT("PreparePackagesSlowTask", "Preparing packages...");
		RenamingSlowTask.EnterProgressFrame(0.1f, CheckOutSlowTask);
		const bool bUserAcceptedCheckout = CheckOutPackages(AssetsToRename, ReferencingPackagesToSave, bAutoCheckout);

		if (bUserAcceptedCheckout || bSoftReferencesOnly)
		{
			// If any referencing packages are left read-only, the checkout failed or SCC was not enabled. Trim them from the save list and leave redirectors.
			DetectReadOnlyPackages(AssetsToRename, ReferencingPackagesToSave);


			// Make public any private asset that will be referenced from another plugin after the rename.
			// If the asset cannot be made public or if moving it requires a referenced asset that's not being modified to become public,
			// its rename will fail and other assets that have dependencies between them will also fail to be renamed.
			SetupPublicAssets(AssetsToRename);

			if (bSoftReferencesOnly)
			{
				if (ReferencingPackagesToSave.Num() > 0)
				{
					// Only do the rename if there are actually packages with references
					PerformAssetRename(AssetsToRename, RenamingSlowTask);

					for (const FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
					{
						// Add source and destination packages so those get saved at the same time
						UPackage* OldPackage = FindPackage(nullptr, *RenameData.OldObjectPath.GetLongPackageName());
						UPackage* NewPackage = FindPackage(nullptr, *RenameData.NewObjectPath.GetLongPackageName());

						ReferencingPackagesToSave.AddUnique(OldPackage);
						ReferencingPackagesToSave.AddUnique(NewPackage);
					}

					FString AssetNames;
					for (const FString& AssetName : IAssetTools::Get().GetUserFacingLongPackageNames(ReferencingPackagesToSave))
					{
						AssetNames += FString("\n") + AssetName;
					}

					// Warn user before saving referencing packages
					bool bAgreedToSaveReferencingPackages = bAutoCheckout;
					if (!bAgreedToSaveReferencingPackages)
					{
						const FText MessageText = FText::Format(LOCTEXT("SoftReferenceFixedUp", "The following packages were fixed up because they have soft references to a renamed object: \n{0}\n\nDo you want to save them now?\nIf you quit without saving references will be broken!"), FText::FromString(AssetNames));
						bAgreedToSaveReferencingPackages = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, MessageText) == EAppReturnType::Yes;
					}

					if (bAgreedToSaveReferencingPackages)
					{
						SaveReferencingPackages(ReferencingPackagesToSave);
					}
				}
			}
			else
			{
				// Perform the rename, leaving redirectors only for assets which need them
				// Also save all packages that were referencing any of the assets that were moved without redirectors
				PerformAssetRename(AssetsToRename, ReferencingPackagesToSave, RenamingSlowTask);

				// Issue post rename event
				AssetPostRenameEvent.Broadcast(AssetsAndNames);
			}
		}
	}

	// Finally, report any failures that happened during the rename
	return ReportFailures(AssetsToRename, bWithDialog) == 0;
}


struct FSoftObjectPathRenameSerializer : public FArchiveUObject
{
	void StartSerializingObject(UObject* InCurrentObject)
	{
		CurrentObject = InCurrentObject;
		bFoundReference = false;
	}
	bool HasFoundReference() const
	{
		return bFoundReference;
	}

	FSoftObjectPathRenameSerializer(const TMap<FSoftObjectPath, FSoftObjectPath>& InRedirectorMap,
		bool bInCheckOnly,
		TMap<FSoftObjectPath, TSet<FWeakObjectPtr>>* InCachedObjectPaths,
		const FName InPackageName = NAME_None)
		: RedirectorMap(InRedirectorMap)
		, CachedObjectPaths(InCachedObjectPaths)
		, CurrentObject(nullptr)
		, PackageName(InPackageName)
		, bSearchOnly(bInCheckOnly)
		, bFoundReference(false)
	{
		if (InCachedObjectPaths)
		{
			DirtyDelegateHandle = UPackage::PackageMarkedDirtyEvent.AddRaw(this, &FSoftObjectPathRenameSerializer::OnMarkPackageDirty);
		}

		this->ArIsObjectReferenceCollector = true;
		this->ArIsModifyingWeakAndStrongReferences = true;

		// Mark it as saving to correctly process all references
		this->SetIsSaving(true);
	}

	virtual ~FSoftObjectPathRenameSerializer()
	{
		UPackage::PackageMarkedDirtyEvent.Remove(DirtyDelegateHandle);
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		if (InProperty->HasAnyPropertyFlags(CPF_Transient | CPF_Deprecated | CPF_IsPlainOldData))
		{
			return true;
		}

		FFieldClass* PropertyClass = InProperty->GetClass();
		if (PropertyClass->GetCastFlags() & (CASTCLASS_FBoolProperty | CASTCLASS_FNameProperty | CASTCLASS_FStrProperty | CASTCLASS_FTextProperty | CASTCLASS_FMulticastDelegateProperty))
		{
			return true;
		}

		if (PropertyClass->GetCastFlags() & (CASTCLASS_FArrayProperty | CASTCLASS_FMapProperty | CASTCLASS_FSetProperty))
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
			{
				return ShouldSkipProperty(ArrayProperty->Inner);
			}
			else if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
			{
				return ShouldSkipProperty(MapProperty->KeyProp) && ShouldSkipProperty(MapProperty->ValueProp);
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
			{
				return ShouldSkipProperty(SetProperty->ElementProp);
			}
		}

		return false;
	}

	FArchive& operator<<(FSoftObjectPath& Value)
	{
		using namespace AssetRenameManagerImpl;

		// Ignore untracked references if just doing a search only. We still want to fix them up if they happen to be there
		if (bSearchOnly)
		{
			FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();
			FName ReferencingPackageName, ReferencingPropertyName;
			ESoftObjectPathCollectType CollectType = ESoftObjectPathCollectType::AlwaysCollect;
			ESoftObjectPathSerializeType SerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;

			ThreadContext.GetSerializationOptions(ReferencingPackageName, ReferencingPropertyName, CollectType, SerializeType, this);

			if (!UE::SoftObjectPath::IsCollectable(CollectType))
			{
				return *this;
			}
		}

		if (CachedObjectPaths)
		{
			TSet<FWeakObjectPtr>* ObjectSet = &CachedObjectPaths->FindOrAdd(Value);
			ObjectSet->Add(CurrentObject);
		}

		const FString& SubPath = Value.GetSubPathString();
		for (const TPair<FSoftObjectPath, FSoftObjectPath>& Pair : RedirectorMap)
		{
			if (Pair.Key.GetAssetPath() == Value.GetAssetPath())
			{
				// Same asset, fix sub path. Asset will be fixed by normal serializePath call below
				const FString& CheckSubPath = Pair.Key.GetSubPathString();

				if (IsSubPath(SubPath, CheckSubPath))
				{
					bFoundReference = true;

					if (!bSearchOnly)
					{
						if (CurrentObject)
						{
							check(!CachedObjectPaths); // Modify can invalidate the object paths map, not allowed to be modifying and using the cache at the same time
							CurrentObject->Modify(true);
						}

						FString NewSubPath(SubPath);
						NewSubPath.ReplaceInline(*CheckSubPath, *Pair.Value.GetSubPathString());
						Value = FSoftObjectPath(Pair.Value.GetAssetPath(), NewSubPath);
					}
					break;
				}
			}
		}

		return *this;
	}

	void OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty)
	{
		UPackage::PackageMarkedDirtyEvent.Remove(DirtyDelegateHandle);

		if (CachedObjectPaths && Pkg && Pkg->GetFName() == PackageName)
		{
			UE_LOGF(LogAssetTools, VeryVerbose, "Performance: Package unexpectedly modified during serialization by FSoftObjectPathRenameSerializer: %ls", *Pkg->GetFullName());
		}
	}

private:
	const TMap<FSoftObjectPath, FSoftObjectPath>& RedirectorMap;
	TMap<FSoftObjectPath, TSet<FWeakObjectPtr>>* CachedObjectPaths;
	FDelegateHandle DirtyDelegateHandle;
	UObject* CurrentObject;
	FName PackageName;
	bool bSearchOnly;
	bool bFoundReference;

};

void FAssetRenameManager::FindCDOReferences(const TArrayView<FAssetRenameDataWithReferencers>& AssetsToRename, TArray<FAssetRenameDataWithReferencers*>& OutHardReferences, TArray<FAssetRenameDataWithReferencers*>& OutSoftReferences, bool bSetRedirectorFlags) const
{
	// Checking reference candidates off as we find them to reduce workload
	TArray<FAssetRenameDataWithReferencers*> RemainingHardRefAssetChecklist;
	TArray<FAssetRenameDataWithReferencers*> RemainingSoftRefAssetChecklist;

	for (FAssetRenameDataWithReferencers& AssetToRename : AssetsToRename)
	{
		if (AssetToRename.Asset.IsValid())
		{
			RemainingHardRefAssetChecklist.Push(&AssetToRename);
			RemainingSoftRefAssetChecklist.Push(&AssetToRename);
		}
	}

	// Run over all CDOs and check for any references to the assets
	for (TObjectIterator<UClass> ClassDefaultObjectIt; ClassDefaultObjectIt; ++ClassDefaultObjectIt)
	{
		UClass* Cls = (*ClassDefaultObjectIt);
		UObject* CDO = Cls->GetDefaultObject(false);

		if (!CDO || !CDO->HasAllFlags(RF_ClassDefaultObject) || !IsValidChecked(CDO) || Cls->ClassGeneratedBy != nullptr)
		{
			continue;
		}

		// Ignore deprecated and temporary trash classes.
		if (Cls->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists) || FKismetEditorUtilities::IsClassABlueprintSkeleton(Cls))
		{
			continue;
		}

		// Search this CDO for hard references
		for (TFieldIterator<FObjectProperty> PropertyIt(Cls); PropertyIt && RemainingHardRefAssetChecklist.Num(); ++PropertyIt)
		{
			const UObject* Object = PropertyIt->GetPropertyValue(PropertyIt->ContainerPtrToValuePtr<UObject>(CDO));

			for (FAssetRenameDataWithReferencers* AssetToRename : RemainingHardRefAssetChecklist)
			{
				if (Object == AssetToRename->Asset.Get())
				{
					OutHardReferences.Push(AssetToRename);
					RemainingHardRefAssetChecklist.Remove(AssetToRename);
					break;
				}
			}
		}

		//Search this CDO for soft references
		TMap<FSoftObjectPath, FSoftObjectPath> DummyEmptyRedirectorMap;
		TMap<FSoftObjectPath, TSet<FWeakObjectPtr>> SoftReferenceMap;
		FSoftObjectPathRenameSerializer SoftRefCheckSerializer(DummyEmptyRedirectorMap, true, &SoftReferenceMap);

		// Gather all soft references
		SoftRefCheckSerializer.StartSerializingObject(CDO);
		CDO->Serialize(SoftRefCheckSerializer);

		// Check all soft references in the CDO for matching items that are to be renamed, with special handling for UBlueprint assets
		for (auto Iter = SoftReferenceMap.CreateIterator(); Iter && RemainingSoftRefAssetChecklist.Num(); ++Iter)
		{
			FSoftObjectPath SoftRefObjPath = Iter.Key();

			if (SoftRefObjPath.IsValid())
			{
				TWeakObjectPtr<UObject> Object = SoftRefObjPath.ResolveObject();
				TWeakObjectPtr<UBlueprint> ObjectAsBP = UBlueprint::GetBlueprintFromClass(Cast<UBlueprintGeneratedClass>(Object));

				// Resolve to the redirected asset path if necessary
				FSoftObjectPath FinalSoftObjPath = SoftRefObjPath.GetWithoutSubPath();

				if (!Object.IsValid() && SoftRefObjPath.IsValid())
				{
					FSoftObjectPath RedirObjectPath = GRedirectCollector.GetAssetPathRedirection(SoftRefObjPath.GetWithoutSubPath());

					if (!RedirObjectPath.IsNull() && RedirObjectPath != FinalSoftObjPath)
					{
						FinalSoftObjPath = RedirObjectPath;
					}
				}

				// Check for any matching rename requests
				for (FAssetRenameDataWithReferencers* AssetToRename : RemainingSoftRefAssetChecklist)
				{
					// Look for loaded references, indirect blueprint refs to their generated class counterparts, or path name matching
					if ((Object == AssetToRename->Asset) ||
						(ObjectAsBP != nullptr && ObjectAsBP == AssetToRename->Asset) ||
						(!FinalSoftObjPath.IsNull() && FinalSoftObjPath == AssetToRename->OldObjectPath.GetWithoutSubPath()))
					{
						AssetToRename->bCreateRedirector |= bSetRedirectorFlags;

						OutSoftReferences.Push(AssetToRename);
						RemainingSoftRefAssetChecklist.Remove(AssetToRename);

						break;
					}
				}
			}
		}
	}
}

void FAssetRenameManager::PopulateAssetReferencers(TArray<FAssetRenameDataWithReferencers>& AssetsToPopulate) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TMap<FName, FName> PackageNamesToRename;

	// Get the names of all the packages containing the assets we are renaming so they arent added to the referencing packages list
	for (FAssetRenameDataWithReferencers& AssetToRename : AssetsToPopulate)
	{
		// If we're only fixing soft references we want to check for references inside the original package as we don't save the original package automatically
		if (!AssetToRename.bOnlyFixSoftReferences)
		{
			PackageNamesToRename.Add(AssetToRename.OldObjectPath.GetLongPackageFName(), AssetToRename.NewObjectPath.GetLongPackageFName());
		}
	}

	TMap<FName, TArray<FName>> SoftReferencers;
	TMap<FName, TArray<FName>> PackageReferencers;

	TArray<UPackage*> ExtraPackagesToCheckForSoftReferences;
	FEditorFileUtils::GetDirtyWorldPackages(ExtraPackagesToCheckForSoftReferences);
	FEditorFileUtils::GetDirtyContentPackages(ExtraPackagesToCheckForSoftReferences);

	// Gather all referencing packages for all assets that are being renamed
	for (FAssetRenameDataWithReferencers& AssetToRename : AssetsToPopulate)
	{
		AssetToRename.NotRenamedReferencingPackageNames.Empty();
		AssetToRename.RenamedReferencingPackageNames.Empty();

		FName OldPackageName = AssetToRename.OldObjectPath.GetLongPackageFName();

		TMap<FName, TArray<FName>>& ReferencersMap = AssetToRename.bOnlyFixSoftReferences ? SoftReferencers : PackageReferencers;
		if (!ReferencersMap.Contains(OldPackageName))
		{
			TArray<FName>& Referencers = ReferencersMap.Add(OldPackageName);
			AssetRegistryModule.Get().GetReferencers(OldPackageName, Referencers, UE::AssetRegistry::EDependencyCategory::Package, AssetToRename.bOnlyFixSoftReferences ? UE::AssetRegistry::EDependencyQuery::Soft : UE::AssetRegistry::EDependencyQuery::NoRequirements);
		}

		for (const FName& ReferencingPackageName : ReferencersMap.FindChecked(OldPackageName))
		{
			if (FName* NewPackageName = PackageNamesToRename.Find(ReferencingPackageName))
			{
				AssetToRename.RenamedReferencingPackageNames.Add(ReferencingPackageName, *NewPackageName);
			}
			else
			{
				AssetToRename.NotRenamedReferencingPackageNames.Add(ReferencingPackageName);
			}
		}

		if (AssetToRename.bOnlyFixSoftReferences)
		{
			AssetToRename.NotRenamedReferencingPackageNames.Add(OldPackageName);
			AssetToRename.NotRenamedReferencingPackageNames.Add(AssetToRename.NewObjectPath.GetLongPackageFName());

			// Add dirty packages and the package that owns the reference. They will get filtered out in LoadReferencingPackages if they aren't valid
			for (UPackage* Package : ExtraPackagesToCheckForSoftReferences)
			{
				AssetToRename.NotRenamedReferencingPackageNames.Add(Package->GetFName());
			}
		}
	}
}

bool FAssetRenameManager::UpdatePackageStatus(TArray<FAssetRenameDataWithReferencers>& AssetsToRename) const
{
	bool bSucceeded = true;

	if (ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		// Update the source control server availability if necessary to make sure we can do the rename operation
		if (!SourceControlProvider.IsAvailable())
		{
			SourceControlProvider.Login();
		}

		if (SourceControlProvider.IsAvailable())
		{
			// Gather asset package names to update SCC states in a single SCC request
			TArray<UPackage*> PackagesToUpdate;
			PackagesToUpdate.Reserve(AssetsToRename.Num());

			for (const FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
			{
				if (UObject* Asset = RenameData.Asset.Get())
				{
					PackagesToUpdate.AddUnique(Asset->GetOutermost());
				}
			}

			bSucceeded = SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), PackagesToUpdate) == ECommandResult::Succeeded;
		}
		else
		{
			bSucceeded = false;
		}
	}

	if (!bSucceeded)
	{
		// Mark the renames as a failure to report it later
		for (FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
		{
			RenameData.bRenameFailed = true;
			RenameData.FailureReason = LOCTEXT("RenameFailedUnavailable", "Revision control is unresponsive.");
		}
	}

	return bSucceeded;
}

void FAssetRenameManager::LoadReferencingPackages(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, bool bLoadAllPackages, bool bCheckStatus, TArray<UPackage*>& OutReferencingPackagesToSave, TArray<UObject*>& OutSoftReferencingObjects) const
{
	const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
	bool bLoadPackagesForSoftReferences = EditorProjectSettings->bValidateUnloadedSoftActorReferences;
	bool bStartedSlowTask = false;
	const FText ReferenceUpdateSlowTask = LOCTEXT("ReferenceUpdateSlowTask", "Updating Asset References");

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	if (bCheckStatus)
	{
		// Bulk request a state update for all packages, which is faster than asking for each one.
		TArray<UPackage*> AllPackagesToUpdateSCCStatus;
		for (const FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
		{
			if (UObject* Asset = RenameData.Asset.Get())
			{
				AllPackagesToUpdateSCCStatus.Add(Asset->GetOutermost());
			}
		}

		if (AllPackagesToUpdateSCCStatus.Num() > 0)
		{
			TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe>> States;
			SourceControlProvider.GetState(AllPackagesToUpdateSCCStatus, States, EStateCacheUsage::ForceUpdate);
		}
	}

	for (int32 AssetIdx = 0; AssetIdx < AssetsToRename.Num(); ++AssetIdx)
	{
		if (bStartedSlowTask)
		{
			GWarn->StatusUpdate(AssetIdx, AssetsToRename.Num(), ReferenceUpdateSlowTask);
		}

		FAssetRenameDataWithReferencers& RenameData = AssetsToRename[AssetIdx];

		TSet<FName> ReferencingExternalPackageNames;

		UObject* Asset = RenameData.Asset.Get();
		if (Asset)
		{
			// External packages must always be resaved
			for (UPackage* ExternalPackage : Asset->GetPackage()->GetExternalPackages())
			{
				FName ExternalPackageName = ExternalPackage->GetFName();
				ReferencingExternalPackageNames.Add(ExternalPackageName);
				OutReferencingPackagesToSave.Add(ExternalPackage);
			}

			// Make sure this asset is local. Only local assets should be renamed without a redirector
			if (bCheckStatus)
			{
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Asset->GetOutermost(), EStateCacheUsage::Use);
				const bool bLocalFile = !SourceControlState.IsValid() || SourceControlState->IsLocal();
				if (!bLocalFile)
				{
					if (SourceControlState->IsSourceControlled())
					{
						// If this asset is locked or not current, mark it failed to prevent it from being renamed
						if (SourceControlState->IsCheckedOutOther())
						{
							RenameData.bRenameFailed = true;
							RenameData.FailureReason = LOCTEXT("RenameFailedCheckedOutByOther", "Checked out by another user.");
						}
						else if (!SourceControlState->IsCurrent())
						{
							RenameData.bRenameFailed = true;
							RenameData.FailureReason = LOCTEXT("RenameFailedNotCurrent", "Out of date.");
						}
					}

					// This asset is not local. It is not safe to rename it without leaving a redirector
					RenameData.bCreateRedirector = true;
					if (!bLoadAllPackages)
					{
						continue;
					}
				}
			}
		}
		else
		{
			// The asset for this rename must have been GCed or is otherwise invalid. Skip it unless this is a soft reference only fix
			if (!bLoadAllPackages)
			{
				continue;
			}
		}

		TMap<FSoftObjectPath, FSoftObjectPath> ModifiedPaths;
		ModifiedPaths.Add(RenameData.OldObjectPath, RenameData.NewObjectPath);

		TArray<UPackage*> PackagesToSaveForThisAsset;
		bool bAllPackagesLoadedForThisAsset = true;

		for (auto It = RenameData.NotRenamedReferencingPackageNames.CreateIterator(); It; ++It)
		{
			FName PackageName = *It;

			// Ignore external packages of this asset, those are already added to the list of packages to save
			if (ReferencingExternalPackageNames.Contains(PackageName))
			{
				continue;
			}

			// Check if the package is a map before loading it!
			if (!bLoadAllPackages && FEditorFileUtils::IsMapPackageAsset(PackageName.ToString()))
			{
				// This reference was a map package, don't load it and leave a redirector for this asset
				// For subobjects we want to load maps packages and treat them normally
				RenameData.bCreateRedirector = true;
				bAllPackagesLoadedForThisAsset = false;
				break;
			}
			UPackage* Package = FindPackage(nullptr, *PackageName.ToString());

			// Don't load package if this is a soft reference fix and the project settings say not to
			if (!Package && (!RenameData.bOnlyFixSoftReferences || bLoadPackagesForSoftReferences))
			{
				if (!bStartedSlowTask)
				{
					bStartedSlowTask = true;
					GWarn->BeginSlowTask(ReferenceUpdateSlowTask, true);
				}
				Package = LoadPackage(nullptr, *PackageName.ToString(), LOAD_None);
			}

			if (Package)
			{
				bool bFoundSoftReference = CheckPackageForSoftObjectReferences(Package, ModifiedPaths, OutSoftReferencingObjects);

				// Only add to list if we're doing a hard reference fixup or we found a soft reference
				bool bAdd = !RenameData.bOnlyFixSoftReferences || bFoundSoftReference;

				if (bAdd)
				{
					PackagesToSaveForThisAsset.Add(Package);
				}
				else
				{
					// This package does not actually reference the asset, so remove it
					It.RemoveCurrent();
				}
			}
			else
			{
				RenameData.bCreateRedirector = true;
				if (!bLoadAllPackages)
				{
					bAllPackagesLoadedForThisAsset = false;
					break;
				}
			}
		}

		if (bAllPackagesLoadedForThisAsset)
		{
			for (UPackage* Package : PackagesToSaveForThisAsset)
			{
				OutReferencingPackagesToSave.AddUnique(Package);
			}
		}
	}

	if (bStartedSlowTask)
	{
		GWarn->EndSlowTask();
	}
}

void FAssetRenameManager::GatherReferencingObjects(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, TMap<FSoftObjectPath, TArray<UObject*>>& OutSoftReferencingObjects) const
{
	const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
	bool bLoadPackagesForSoftReferences = EditorProjectSettings->bValidateUnloadedSoftActorReferences;

	TMap<UPackage*, TMap<FSoftObjectPath, FSoftObjectPath>> ReferencingPackages;

	for (int32 AssetIdx = 0; AssetIdx < AssetsToRename.Num(); ++AssetIdx)
	{
		FAssetRenameDataWithReferencers& RenameData = AssetsToRename[AssetIdx];

		UObject* Asset = RenameData.Asset.Get();
		if (!Asset)
		{
			// The asset for this rename must have been GCed or is otherwise invalid. Skip it unless this is a soft reference only fix
			continue;
		}

		for (FName PackageName : RenameData.NotRenamedReferencingPackageNames)
		{
			UPackage* Package = FindPackage(nullptr, *PackageName.ToString());

			// Don't load package if this is a soft reference fix and the project settings say not to
			if (!Package && (!RenameData.bOnlyFixSoftReferences || bLoadPackagesForSoftReferences))
			{
				Package = LoadPackage(nullptr, *PackageName.ToString(), LOAD_None);
			}

			if (Package)
			{
				ReferencingPackages.FindOrAdd(Package).Add(RenameData.OldObjectPath, RenameData.NewObjectPath);
			}
		}
	}

	TArray<UPackage*> PackagesToSaveForThisAsset;
	bool bAllPackagesLoadedForThisAsset = true;
	for (const auto& ReferencingPackage : ReferencingPackages)
	{
		CheckPackageForSoftObjectReferences(ReferencingPackage.Key, ReferencingPackage.Value, OutSoftReferencingObjects);
	}
}

bool FAssetRenameManager::CheckOutPackages(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, TArray<UPackage*>& InOutReferencingPackagesToSave, bool bAutoCheckout) const
{
	bool bUserAcceptedCheckout = true;

	// Build list of packages to check out: the source package and any referencing packages (in the case that we do not create a redirector)
	TArray<UPackage*> PackagesToCheckOut;
	PackagesToCheckOut.Reset(AssetsToRename.Num() + InOutReferencingPackagesToSave.Num());

	for (const FAssetRenameDataWithReferencers& AssetToRename : AssetsToRename)
	{
		if (!AssetToRename.bRenameFailed && AssetToRename.Asset.IsValid())
		{
			PackagesToCheckOut.Add(AssetToRename.Asset->GetOutermost());
		}
	}

	for (UPackage* ReferencingPackage : InOutReferencingPackagesToSave)
	{
		PackagesToCheckOut.Add(ReferencingPackage);
	}

	// Check out the packages
	if (PackagesToCheckOut.Num() > 0)
	{
		TArray<UPackage*> PackagesCheckedOutOrMadeWritable;
		TArray<UPackage*> PackagesNotNeedingCheckout;
		bUserAcceptedCheckout = bAutoCheckout ? AutoCheckOut(PackagesToCheckOut) : FEditorFileUtils::PromptToCheckoutPackages(false, PackagesToCheckOut, &PackagesCheckedOutOrMadeWritable, &PackagesNotNeedingCheckout);
		if (bUserAcceptedCheckout)
		{
			// Make a list of any packages in the list which weren't checked out for some reason
			TArray<UPackage*> PackagesThatCouldNotBeCheckedOut = PackagesToCheckOut;

			for (UPackage* Package : PackagesCheckedOutOrMadeWritable)
			{
				PackagesThatCouldNotBeCheckedOut.RemoveSwap(Package);
			}

			for (UPackage* Package : PackagesNotNeedingCheckout)
			{
				PackagesThatCouldNotBeCheckedOut.RemoveSwap(Package);
			}

			// If there's anything which couldn't be checked out, abort the operation.
			if (PackagesThatCouldNotBeCheckedOut.Num() > 0)
			{
				bUserAcceptedCheckout = false;
			}
		}

		// If the checkout was declined (or failed), then fail the entire rename request
		if (!bUserAcceptedCheckout)
		{
			for (FAssetRenameDataWithReferencers& AssetToRename : AssetsToRename)
			{
				if (!AssetToRename.bRenameFailed)
				{
					AssetToRename.bRenameFailed = true;
					AssetToRename.FailureReason = LOCTEXT("RenameFailedNotCheckedOutOrWritable", "Not checked-out or writable.");
				}
			}
		}
	}

	return bUserAcceptedCheckout;
}

bool FAssetRenameManager::AutoCheckOut(TArray<UPackage*>& PackagesToCheckOut) const
{
	bool bSomethingFailed = false;
	if (PackagesToCheckOut.Num() > 0)
	{
		if (ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			ECommandResult::Type StatusResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), PackagesToCheckOut);

			if (StatusResult != ECommandResult::Succeeded)
			{
				bSomethingFailed = true;
			}
			else
			{
				for (int32 Index = PackagesToCheckOut.Num() - 1; Index >= 0; --Index)
				{
					UPackage* Package = PackagesToCheckOut[Index];
					FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);
					if (SourceControlState->IsCheckedOutOther())
					{
						UE_LOGF(LogAssetTools, Warning, "FAssetRenameManager::AutoCheckOut: package %ls is already checked out by someone, will not check out", *SourceControlState->GetFilename());
						bSomethingFailed = true;
					}
					else if (!SourceControlState->IsCurrent())
					{
						UE_LOGF(LogAssetTools, Warning, "FAssetRenameManager::AutoCheckOut: package %ls is not at head, will not check out", *SourceControlState->GetFilename());
						bSomethingFailed = true;
					}
					else if (!SourceControlState->IsSourceControlled() || SourceControlState->CanEdit())
					{
						PackagesToCheckOut.RemoveAtSwap(Index);
					}
				}

				if (!bSomethingFailed && PackagesToCheckOut.Num() > 0)
				{
					bSomethingFailed = (SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), PackagesToCheckOut) != ECommandResult::Succeeded);
					if (!bSomethingFailed)
					{
						UE_LOGF(LogAssetTools, Warning, "FAssetRenameManager::AutoCheckOut: was not not able to auto checkout.");
						PackagesToCheckOut.Empty();
					}
				}
			}
		}
		else
		{
			for (int32 Index = PackagesToCheckOut.Num() - 1; Index >= 0; --Index)
			{
				UPackage* Package = PackagesToCheckOut[Index];
				const FString PackageFilename = USourceControlHelpers::PackageFilename(Package);

				if (IFileManager::Get().FileExists(*PackageFilename) && IFileManager::Get().IsReadOnly(*PackageFilename))
				{
					UE_LOGF(LogAssetTools, Warning, "FAssetRenameManager::AutoCheckOut: package %ls is read-only, will not make writable", *PackageFilename);
					bSomethingFailed = true;
				}
				else
				{
					PackagesToCheckOut.RemoveAtSwap(Index);
				}
			}
		}
	}

	return !bSomethingFailed;
}

void FAssetRenameManager::DetectReferencingCollections(TArray<FAssetRenameDataWithReferencers>& AssetsToRename) const
{
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	CollectionManagerModule.Get().GetCollectionContainers(CollectionContainers);

	TArray<FCollectionNameType> ReferencingCollections;
	for (FAssetRenameDataWithReferencers& AssetToRename : AssetsToRename)
	{
		if (AssetToRename.Asset.IsValid())
		{
			TArray<FCollectionNameType> ObjectCollections;
			for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
			{
				ReferencingCollections.Reset();
				CollectionContainer->GetCollectionsContainingObject(FSoftObjectPath(AssetToRename.Asset.Get()), ReferencingCollections);

				if (ReferencingCollections.Num() > 0)
				{
					AssetToRename.bCreateRedirector = true;
					break;
				}
			}
		}
	}
}

void FAssetRenameManager::DetectReadOnlyPackages(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, TArray<UPackage*>& InOutReferencingPackagesToSave) const
{
	// For each valid package...
	for (int32 PackageIdx = InOutReferencingPackagesToSave.Num() - 1; PackageIdx >= 0; --PackageIdx)
	{
		UPackage* Package = InOutReferencingPackagesToSave[PackageIdx];

		if (Package)
		{
			// Find the package filename
			FString Filename;
			if (FPackageName::DoesPackageExist(Package->GetName(), &Filename))
			{
				// If the file is read only
				if (IFileManager::Get().IsReadOnly(*Filename))
				{
					FName PackageName = Package->GetFName();

					// Find all assets that were referenced by this package to create a redirector when named
					for (FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
					{
						if (RenameData.NotRenamedReferencingPackageNames.Contains(PackageName))
						{
							RenameData.bCreateRedirector = true;
						}
					}

					// Remove the package from the save list
					InOutReferencingPackagesToSave.RemoveAt(PackageIdx);
				}
			}
		}
	}
}

void FAssetRenameManager::SetupPublicAssets(TArray<FAssetRenameDataWithReferencers>& AssetsToRename) const
{
	static const IConsoleVariable* EnablePublicAssetFeatureCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("AssetTools.EnablePublicAssetFeature"));
	if (!EnablePublicAssetFeatureCVar || !EnablePublicAssetFeatureCVar->GetBool())
	{
		return;
	}

	FScopedSlowTask SlowTask(static_cast<float>(AssetsToRename.Num()), LOCTEXT("SetupPublicAssets", "Setting up public assets..."));
	SlowTask.MakeDialog();

	IAssetTools& AssetTools = IAssetTools::Get();
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	// Build a map of old package name to rename data
	TMap<FName, FAssetRenameDataWithReferencers*> OldPackageNameToRenameData;
	for (FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
	{
		OldPackageNameToRenameData.Add(RenameData.OldObjectPath.GetLongPackageFName(), &RenameData);
	}

	TArray<UPackage*> PackagesToMakePublic;
	TSet<FName> RenamedReferencedPackageNames;
	bool bSomeAssetCannotBeMovedToAnotherPlugin = false;

	// Determine which moved asset need to become public
	for (FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
	{
		SlowTask.EnterProgressFrame();

		// Nothing to do if it won't be moved
		if (RenameData.bRenameFailed || RenameData.bOnlyFixSoftReferences)
		{
			continue;
		}

		UObject* Asset = RenameData.Asset.Get();
		UPackage* Package = Asset ? Asset->GetPackage() : nullptr;
		if (!ensure(Package))
		{
			continue;
		}

		// Nothing to do if the renamed package mount point doesn't change
		const FName OldPackageName = RenameData.OldObjectPath.GetLongPackageFName();
		const FName NewPackageName = RenameData.NewObjectPath.GetLongPackageFName();
		const FNameBuilder OldPackageNameBuilder(OldPackageName);
		const FNameBuilder NewPackageNameBuilder(NewPackageName);
		const FStringView OldPackageMountPoint = FPathViews::GetMountPointNameFromPath(OldPackageNameBuilder);
		const FStringView NewPackageMountPoint = FPathViews::GetMountPointNameFromPath(NewPackageNameBuilder);
		if (NewPackageMountPoint.Equals(OldPackageMountPoint, ESearchCase::IgnoreCase))
		{
			continue;
		}

		// Make sure the asset won't be referencing a private asset in a different mount point after the move
		{
			TArray<FName> DependencyPackageNames;
			AssetRegistry.GetDependencies(Package->GetFName(), DependencyPackageNames);
			for (FName DependencyPackageName : DependencyPackageNames)
			{
				if (OldPackageNameToRenameData.Find(DependencyPackageName))
				{
					RenamedReferencedPackageNames.Add(DependencyPackageName);

					// Dependency is being renamed as well so it'll be handled later
					continue;
				}

				{
					const FNameBuilder DepPackageNameBuilder(DependencyPackageName);
					const FStringView DependencyMountPoint = FPathViews::GetMountPointNameFromPath(DepPackageNameBuilder);
					if (DependencyMountPoint.Equals(NewPackageMountPoint, ESearchCase::IgnoreCase))
					{
						// Dependency is under the same mount point as the new asset location
						continue;
					}
				}

				{
					TArray<FAssetData> DependencyAssetDatas;
					if (AssetRegistry.GetAssetsByPackageName(DependencyPackageName, DependencyAssetDatas) && !DependencyAssetDatas.IsEmpty())
					{
						if (DependencyAssetDatas[0].GetAssetAccessSpecifier() == EAssetAccessSpecifier::Private)
						{
							RenameData.bRenameFailed = true;
							RenameData.FailureReason = FText::Format(
								LOCTEXT("MovedAssetReferencingPrivateAsset", "Cannot move asset to {0} because it would be referencing a private asset in a different plugin: {1}"),
								FText::FromString(AssetTools.GetUserFacingPluginName(NewPackageMountPoint)),
								FText::FromString(AssetTools.GetUserFacingLongPackageName(DependencyPackageName)));
							break;
						}
					}
				}
			}

			if (RenameData.bRenameFailed)
			{
				bSomeAssetCannotBeMovedToAnotherPlugin = true;
				continue;
			}
		}

		// If the asset isn't private, it can be referenced externally so we're good
		if (Package->GetAssetAccessSpecifier() != EAssetAccessSpecifier::Private)
		{
			continue;
		}

		// The asset doesn't have to become public if it's not referenced at all
		if (!RenameData.bCreateRedirector && RenameData.NotRenamedReferencingPackageNames.IsEmpty() && RenameData.RenamedReferencingPackageNames.IsEmpty())
		{
			continue;
		}

		// Figure out if it's gonna be referenced from another mount point after the move
		FName ReferencingAssetInDifferentMountPoint;
		if (RenameData.bCreateRedirector)
		{
			ReferencingAssetInDifferentMountPoint = OldPackageName;
		}
		if (ReferencingAssetInDifferentMountPoint.IsNone())
		{
			for (FName It : RenameData.NotRenamedReferencingPackageNames)
			{
				const FNameBuilder ReferencingPackageName(It);
				const FStringView ReferencingPackagMountPoint = FPathViews::GetMountPointNameFromPath(ReferencingPackageName);
				if (!ReferencingPackagMountPoint.Equals(NewPackageMountPoint, ESearchCase::IgnoreCase))
				{
					ReferencingAssetInDifferentMountPoint = It;
					break;
				}
			}
		}
		if (ReferencingAssetInDifferentMountPoint.IsNone())
		{
			for (const TPair<FName, FName>& It : RenameData.RenamedReferencingPackageNames)
			{
				const FName OldReferencingPackageName = It.Key;
				const FName NewReferencingPackageName = It.Value;

				FAssetRenameDataWithReferencers** FoundReferencingRenameData = OldPackageNameToRenameData.Find(OldReferencingPackageName);
				if (!ensureAlways(FoundReferencingRenameData))
				{
					continue;
				}
				FAssetRenameDataWithReferencers& ReferencingRenameData = **FoundReferencingRenameData;

				FName ReferencingPackageName;
				if (ReferencingRenameData.bRenameFailed || ReferencingRenameData.bOnlyFixSoftReferences)
				{
					ReferencingPackageName = OldReferencingPackageName;
				}
				else
				{
					ReferencingPackageName = NewReferencingPackageName;
				}

				const FNameBuilder ReferencingPackageNameBuilder(ReferencingPackageName);
				const FStringView ReferencingPackagMountPoint = FPathViews::GetMountPointNameFromPath(ReferencingPackageNameBuilder);
				if (!NewPackageMountPoint.Equals(ReferencingPackagMountPoint, ESearchCase::IgnoreCase))
				{
					ReferencingAssetInDifferentMountPoint = ReferencingPackageName;
					break;
				}
			}
		}

		// Check if the asset can be made public
		if (!ReferencingAssetInDifferentMountPoint.IsNone())
		{
			const FString OldAssetPath = RenameData.OldObjectPath.GetAssetPathString();
			FText DisallowedSetPublicReason;
			if (AssetTools.CanAssetBePublic(OldAssetPath, &DisallowedSetPublicReason))
			{
				PackagesToMakePublic.Add(Package);
			}
			else
			{
				bSomeAssetCannotBeMovedToAnotherPlugin = true;
				RenameData.bRenameFailed = true;
				if (RenameData.bCreateRedirector)
				{
					RenameData.FailureReason = FText::Format(
						LOCTEXT("AssetCannotBePublicForRedirector", "Cannot move asset to {0} because it cannot be made public in order to be referenced from a different plugin by its redirector. Additional Reason: {1}"),
						FText::FromStringView(AssetTools.GetUserFacingPluginName(NewPackageMountPoint)),
						DisallowedSetPublicReason);
				}
				else
				{
					RenameData.FailureReason = FText::Format(
						LOCTEXT("AssetCannotBePublic", "Cannot move asset to {0} because it cannot be made public in order to be referenced from a different plugin by {1}. Additional Reason: {2}"),
						FText::FromStringView(AssetTools.GetUserFacingPluginName(NewPackageMountPoint)),
						FText::FromString(AssetTools.GetUserFacingLongPackageName(ReferencingAssetInDifferentMountPoint)),
						DisallowedSetPublicReason);
				}
			}
		}
	}

	if (bSomeAssetCannotBeMovedToAnotherPlugin)
	{
		// When an asset cannot be moved to another plugin, it changes the set of inter-plugin dependencies and it can require other assets 
		// to become public as a result, which might not be intended by the user and those cascading effects would be difficult to grok.
		// In this case, only rename assets that have no dependency against each other.
		// @fixme: This is a little too aggressive. Ideally we'd check whether each asset actually has a direct or indirect
		//         dependency with an asset that failed to be moved to another plugin but that could be expensive.
		const FText FailureReason = LOCTEXT("DependentAssetCannotBeMoved", "Cannot rename asset because dependent assets could not be moved to a different plugin");
		TSet<FName> RenamedReferencingPackageNames;

		for (FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
		{
			for (const TPair<FName, FName>& It : RenameData.RenamedReferencingPackageNames)
			{
				RenamedReferencingPackageNames.Add(It.Key);
			}

			if (!RenameData.bRenameFailed && !RenameData.bOnlyFixSoftReferences && !RenameData.RenamedReferencingPackageNames.IsEmpty())
			{
				RenameData.bRenameFailed = true;
				RenameData.FailureReason = FailureReason;
			}
		}

		for (FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
		{
			if (!RenameData.bRenameFailed && !RenameData.bOnlyFixSoftReferences)
			{
				const FName PackageName = RenameData.OldObjectPath.GetLongPackageFName();
				if (RenamedReferencingPackageNames.Contains(PackageName) || RenamedReferencedPackageNames.Contains(PackageName))
				{
					RenameData.bRenameFailed = true;
					RenameData.FailureReason = FailureReason;
				}
			}
		}
	}
	else
	{
		// Make assets public
		for (UPackage* Package : PackagesToMakePublic)
		{
			Package->SetAssetAccessSpecifier(EAssetAccessSpecifier::Public);
		}
	}
}

void FAssetRenameManager::RenameReferencingSoftObjectPaths(const TArray<UPackage*> PackagesToCheck, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap) const
{
	// Add redirects as needed
	for (const TPair<FSoftObjectPath, FSoftObjectPath>& Pair : AssetRedirectorMap)
	{
		if (Pair.Key.IsAsset())
		{
			GRedirectCollector.AddAssetPathRedirection(Pair.Key.GetWithoutSubPath(), Pair.Value.GetWithoutSubPath());
		}
	}

	FSoftObjectPathRenameSerializer RenameSerializer(AssetRedirectorMap, false, nullptr);

	for (UPackage* Package : PackagesToCheck)
	{
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithPackage(Package, ObjectsInPackage);

		for (UObject* Object : ObjectsInPackage)
		{
			if (!IsValid(Object))
			{
				continue;
			}

			RenameSerializer.StartSerializingObject(Object);
			Object->Serialize(RenameSerializer);

			if (UBlueprint* Blueprint = Cast<UBlueprint>(Object))
			{
				// Serialize may have dirtied the BP bytecode in some way
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}
	}
}

void FAssetRenameManager::OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty)
{
	// Remove from cache
	CachedSoftReferences.Remove(Pkg->GetFName());
}

bool FAssetRenameManager::CheckPackageForSoftObjectReferences(UPackage* Package, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap, TArray<UObject*>& OutReferencingObjects) const
{
	TMap<FSoftObjectPath, TArray<UObject*>> ReferencingObjectsMap;

	CheckPackageForSoftObjectReferences(Package, AssetRedirectorMap, ReferencingObjectsMap);

	// Build an array out of the map results.
	for (const auto& It : ReferencingObjectsMap)
	{
		for (UObject* Obj : It.Value)
		{
			OutReferencingObjects.AddUnique(Obj);
		}
	}
	return OutReferencingObjects.Num() != 0;
}

bool FAssetRenameManager::CheckPackageForSoftObjectReferences(UPackage* Package, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap, TMap<FSoftObjectPath, TArray<UObject*>>& OutReferencingObjects) const
{
	using namespace AssetRenameManagerImpl;

	bool bFoundReference = false;

	// First check cache
	FCachedSoftReference* CachedReferences = CachedSoftReferences.Find(Package->GetFName());

	if (CachedReferences == nullptr)
	{
		// Bind to dirty callback if we aren't already
		if (!DirtyDelegateHandle.IsValid())
		{
			DirtyDelegateHandle = UPackage::PackageMarkedDirtyEvent.AddSP(const_cast<FAssetRenameManager*>(this), &FAssetRenameManager::OnMarkPackageDirty);
		}

		//Extract all objects soft references along with their referencer and cache them to avoid having to serialize again
		TMap<FSoftObjectPath, FSoftObjectPath> EmptyMap;
		TMap<FSoftObjectPath, TSet<FWeakObjectPtr>> MapForCache;
		FSoftObjectPathRenameSerializer CheckSerializer(EmptyMap, true, &MapForCache, Package->GetFName());

		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithPackage(Package, ObjectsInPackage);

		for (UObject* Object : ObjectsInPackage)
		{
			if (!IsValid(Object))
			{
				continue;
			}

			CheckSerializer.StartSerializingObject(Object);
			Object->Serialize(CheckSerializer);
		}

		CachedReferences = &CachedSoftReferences.Add(Package->GetFName());
		CachedReferences->Map = MoveTemp(MapForCache);

		CachedReferences->Map.GenerateKeyArray(CachedReferences->Keys);

		// Keys need to be sorted for binary search
		CachedReferences->Keys.Sort(FSoftObjectPathFastLess());
	}

	for (const TPair<FSoftObjectPath, FSoftObjectPath>& Pair : AssetRedirectorMap)
	{
		const FString& CheckSubPath = Pair.Key.GetSubPathString();
		const FTopLevelAssetPath CheckAssetPath = Pair.Key.GetAssetPath();

		// Find where we're going to start iterating
		int32 Index = Algo::LowerBound(CachedReferences->Keys, Pair.Key, FSoftObjectPathFastLess());
		for (int32 Num = CachedReferences->Keys.Num(); Index < Num; ++Index)
		{
			const FSoftObjectPath& CachedKey = CachedReferences->Keys[Index];
			const FString& SubPath = CachedKey.GetSubPathString();

			// Stop as soon as we're not anymore in the range we're searching
			if (CheckAssetPath != CachedKey.GetAssetPath())
			{
				break;
			}

			// Check if CheckSubPath is included in SubPath first to handle this case:
			//
			// SubPath:      PersistentLevel.Level_1_4__Head_Level_300.Level_1_4__Head_Level_300
			//    which is >
			// CheckSubPath: PersistentLevel.Level_1_4__Head_Level_300
			//
			if (IsSubPath(SubPath, CheckSubPath))
			{
				for (const FWeakObjectPtr& WeakPtr : *CachedReferences->Map.Find(CachedKey))
				{
					UObject* ObjectPtr = WeakPtr.Get();
					if (ObjectPtr)
					{
						// Skip references from sub-objects of the redirected path:
						// CacheKey is a reference that was found in the package that is a sub-path of the current asset we are validating against (Pair.Key)
						// but ObjectPtr is the actual object that has that reference and if that ObjectPtr is actually a sub-object of the asset we are validating against (Pair.Key) 
						// then that sub-object is itself getting redirected so do not report.
						const FSoftObjectPath ObjectPath(ObjectPtr);
						if (ObjectPath.GetAssetPath() == CheckAssetPath && IsSubPath(ObjectPath.GetSubPathString(), CheckSubPath))
						{
							continue;
						}

						bFoundReference = true;
						OutReferencingObjects.FindOrAdd(CachedKey).AddUnique(ObjectPtr);
					}
				}
			}
			else if (SubPath > CheckSubPath)
			{
				// Stop once CheckSubPath is not included in SubPath anymore and we're out of search range
				break;
			}
		}
	}

	return bFoundReference;
}

void FAssetRenameManager::PerformAssetRename(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, FScopedSlowTask& RenamingSlowTask) const
{
	PerformAssetRename(AssetsToRename, TArray<UPackage*>(), RenamingSlowTask);
}

void FAssetRenameManager::PerformAssetRename(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, const TArray<UPackage*>& ReferencingPackagesToSave, FScopedSlowTask& RenamingSlowTask) const
{
	/**
	 * We need to collect and check those cause dependency graph is only
	 * representing on-disk state and we want to support rename for in-memory
	 * objects. It is only needed for string references as in memory references
	 * for other objects are pointers, so renames doesn't apply to those.
	 */
	TArray<UPackage*> DirtyPackagesToCheckForSoftReferences;

	FEditorFileUtils::GetDirtyWorldPackages(DirtyPackagesToCheckForSoftReferences);
	FEditorFileUtils::GetDirtyContentPackages(DirtyPackagesToCheckForSoftReferences);

	TArray<UPackage*> PackagesToSave = ReferencingPackagesToSave;
	TArray<UPackage*> PotentialPackagesToDelete;
	float ProgressStep = 2.0f / static_cast<float>(AssetsToRename.Num());
	for (int32 AssetIdx = 0; AssetIdx < AssetsToRename.Num(); ++AssetIdx)
	{
		RenamingSlowTask.EnterProgressFrame(ProgressStep);

		FAssetRenameDataWithReferencers& RenameData = AssetsToRename[AssetIdx];

		if (RenameData.bRenameFailed)
		{
			// The rename failed at some earlier step, skip this asset
			continue;
		}

		UObject* Asset = RenameData.Asset.Get();
		TArray<UPackage*> PackagesToCheckForSoftReferences;

		bool bIsCaseChangeOnly = false;
		if (!RenameData.bOnlyFixSoftReferences)
		{
			// If bOnlyFixSoftReferences was set these got appended in find references
			PackagesToCheckForSoftReferences.Append(DirtyPackagesToCheckForSoftReferences);

			if (!Asset)
			{
				// This asset was invalid or GCed before the rename could occur
				RenameData.bRenameFailed = true;
				continue;
			}

			ObjectTools::FPackageGroupName PGN;
			PGN.ObjectName = RenameData.NewName;
			PGN.GroupName = TEXT("");
			PGN.PackageName = RenameData.NewPackagePath / PGN.ObjectName;
			bool bLeaveRedirector = RenameData.bCreateRedirector;

			UPackage* OldPackage = Asset->GetOutermost();

			if (OldPackage->GetFName() == FName(PGN.PackageName) && !OldPackage->IsRooted())
			{
				// Handle case change only.
				bLeaveRedirector = false;

				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
				FString PackageName;
				FString AssetName;
				FString BasePath = RenameData.NewPackagePath + TEXT("/RenameTmp");
				AssetToolsModule.Get().CreateUniqueAssetName(BasePath, TEXT(""), PackageName, AssetName);

				ObjectTools::FPackageGroupName TempPGN;
				TempPGN.ObjectName = AssetName;
				TempPGN.GroupName = TEXT("");
				TempPGN.PackageName = RenameData.NewPackagePath / TempPGN.ObjectName;

				TSet<UPackage*> ObjectsUserRefusedToFullyLoad;
				FText ErrorMessage;

				// Case insensitive file systems and source control providers clients often handle poorly a case change.
				bool bIsLocal = true;
				if (ISourceControlModule::Get().IsEnabled())
				{
					ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
					if (FSourceControlStatePtr StatePtr = Provider.GetState(Asset->GetPackage(), EStateCacheUsage::ForceUpdate))
					{
						if (StatePtr->IsSourceControlled())
						{
							bIsLocal = false;
							ErrorMessage = LOCTEXT("ErrorCaseChangeRenameWithSourceControl", "Couldn't perform a case-only rename on a revision controlled asset, as this is not supported.");
						}
					}
				}

				if (bIsLocal && ObjectTools::RenameSingleObject(Asset, TempPGN, ObjectsUserRefusedToFullyLoad, ErrorMessage, nullptr, bLeaveRedirector))
				{
					TArray<UPackage*> OldPackageToClean;
					OldPackageToClean.Add(OldPackage);
					OldPackage = Asset->GetPackage();
					OldPackage->AddToRoot();
					ObjectTools::CleanupAfterSuccessfulDelete(OldPackageToClean);
					OldPackage->RemoveFromRoot();
					bIsCaseChangeOnly = true;
				}
				else
				{
					RenameData.bRenameFailed = true;
					RenameData.FailureReason = ErrorMessage;
					continue;
				}
			}


			bool bOldPackageAddedToRootSet = false;
			if (!bLeaveRedirector && OldPackage && !OldPackage->IsRooted())
			{
				bOldPackageAddedToRootSet = true;
				OldPackage->AddToRoot();
			}

			TSet<UPackage*> ObjectsUserRefusedToFullyLoad;
			FText ErrorMessage;
			if (ObjectTools::RenameSingleObject(Asset, PGN, ObjectsUserRefusedToFullyLoad, ErrorMessage, nullptr, bLeaveRedirector))
			{
				/**
				 * Do not save the package when the user simply changing a case and there is no referencer to it
				 */
				if (!(bIsCaseChangeOnly && RenameData.NotRenamedReferencingPackageNames.IsEmpty()))
				{
					PackagesToSave.AddUnique(Asset->GetOutermost());
				}

				// Automatically save renamed assets
				if (bLeaveRedirector)
				{
					PackagesToSave.AddUnique(OldPackage);
				}
				else if (bOldPackageAddedToRootSet)
				{
					// Since we did not leave a redirector and the old package wasn't already rooted, attempt to delete it when we are done. 
					PotentialPackagesToDelete.AddUnique(OldPackage);
				}
			}
			else
			{
				// No need to keep the old package rooted, the asset was never renamed out of it
				if (bOldPackageAddedToRootSet)
				{
					OldPackage->RemoveFromRoot();
				}

				// Mark the rename as a failure to report it later
				RenameData.bRenameFailed = true;
				RenameData.FailureReason = ErrorMessage;
			}
		}

		if (!RenameData.bRenameFailed && !bIsCaseChangeOnly)
		{
			for (FName PackageName : RenameData.NotRenamedReferencingPackageNames)
			{
				UPackage* PackageToCheck = FindPackage(nullptr, *PackageName.ToString());
				if (PackageToCheck)
				{
					PackagesToCheckForSoftReferences.Add(PackageToCheck);
				}
			}

			TMap<FSoftObjectPath, FSoftObjectPath> RedirectorMap;
			RedirectorMap.Add(RenameData.OldObjectPath, RenameData.NewObjectPath);

			if (UBlueprint* Blueprint = Cast<UBlueprint>(Asset))
			{
				// Add redirect for class and default as well
				RedirectorMap.Add(FString::Printf(TEXT("%s_C"), *RenameData.OldObjectPath.ToString()), FString::Printf(TEXT("%s_C"), *RenameData.NewObjectPath.ToString()));
				RedirectorMap.Add(FString::Printf(TEXT("%s.Default__%s_C"), *RenameData.OldObjectPath.GetLongPackageName(), *RenameData.OldObjectPath.GetAssetName()), FString::Printf(TEXT("%s.Default__%s_C"), *RenameData.NewObjectPath.GetLongPackageName(), *RenameData.NewObjectPath.GetAssetName()));
			}

			RenameReferencingSoftObjectPaths(PackagesToCheckForSoftReferences, RedirectorMap);
		}
	}

	const FText& UpdatingRevisionControlSlowTask = LOCTEXT("UpdatingRevisionControlSlowTask", "Updating Revision Control...");

	// Bulk update SCC status for old packages since it is faster than doing it one by one below
	if (ISourceControlModule::Get().IsEnabled())
	{
		RenamingSlowTask.EnterProgressFrame(0.5f, UpdatingRevisionControlSlowTask);

		TArray<FString> AllSourceFilenamesToCheck;

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		for (const FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
		{
			UPackage* OldPackage = FindPackage(nullptr, *RenameData.OldObjectPath.GetLongPackageName());
			if (!RenameData.bOnlyFixSoftReferences)
			{
				const FString SourceFilename = USourceControlHelpers::PackageFilename(OldPackage);
				AllSourceFilenamesToCheck.Add(SourceFilename);
			}
		}

		if (AllSourceFilenamesToCheck.Num() > 0)
		{
			TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe>> States;
			SourceControlProvider.GetState(AllSourceFilenamesToCheck, States, EStateCacheUsage::ForceUpdate);
		}
	}
	else
	{
		RenamingSlowTask.EnterProgressFrame(0.5f, FText::GetEmpty());
	}

	// Now branch the files in source control if possible
	if (ISourceControlModule::Get().IsEnabled())
	{
		RenamingSlowTask.EnterProgressFrame(2.9f, UpdatingRevisionControlSlowTask);
		FScopedSlowTask BuildingRelationshipSlowTask(2.9f, LOCTEXT("BranchingRenamedAssetsSlowTask", "Building a relationship between the renamed files in Revision Control..."));

		ProgressStep = 2.9f / static_cast<float>(AssetsToRename.Num());
		TMap<UPackage*, UPackage> PackagesToBranch;
		TArray<FString> LongPackageNamesToCheckout;
		for (const FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
		{
			BuildingRelationshipSlowTask.EnterProgressFrame(ProgressStep);

			UPackage* OldPackage = FindPackage(nullptr, *RenameData.OldObjectPath.GetLongPackageName());
			UPackage* NewPackage = FindPackage(nullptr, *RenameData.NewObjectPath.GetLongPackageName());
			if (!RenameData.bOnlyFixSoftReferences)
			{
				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
				const FString SourceFilename = USourceControlHelpers::PackageFilename(OldPackage);
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SourceFilename, EStateCacheUsage::Use);
				if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled())
				{
					// Do not attempt to branch if the old file was open for add
					if (!SourceControlState->IsAdded())
					{
						SourceControlHelpers::BranchPackage(NewPackage, OldPackage, EStateCacheUsage::Use);
						// Also mark the file for edit so that it can be saved later
						LongPackageNamesToCheckout.Add(*RenameData.NewObjectPath.GetLongPackageName());
					}
				}
			}
		}
		SourceControlHelpers::CheckOutFiles(LongPackageNamesToCheckout, true);
	}
	else
	{
		RenamingSlowTask.EnterProgressFrame(2.9f, FText::GetEmpty());
	}

	// Save all renamed assets and any redirectors that were left behind
	{
		RenamingSlowTask.EnterProgressFrame(1.0f, LOCTEXT("SavingRenamedAssetSlowTask", "Saving renamed assets..."));

		TArray<UPackage*> FailedPackages;
		if (PackagesToSave.Num() > 0)
		{
			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			const bool bAlreadyCheckedOut = true;

			// Get the list of filenames before calling save because some of the saved packages can get GCed if they are empty packages
			const TArray<FString> Filenames = USourceControlHelpers::PackageFilenames(PackagesToSave);

			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave, &FailedPackages, bAlreadyCheckedOut);

			ISourceControlModule::Get().QueueStatusUpdate(Filenames);
		}

		// Revert any previously branched files that did not actually save
		if (ISourceControlModule::Get().IsEnabled())
		{
			RenamingSlowTask.EnterProgressFrame(1.0f, UpdatingRevisionControlSlowTask);

			TArray<FString> FilesToRevert = USourceControlHelpers::PackageFilenames(FailedPackages);
			if (FilesToRevert.Num() > 0)
			{
				USourceControlHelpers::RevertFiles(FilesToRevert);
			}
		}
		else
		{
			RenamingSlowTask.EnterProgressFrame(1.0f, FText::GetEmpty());
		}
	}

	// Clean up all packages that were left empty
	if (PotentialPackagesToDelete.Num() > 0)
	{
		for (UPackage* Package : PotentialPackagesToDelete)
		{
			Package->RemoveFromRoot();
		}

		ObjectTools::CleanupAfterSuccessfulDelete(PotentialPackagesToDelete);
	}
}

void FAssetRenameManager::SaveReferencingPackages(const TArray<UPackage*>& ReferencingPackagesToSave) const
{
	if (ReferencingPackagesToSave.Num() > 0)
	{
		// Get the list of filenames before calling save because some of the saved packages can get GCed if they are empty packages
		const TArray<FString> Filenames = USourceControlHelpers::PackageFilenames(ReferencingPackagesToSave);

		const bool bCheckDirty = false;
		const bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(ReferencingPackagesToSave, bCheckDirty, bPromptToSave);

		ISourceControlModule::Get().QueueStatusUpdate(Filenames);
	}
}

void FAssetRenameManager::RenameInterrupted(const TArray<FAssetRenameData>& AssetsToRename, const FText& InterruptionReason, bool bWithDialog) const
{
	TArray<FAssetRenameDataWithReferencers> AssetsToRenameWithReferencers;
	for (const FAssetRenameData& AssetRenameData : AssetsToRename)
	{
		FAssetRenameDataWithReferencers AssetToRenameWithReferencers(AssetRenameData);
		AssetToRenameWithReferencers.bRenameFailed = true;
		AssetToRenameWithReferencers.FailureReason = InterruptionReason;
		AssetsToRenameWithReferencers.Add(AssetToRenameWithReferencers);
	}
	ReportFailures(AssetsToRenameWithReferencers, bWithDialog);
}

int32 FAssetRenameManager::ReportFailures(const TArray<FAssetRenameDataWithReferencers>& AssetsToRename, bool bWithDialog) const
{
	TArray<FText> FailedRenames;
	for (const FAssetRenameDataWithReferencers& RenameData : AssetsToRename)
	{
		if (RenameData.bRenameFailed)
		{
			UObject* Asset = RenameData.Asset.Get();
			if (Asset)
			{
				UE::Core::FVersePath VersePath;
				if (IAssetTools::Get().ShowingContentVersePath())
				{
					VersePath = Asset->GetVersePath();
				}

				FText AssetName;
				if (VersePath.IsValid())
				{
					AssetName = MoveTemp(VersePath).AsText();
				}
				else
				{
					AssetName = FText::FromName(Asset->GetOutermost()->GetFName());
				}

				FFormatNamedArguments Args;
				Args.Add(TEXT("FailureReason"), RenameData.FailureReason);
				Args.Add(TEXT("AssetName"), MoveTemp(AssetName));

				FailedRenames.Add(FText::Format(LOCTEXT("AssetRenameFailure", "{AssetName} - {FailureReason}"), Args));
			}
			else
			{
				FailedRenames.Add(LOCTEXT("InvalidAssetText", "Invalid Asset"));
			}
		}
	}

	if (FailedRenames.Num() > 0)
	{
		if (bWithDialog)
		{
			SFileListReportDialog::OpenListDialog(LOCTEXT("FailedRenamesDialog", "Failed Renames"),
				LOCTEXT("RenameFailureTitle", "The following assets could not be renamed."),
				FailedRenames);
		}
		else
		{
			for (const FText& FailedRename : FailedRenames)
			{
				UE_LOGF(LogAssetTools, Error, "%ls", *FailedRename.ToString());
			}
		}
	}

	return FailedRenames.Num();
}

bool FAssetRenameManager::RequiresCheckingForVariants(const TArray<FAssetRenameData>& InAssetsToRename, TArray<FAssetRenameData>& OutAssetsAndVariants, TArray<FAssetRenameData>& OutAssetsToCheckForVariants) const
{
	const UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get();

	const UEditorProjectAssetSettings* Settings = GetDefault<UEditorProjectAssetSettings>();
	bool bSettingsRequiresCheckingForVariants = !(Settings && !Settings->bRenameLocalizedVariantsAlongsideSourceAsset);
	if (bSettingsRequiresCheckingForVariants)
	{
		OutAssetsAndVariants.Reserve(InAssetsToRename.Num());
		OutAssetsToCheckForVariants.Reserve(InAssetsToRename.Num());
		for (const FAssetRenameData& AssetAndName : InAssetsToRename)
		{
			UClass* CurrentAssetClass = AssetAndName.Asset != nullptr ? AssetAndName.Asset->GetClass() : nullptr;
			const UAssetDefinition* CurrentAssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForClass(CurrentAssetClass);
			bool bAssetRequiresCheckingForVariant = AssetAndName.bAlsoRenameLocalizedVariants && CurrentAssetDefinition != nullptr && CurrentAssetDefinition->CanLocalize(FAssetData()).IsSupported();
			if (bAssetRequiresCheckingForVariant)
			{
				OutAssetsToCheckForVariants.Add(AssetAndName);
			}
			else
			{
				OutAssetsAndVariants.Add(AssetAndName);
			}
		}
	}
	else
	{
		OutAssetsAndVariants = InAssetsToRename;
	}

	return !OutAssetsToCheckForVariants.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
