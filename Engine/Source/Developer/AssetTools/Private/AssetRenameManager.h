// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAssetTools.h"
#include "UObject/SoftObjectPath.h"

struct FAssetRenameDataWithReferencers;

struct FCachedSoftReference
{
	// Insert friendly
	TMap<FSoftObjectPath, TSet<FWeakObjectPtr>> Map;

	// So we can binary search for the TMap keys
	TArray<FSoftObjectPath> Keys;
};

/** 
 * The manager to handle renaming assets.
 * This manager attempts to fix up references in memory if possible and only leaves UObjectRedirectors when needed.
 * Redirectors are left unless ALL of the following are true about the asset
 *    1) The asset has not yet been checked into source control. This does not apply when source control is disabled.
 *    2) The user is able and willing to check out all uasset files that directly reference the asset from source control. The files must be at head revision and not checked out by another user. This rule does not apply when source control is disabled.
 *    3) No maps reference the asset directly.
 *    4) All uasset files that directly reference the asset are writable on disk.
 */
class FAssetRenameManager : public TSharedFromThis<FAssetRenameManager>
{
public:
	/** Renames assets using the specified names. */
	bool RenameAssets(const TArray<FAssetRenameData>& AssetsAndNames) const;

	/** Renames assets using the specified names. */
	EAssetRenameResult RenameAssetsWithDialog(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout = false) const;

	/** Returns list of objects that soft reference the given soft object path. This will load assets into memory to verify */
	void FindSoftReferencesToObject(FSoftObjectPath TargetObject, TArray<UObject*>& ReferencingObjects) const;

	/** Returns list of objects that soft reference the given soft object paths. This will load assets into memory to verify */
	void FindSoftReferencesToObjects(const TArray<FSoftObjectPath>& TargetObjects, TMap<FSoftObjectPath, TArray<UObject*>>& ReferencingObjects) const;

	/** Accessor for post rename event */
	FAssetPostRenameEvent& OnAssetPostRenameEvent() { return AssetPostRenameEvent; }

	/**
	 * Function that renames all FSoftObjectPath object with the old asset path to the new one.
	 *
	 * @param PackagesToCheck Packages to check for referencing FSoftObjectPath.
	 * @param AssetRedirectorMap Map from old asset path to new asset path
	 */
	void RenameReferencingSoftObjectPaths(TArray<UPackage*> PackagesToCheck, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap) const;

	/** Filters packages list depending on if it actually has soft object paths pointing to the specific object being renamed */
	bool CheckPackageForSoftObjectReferences(UPackage* Package, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap, TArray<UObject*>& OutReferencingObjects) const;

	/** Filters packages list depending on if it actually has soft object paths pointing to the specific object being renamed */
	bool CheckPackageForSoftObjectReferences(UPackage* Package, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap, TMap<FSoftObjectPath, TArray<UObject*>>& OutReferencingObjects) const;

private:
	/** Callback used by DiscoverintAssetsDialog to call RenameAssetsAndVariants */
	void RenameAssetsAndVariantsCallback(TArray<FAssetRenameData> InAssetsRenameData, bool bAutoCheckout, bool bWithDialog) const;

	/** Check if renaming these assets should also rename localized variants assets (or source asset) */
	bool RenameAssetsAndVariants(const TArray<FAssetRenameData>& InAssetsRenameData, bool bAutoCheckout, bool bWithDialog) const;

	/** Attempts to load and fix redirector references for the supplied assets */
	bool FixReferencesAndRename(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout, bool bWithDialog, FScopedSlowTask& RenamingSlowTask) const;

	/**
	 * Get lists of assets with references from CDOs
	 * @param AssetsToRename      List of assets to be renamed
	 * @param OutReferences       Output array of hard asset references
	 * @param OutSoftReferences   Output array of soft path asset references
	 * @param bSetRedirectorFlags If true, each renamed asset with a soft reference will be marked as needing a redirector
	 */
	void FindCDOReferences(const TArrayView<FAssetRenameDataWithReferencers>& AssetsToRename, TArray<FAssetRenameDataWithReferencers*>& OutHardReferences, TArray<FAssetRenameDataWithReferencers*>& OutSoftReferences, bool bSetRedirectorFlags) const;
	
	/** Fills out the Referencing packages for all the assets described in AssetsToPopulate */
	void PopulateAssetReferencers(TArray<FAssetRenameDataWithReferencers>& AssetsToPopulate) const;

	/** Updates the source control status of the packages containing the assets to rename */
	bool UpdatePackageStatus(TArray<FAssetRenameDataWithReferencers>& AssetsToRename) const;

	/**
	 * Loads all referencing packages to assets in AssetsToRename, finds assets whose references can
	 * not be fixed up to mark that a redirector should be left, and returns a list of referencing packages to save.
	 * If bLoadAllPackages is true, it will load all referencing packages even if they can't be checked out
	 * If bCheckStatus is true it will check the source control status
	 */
	void LoadReferencingPackages(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, bool bLoadAllPackages, bool bCheckStatus, TArray<UPackage*>& OutReferencingPackagesToSave, TArray<UObject*>& OutSoftReferencingObjects) const;

    /** Gather a list of referencing object for each of the asset in AssetsToRename. Will load all referencing packages */
	void GatherReferencingObjects(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, TMap<FSoftObjectPath, TArray<UObject*>>& OutSoftReferencingObjects) const;

	/** 
	 * Prompts to check out the source package and all referencing packages and marks assets whose referencing packages were not checked out to leave a redirector.
	 * Trims PackagesToSave when necessary.
	 * Returns true if the user opted to continue the operation or no dialog was required.
	 */
	bool CheckOutPackages(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, TArray<UPackage*>& InOutReferencingPackagesToSave, bool bAutoCheckout) const;

	/** Attempts to check out packages, returns false on any failure */
	bool AutoCheckOut(TArray<UPackage*>& PackagesToCheckOut) const;

	/** Finds any collections that are referencing the assets to be renamed. Assets referenced by collections will leave redirectors */
	void DetectReferencingCollections(TArray<FAssetRenameDataWithReferencers>& AssetsToRename) const;

	/** Finds any read only packages and removes them from the save list. Assets referenced by these packages will leave redirectors. */ 
	void DetectReadOnlyPackages(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, TArray<UPackage*>& InOutReferencingPackagesToSave) const;

	/** 
	 * Make public any private asset that will be referenced from another plugin after the rename.
	 * If the asset cannot be made public or if moving it requires a referenced asset that's not being modified to become public,
	 * its rename will fail and other assets that have dependencies between them will also fail to be renamed.
	 */
	void SetupPublicAssets(TArray<FAssetRenameDataWithReferencers>& AssetsToRename) const;

	/** Performs the asset rename after the user has selected to proceed */
	void PerformAssetRename(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, FScopedSlowTask& RenamingSlowTask) const;

	/** Performs the asset rename after the user has selected to proceed, also saving the provided referencing packages at the same time */
	void PerformAssetRename(TArray<FAssetRenameDataWithReferencers>& AssetsToRename, const TArray<UPackage*>& ReferencingPackagesToSave, FScopedSlowTask& RenamingSlowTask) const;

	/** Saves all the referencing packages and updates SCC state */
	void SaveReferencingPackages(const TArray<UPackage*>& ReferencingPackagesToSave) const;

	/** Prematurely interrupt the rename process for the given reason */
	void RenameInterrupted(const TArray<FAssetRenameData>& AssetsToRename, const FText& InterruptionReason, bool bWithDialog) const;

	/** Report any failures that may have happened during the rename. Return the number of failures */
	int32 ReportFailures(const TArray<FAssetRenameDataWithReferencers>& AssetsToRename, bool bWithDialog) const;

	/** Internal check to know if we should look for localized variants */
	bool RequiresCheckingForVariants(const TArray<FAssetRenameData>& InAssetsToRename, TArray<FAssetRenameData>& OutAssetsAndVariants, TArray<FAssetRenameData>& OutAssetsToCheckForVariants) const;

	/** Called when a package is dirtied, clears the cache */
	void OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty);

	/** Event issued at the end of the rename process */
	FAssetPostRenameEvent AssetPostRenameEvent;

	/** Cache of package->soft references, to avoid serializing the same package over and over */
	mutable TMap<FName, FCachedSoftReference> CachedSoftReferences;
	mutable FDelegateHandle DirtyDelegateHandle;
};
