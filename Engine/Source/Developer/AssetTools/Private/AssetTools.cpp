// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools.h"
#include "Factories/Factory.h"
#include "Factories/BlueprintFactory.h"
#include "MaterialShared.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Blueprint.h"
#include "Engine/SCS_Node.h"
#include "Exporters/Exporter.h"
#include "Editor/EditorEngine.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Editor.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "AssetToolsLog.h"
#include "AssetToolsModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "AssetDefinition_AssetTypeActionsProxy.h"
#include "IClassTypeActions.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "AssetTypeActions/AssetTypeActions_AnimBoneCompressionSettings.h"
#include "AssetTypeActions/AssetTypeActions_AnimCurveCompressionSettings.h"
#include "AssetTypeActions/AssetTypeActions_VariableFrameStrippingSettings.h"
#include "AssetTypeActions/AssetTypeActions_DeletedObjectPlaceholder.h"
#include "WorldPartition/WorldPartition.h"
#include "SDiscoveringAssetsDialog.h"
#include "AssetFixUpRedirectors.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DesktopPlatformModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SPackageReportDialog.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Logging/MessageLog.h"
#include "UnrealExporter.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AutomatedAssetImportData.h"
#include "AssetImportTask.h"
#include "Dialogs/DlgPickPath.h"
#include "Misc/FeedbackContext.h"
#include "BusyCursor.h"
#include "AssetExportTask.h"
#include "Containers/VersePath.h"
#include "Containers/UnrealString.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "AdvancedCopyCustomization.h"
#include "SAdvancedCopyReportDialog.h"
#include "AssetToolsSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/NamePermissionList.h"
#include "InterchangeManager.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeProjectSettings.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Animation/AnimSequence.h"
#include "UObject/SavePackage.h"
#include "Dialogs/Dialogs.h"
#include "VerseVM/VVMVerseClass.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Dialog/SMessageDialog.h"
#include "Interfaces/IPluginManager.h"
#include "Settings/ContentBrowserSettings.h"
#include "Algo/Count.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "UObject/NameTypes.h"
#include "ExternalPackageHelper.h"
#include "UObject/LinkerInstancingContext.h"
#include "PackageMigrationContext.h"
#include "ComponentReregisterContext.h"
#include "AssetViewUtils.h"
#include "LocalizedAssetTools.h"

#if WITH_EDITOR
#include "Subsystems/AssetEditorSubsystem.h"
#include "Settings/EditorExperimentalSettings.h"
#include "ObjectTools.h"
#endif

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "DiffUtils.h"
#include "VirtualTexturingEditorModule.h"
#include "Algo/AnyOf.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Factories/SceneImportFactory.h"
#include "Misc/AssetFilterData.h"

#include "AssetHeaderPatcher.h"
#include "Algo/Copy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetTools)

#define LOCTEXT_NAMESPACE "AssetTools"

namespace UE::Private
{
	FText GetInvalidFactoryMessage()
	{
		return LOCTEXT("InvalidFactory", "Cannot create the new asset because the supplied factory does not support the supplied class.");
	} 
}

class FAssetDefinitionProxy : public FAssetTypeActions_Base
{
public:
	FAssetDefinitionProxy(UAssetDefinition* AssetDefinition)
		: AssetDefinitionPtr(AssetDefinition)
	{
	}
	
	virtual bool IsAssetDefinitionInDisguise() const override
	{
		return true;
	}

	virtual bool ShouldCallGetActions() const override { return false; }

	UAssetDefinition* GetAssetDefinition() const { return AssetDefinitionPtr.Get(); }
	
	virtual FString GetObjectDisplayName(UObject* Object) const override { return AssetDefinitionPtr.Get()->GetObjectDisplayNameText(Object).ToString(); }
	virtual FText GetName() const override { return AssetDefinitionPtr.Get()->GetAssetDisplayName(); }
	virtual UClass* GetSupportedClass() const override { return AssetDefinitionPtr.Get()->GetAssetClass().LoadSynchronous(); }
	virtual FColor GetTypeColor() const override { return AssetDefinitionPtr.Get()->GetAssetColor().ToFColor(true); }
	
	virtual bool CanLocalize() const override { return AssetDefinitionPtr.Get()->CanLocalize(FAssetData()).IsSupported(); }
	virtual bool IsImportedAsset() const override { return AssetDefinitionPtr.Get()->CanImport(); }
	virtual bool CanMerge() const override { return AssetDefinitionPtr.Get()->CanMerge(); }
	virtual bool ShouldFindEditorForAsset() const override { return AssetDefinitionPtr.Get()->ShouldFindEditorForAsset(); }

	virtual bool CanRename(const FAssetData& InAsset, FText* OutErrorMsg) const override
    {
		FAssetSupportResponse Response = AssetDefinitionPtr.Get()->CanRename(InAsset);
		if (OutErrorMsg)
		{
			(*OutErrorMsg) = Response.GetErrorText();
		}
		return Response.IsSupported();
    }

    virtual bool CanDuplicate(const FAssetData& InAsset, FText* OutErrorMsg) const override
    {
    	FAssetSupportResponse Response = AssetDefinitionPtr.Get()->CanDuplicate(InAsset);
		if (OutErrorMsg)
		{
			(*OutErrorMsg) = Response.GetErrorText();
		}
        return Response.IsSupported();
    }
	
	virtual uint32 GetCategories() override
	{
		static IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		uint32 CategoryBits = 0;
		for (const FAssetCategoryPath& Category : AssetDefinitionPtr.Get()->GetAssetCategories())
		{
			const FName& CategoryName = Category.GetCategory();
			if (!ensureMsgf(CategoryName.IsValid(),	TEXT("Found an invalid category path for Asset Definition: %s"), *(AssetDefinitionPtr.Get()->GetName())))
			{
				return CategoryBits;
			}

			if (CategoryName == EAssetCategoryPaths::Basic.GetCategory())
			{
				CategoryBits |= EAssetTypeCategories::Basic;
			}
			else
			{
				if (CategoryName != EAssetCategoryPaths::Misc.GetCategory())
				{
					const EAssetTypeCategories::Type AdvancedCategoryBit = AssetTools.FindAdvancedAssetCategory(CategoryName);
					if (AdvancedCategoryBit == EAssetTypeCategories::Misc)
					{
						CategoryBits |= AssetTools.RegisterAdvancedAssetCategory(CategoryName, Category.GetCategoryText());
					}
					else
					{
						CategoryBits |= AdvancedCategoryBit;
					}
				}
				else
				{
					CategoryBits |= EAssetTypeCategories::Misc;
				} 
			}
		}

		return CategoryBits;
	}

private:
	mutable TArray<FText> SubMenus;
	mutable bool SubmenusInitialized = false;

public:
	/** Returns array of sub-menu names that this asset type is parented under in the Asset Creation Context Menu. */
	virtual const TArray<FText>& GetSubMenus() const override
	{
		static IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		if (!SubmenusInitialized)
		{
			SubmenusInitialized = true;
			
			for (const FAssetCategoryPath& Category : AssetDefinitionPtr.Get()->GetAssetCategories())
			{
				Category.GetSubCategoriesText(SubMenus);
			}
		}
    
		return SubMenus;
	}

	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor) override
	{
		OpenAssetEditor(InObjects, EAssetTypeActivationOpenedMethod::Edit, EditWithinLevelEditor);
	}
	
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, const EAssetTypeActivationOpenedMethod OpenedMethod, TSharedPtr<IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override
	{
		TArray<FAssetData> Assets;
		Algo::Transform(InObjects, Assets, [](UObject* Object){ return FAssetData(Object); });
		
		FAssetOpenArgs OpenArgs;
		OpenArgs.OpenMethod = OpenedMethod == EAssetTypeActivationOpenedMethod::Edit ? EAssetOpenMethod::Edit : EAssetOpenMethod::View;
		OpenArgs.ToolkitHost = EditWithinLevelEditor;
		OpenArgs.Assets = Assets;
		AssetDefinitionPtr.Get()->OpenAssets(OpenArgs);		
	}

	virtual TArray<FAssetData> GetValidAssetsForPreviewOrEdit(TArrayView<const FAssetData> InAssetDatas, bool bIsPreview) override
	{
		FAssetActivateArgs ActivateArgs;
		ActivateArgs.ActivationMethod = bIsPreview ? EAssetActivationMethod::Previewed : EAssetActivationMethod::Opened;
		ActivateArgs.Assets = InAssetDatas;

		return AssetDefinitionPtr.Get()->PrepareToActivateAssets(ActivateArgs);
	}
	
	virtual bool AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType) override
	{
		EAssetActivationMethod ActivationMethod = EAssetActivationMethod::Opened;
		switch(ActivationType)
		{
		case EAssetTypeActivationMethod::DoubleClicked:
			ActivationMethod = EAssetActivationMethod::DoubleClicked;
			break;
		case EAssetTypeActivationMethod::Opened:
			ActivationMethod = EAssetActivationMethod::Opened;
			break;
		case EAssetTypeActivationMethod::Previewed:
			ActivationMethod = EAssetActivationMethod::Previewed;
			break;
		}
		
		TArray<FAssetData> Assets;
		Algo::Transform(InObjects, Assets, [](UObject* Object){ return FAssetData(Object);});
		
		FAssetActivateArgs Args;
		Args.ActivationMethod = ActivationMethod;
		Args.Assets = Assets;
		return AssetDefinitionPtr->ActivateAssets(Args) == EAssetCommandResult::Handled;
	}

	virtual bool CanFilter() override
	{
		const TSharedRef<FAssetFilterDataCache> FilterCache = AssetDefinitionPtr->GetFilters();
		return FilterCache->Filters.Num() > 0;
	}

	virtual FName GetFilterName() const override
	{
		const TSharedRef<FAssetFilterDataCache> FilterCache = AssetDefinitionPtr->GetFilters();
		return FilterCache->Filters.Num() > 0 ? FName(*FilterCache->Filters[0].Name) : NAME_None;
	}

	virtual void BuildBackendFilter(FARFilter& InFilter) override
	{
		const TSharedRef<FAssetFilterDataCache> FilterCache = AssetDefinitionPtr->GetFilters();
		if (FilterCache->Filters.Num() > 0)
		{
			InFilter = FilterCache->Filters[0].Filter;
		}
	}

	virtual FText GetDisplayNameFromAssetData(const FAssetData& AssetData) const override
	{
		return AssetDefinitionPtr->GetAssetDisplayName(AssetData);
	}

	virtual UThumbnailInfo* GetThumbnailInfo(UObject* Object) const override
	{
		return AssetDefinitionPtr->LoadThumbnailInfo(FAssetData(Object));
	}

	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override
	{
		return AssetDefinitionPtr->GetThumbnailBrush(InAssetData, InClassName);
	}
	
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override
	{
		return AssetDefinitionPtr->GetIconBrush(InAssetData, InClassName);
	}

	virtual FTopLevelAssetPath GetClassPathName() const override
	{
		return AssetDefinitionPtr.Get()->GetAssetClass().ToSoftObjectPath().GetAssetPath();
	}

	/** Begins a merge operation for InObject (automatically determines remote/base versions needed to resolve) */
	virtual void Merge( UObject* InObject ) override
	{
		FAssetAutomaticMergeArgs MergeArgs;
		MergeArgs.LocalAsset = InObject;

		AssetDefinitionPtr.Get()->Merge(MergeArgs);
	}

	/** Begins a merge between the specified assets */
	virtual void Merge(UObject* BaseAsset, UObject* RemoteAsset, UObject* LocalAsset, const FOnMergeResolved& ResolutionCallback) override
	{
		FAssetManualMergeArgs MergeArgs;
		MergeArgs.LocalAsset = LocalAsset;
		MergeArgs.BaseAsset = BaseAsset;
		MergeArgs.RemoteAsset = RemoteAsset;
		MergeArgs.ResolutionCallback = FOnAssetMergeResolved::CreateLambda([ResolutionCallback](const FAssetMergeResults& Results)
		{
			ResolutionCallback.Execute(Results.MergedPackage, static_cast<EMergeResult::Type>(Results.Result));
		});

		AssetDefinitionPtr.Get()->Merge(MergeArgs);
	}
	
	virtual void PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override
	{
		FAssetDiffArgs DiffArgs;
		DiffArgs.OldAsset = OldAsset;
		DiffArgs.NewAsset = NewAsset;
		DiffArgs.OldRevision = OldRevision;
		DiffArgs.NewRevision = NewRevision;
		
		AssetDefinitionPtr.Get()->PerformAssetDiff(DiffArgs);
	}

	/** Returns additional tooltip information for the specified asset, if it has any (otherwise return the null widget) */
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override
	{
		return AssetDefinitionPtr.Get()->GetAssetDescription(AssetData);
	}

	/** Collects the resolved source paths for the imported assets */
	virtual void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const override
	{
		ensureMsgf(false, TEXT("This code path is not expected to be called any more.  Expected UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(YourAsset)->GetSourceFiles to be called directly."));
	}
	
	/** Collects the source file labels for the imported assets */
	virtual void GetSourceFileLabels(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFileLabels) const override
	{
		ensureMsgf(false, TEXT("This code path is not expected to be called any more.  Expected UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(YourAsset)->GetSourceFiles to be called directly."));
	}

	/** Does this asset support edit or view methods? */
	virtual bool SupportsOpenedMethod(const EAssetTypeActivationOpenedMethod OpenedMethod) const override
	{
		FAssetOpenSupportArgs SupportArgs;
		SupportArgs.OpenMethod = OpenedMethod == EAssetTypeActivationOpenedMethod::Edit ? EAssetOpenMethod::Edit : EAssetOpenMethod::View;
		return AssetDefinitionPtr.Get()->GetAssetOpenSupport(SupportArgs).IsSupported;
	}
	
	/** Optionally returns a custom widget to overlay on top of this assets' thumbnail */
    virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override
    {
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
    	return AssetDefinitionPtr.Get()->GetThumbnailOverlay(AssetData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
    }
	
	virtual EThumbnailPrimType GetDefaultThumbnailPrimitiveType(UObject* Asset) const override
	{
		ensureMsgf(false, TEXT("This code path is not expected to be called any more.  Expected UAssetDefinitionRegistry::Get()->LoadThumbnail(YourAsset).  The default thumbnail primitive is now a property on USceneThumbnailInfoWithPrimitive.  The default only applies to this thumbnail type, so it has been relocated as an implementation detail of it."));
		return EThumbnailPrimType::TPT_Sphere;
	}

	/** @return True if we should force world-centric mode for newly-opened assets */
	virtual bool ShouldForceWorldCentric() override
	{
		FAssetOpenSupportArgs SupportArgs;
        SupportArgs.OpenMethod = EAssetOpenMethod::Edit;
        if (AssetDefinitionPtr.Get()->GetAssetOpenSupport(SupportArgs).RequiredToolkitMode.IsSet())
        {
        	return AssetDefinitionPtr.Get()->GetAssetOpenSupport(SupportArgs).RequiredToolkitMode.GetValue() == EToolkitMode::WorldCentric; 
        }

		return false;
	}
	
private:

	TWeakObjectPtr<UAssetDefinition> AssetDefinitionPtr;
};

//----------------------------------------------------------------------------------------------------------------------

namespace UE::AssetTools::Private
{
	bool bUseNewPackageMigration = true;
	static FAutoConsoleVariableRef CVarUseNewPackageMigration(
		TEXT("AssetTools.UseNewPackageMigration"),
		bUseNewPackageMigration,
		TEXT("When set, The package migration will use the new implementation made for 5.1.")
	);

	bool bFollowRedirectorsWhenImporting = false;
	static FAutoConsoleVariableRef CVarFollowRedirectorsWhenImporting(
		TEXT("AssetTools.FollowRedirectorsWhenImporting"),
		bFollowRedirectorsWhenImporting,
		TEXT("When set, if you import an asset at a location with a redirector, you'll instead import to the redirector's destination location")
	);

	FAutoConsoleCommand LogFolderPermissionsCommand = FAutoConsoleCommand(
		TEXT("AssetTools.LogFolderPermissions"),
		TEXT("Logs the read and write permissions for folders"),
		FConsoleCommandWithArgsDelegate::CreateLambda(
			[](const TArray<FString>& Args)
			{
				UE_LOGF(LogAssetTools, Log, "Folder Read Permissions:\n%ls", *IAssetTools::Get().GetFolderPermissionList()->ToString());
				UE_LOGF(LogAssetTools, Log, "Folder Write Permissions:\n%ls", *IAssetTools::Get().GetWritableFolderPermissionList()->ToString());
			}));

	static bool bEnablePublicAssetFeature = false;
	static FAutoConsoleVariableRef CVarEnablePublicAssetFeature(
		TEXT("AssetTools.EnablePublicAssetFeature"),
		bEnablePublicAssetFeature,
		TEXT("Enables the Experimental Public Asset Feature (False: disabled, True:enabled")
	);

	static bool bEnableEpicInternalAssetFeature = false;
	static FAutoConsoleVariableRef CVarEnableEpicInternalAssetFeature(
		TEXT("AssetTools.EnableEpicInternalAssetFeature"),
		bEnableEpicInternalAssetFeature,
		TEXT("Enables the Epic Internal Asset Feature UI (False: disabled, True: enabled")
	);

	/** 
	 * CVar to specify if we should use Header patching in advanced copy.
	 * Default is false.
	 */
	bool bEnableHeaderPatching = false;
	FAutoConsoleVariableRef CVarEnableHeaderPatching(
		TEXT("AssetTools.UseHeaderPatchingAdvancedCopy"),
		bEnableHeaderPatching,
		TEXT("If set to true, this will use Header Patching to copy the files instead of performing a full load."),
		ECVF_Default
	);

	// use a struct as a namespace to allow easier friend declarations
	struct FPackageMigrationImpl
	{
	public:
		struct FPackageMigrationImplContext
		{
			FPackageMigrationImplContext()
				: SlowTask(100, LOCTEXT("MigratePackages", "Migrating Packages"))
			{
				InstancingContext.SetSoftObjectPathRemappingEnabled(true);
				InstancingContext.AddTag(ULevel::DontLoadExternalObjectsTag);

				const bool bCanCancel = true;
				SlowTask.MakeDialog(bCanCancel);
			}

			bool bWasCanceled = false;
			FLinkerInstancingContext InstancingContext;
			FScopedSlowTask SlowTask;
			FMigrationOptions Options;
		};


		struct FTemporaryWritableFolderPermission
		{
			FTemporaryWritableFolderPermission(const TSharedRef<FPathPermissionList>& InWritableFolder)
				: WritableFolder(InWritableFolder)
			{
			}

			~FTemporaryWritableFolderPermission()
			{
				for (const FName Owner : DenyList)
				{
					WritableFolder->AddDenyListItem(Owner, TempWritableRootPath);
				}

				WritableFolder->UnregisterOwner(TemporalyPermittedByMigration);
			}

			void AddTemporaryWriteFolderPermission(const FStringView InWritableRootPath)
			{
				TempWritableRootPath = InWritableRootPath;

				if (WritableFolder->ContainsDenyListItem(TempWritableRootPath))
				{
					DenyList = WritableFolder->RemoveDenyListItemAndGetOwners(TempWritableRootPath);
				}

				// No need to add a permission if the list is empty since everything is permitted (except the deny list)
				if (WritableFolder->HasAllowListEntries())
				{
					WritableFolder->AddAllowListItem(TemporalyPermittedByMigration, TempWritableRootPath);
				}
			}

		private:
			TSharedRef<FPathPermissionList> WritableFolder;
			FString TempWritableRootPath;
			FPermissionListOwners DenyList;

			static const inline TCHAR* TemporalyPermittedByMigration = TEXT("TemporalyPermittedByMigration");
		};

		static FString GetMountPointRootPath(const FString& DestinationFolder)
		{
			FString MountPointRootPath;

			TArray<FString> Files;
			IFileManager::Get().FindFiles(Files, *(DestinationFolder + TEXT("../")), TEXT("uproject"));
			if (!Files.IsEmpty())
			{
				MountPointRootPath = TEXT("/Game/");
			}
			else
			{
				IFileManager::Get().FindFiles(Files, *(DestinationFolder + TEXT("../")), TEXT("uplugin"));

				if (Files.Num() == 1)
				{
					FStringView PluginNameView = FPathViews::GetBaseFilename(Files[0]);
					MountPointRootPath.Reserve(PluginNameView.Len() + 2);
					MountPointRootPath.AppendChar(TEXT('/'));
					MountPointRootPath.Append(PluginNameView);
					MountPointRootPath.AppendChar(TEXT('/'));
				}
				else
				{
					UE_LOGF(LogAssetTools
						, Error
						, "{%ls} does not appear to be a game Content folder. Migrated content only work if placed in a Content folder. Aborting the Migration."
						, *DestinationFolder
					);
				}
			}

			return MountPointRootPath;
		}

		static UObject* FindAssetInPackage(UPackage* Package)
		{
			// Inspired from the save code (UEditorEngine::Save)
			UObject* Asset = nullptr;
			if (Package->HasAnyPackageFlags(PKG_ContainsMap))
			{
				Asset = UWorld::FindWorldInPackage(Package);
			}
			
			if (!Asset)
			{
				// Otherwise find the main asset of the package
				Asset = Package->FindAssetInPackage();
			}

			return Asset;
		};

		static bool CanSavePackageToFile(const FString& DestinationFile, const FStringView PackageName, FPackageMigrationContext* OptionalPackageMigrationContext)
		{
			// Don't try to migrate the package if the destination is read only
			if (IFileManager::Get().IsReadOnly(*DestinationFile))
			{
				if (OptionalPackageMigrationContext)
				{
					OptionalPackageMigrationContext->AddErrorMigrationMessage(FText::Format(LOCTEXT("MigratePackages_SaveFailedReadOnly", "Couldn't migrate package ({0}) because the destination file is read only. Destination File ({1})")
						, FText::FromStringView(PackageName)
						, FText::FromString(DestinationFile)
						));
				}

				return false;
			}

			return true;
		}

		// Prepare the MigrationPackagesData in the migration context and log any info about the package that will be renamed
		static void SetupPublicAssetPackagesMigrationData(const TSharedPtr<TArray<ReportPackageData>>& PackageDataToMigrate
			, TArray<TPair<const ReportPackageData*, const FAssetData>>& ExternalPackageDatas
			, TMap<const FStringView, int32>& ExistingPackageNameToMigrationDataIndex
			, FPackageMigrationContext& PackageMigrationContext
			, FPackageMigrationImplContext& PackageMigrationImplContext
			, FMessageLog& MigrateLog)
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
			TArray<FAssetData> AssetsData;

			// Used to avoid a name collision between the instanced packages
			TSet<FString> UsedPackagesName;
			UsedPackagesName.Reserve(PackageDataToMigrate->Num());
			IFileManager& FileManager = IFileManager::Get();


			ExternalPackageDatas.Reserve(PackageDataToMigrate->Num());
			ExistingPackageNameToMigrationDataIndex.Reserve(PackageDataToMigrate->Num());

			TArray<FText> RenamedPackagesInfo;


			// Handle prompt response and automation
			EAppReturnType::Type LastResponse = EAppReturnType::Yes;
			if (!PackageMigrationImplContext.Options.bPrompt)
			{
				switch (PackageMigrationImplContext.Options.AssetConflict)
				{
				case EAssetMigrationConflict::Overwrite:
					LastResponse = EAppReturnType::YesAll;
					break;
				case EAssetMigrationConflict::Skip:
					LastResponse = EAppReturnType::NoAll;
					break;
				case EAssetMigrationConflict::Cancel:
					PackageMigrationImplContext.bWasCanceled = true;
					break;
				}
			}

			// Process the normal package here (no external object or actors).
			for (const ReportPackageData& PackageData : *(PackageDataToMigrate.Get()))
			{
				if (PackageMigrationImplContext.SlowTask.ShouldCancel() || PackageMigrationImplContext.bWasCanceled)
				{
					PackageMigrationImplContext.bWasCanceled = true;
					break;
				}

				if (!PackageData.bShouldMigratePackage)
				{
					continue;
				}

				AssetsData.Reset();
				const bool bIncludeOndiskAssetOnly = true;
				// We want all the asset including those not supported for the current platform.
				const bool bSkipARFilteredAssets = false;
				AssetRegistry.GetAssetsByPackageName(*PackageData.Name, AssetsData, bIncludeOndiskAssetOnly, bSkipARFilteredAssets);

				// Detect if the package contains some external object or actors
				{
					bool bIsExternalPackage = false;
					for (FAssetData& AssetData : AssetsData)
					{
						if (!AssetData.GetOptionalOuterPathName().IsNone())
						{
							bIsExternalPackage = true;
							ExternalPackageDatas.Emplace(&PackageData, MoveTemp(AssetData));
							break;
						}
					}

					if (bIsExternalPackage)
					{
						// Skip this asset since it is a external package. They are done in another pass
						continue;
					}
				}


				const FPackageMigrationContext::FScopedMountPoint& MountPoint = PackageMigrationContext.GetDestinationMountPoint();

				FString NewPackageName = MountPoint.GetNewPackageNameForMigration(PackageData.Name);

				// Is there a simpler way to get the package extension?
				FString ExistingFilename;
				FPackageName::DoesPackageExist(PackageData.Name, &ExistingFilename);

				const bool bIncludeDot = true;
				FString NewPackageFilename = MountPoint.GetMigratedPackageFilename(NewPackageName, FPathViews::GetExtension(ExistingFilename, bIncludeDot));
				uint32 NewPackageNameHash = GetTypeHash(NewPackageName);

				// Check we can use the new name for the asset
				const bool bIsNameChangeCausedByOtherMigrationPackage = UsedPackagesName.ContainsByHash(NewPackageNameHash, NewPackageName);
				const bool bIsNameChangeCausedByExistingAssetInDestination = FileManager.FileExists(*NewPackageFilename);

				bool bShouldMigrate = true;

				// Ask the user what to do if an asset already exist in the destination
				if (bIsNameChangeCausedByExistingAssetInDestination)
				{
					if (!CanSavePackageToFile(NewPackageFilename, PackageData.Name, &PackageMigrationContext))
					{
						bShouldMigrate = false;
					}
					else
					{ 
						// Handle name collision in the destination project. Post 5.1 should expose the possibility to rename the migrated package
						EAppReturnType::Type Response;
						if (!UAssetToolsImpl::CanShowDialogs() || !PackageMigrationImplContext.Options.bPrompt || LastResponse == EAppReturnType::YesAll || LastResponse == EAppReturnType::NoAll)
						{
							Response = LastResponse;
						}
						else
						{
							const FText Message = FText::Format(LOCTEXT("MigratePackages_AlreadyExists", "An asset already exists at location {0} would you like to overwrite it?"), FText::FromString(NewPackageFilename));
							Response = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAllCancel, Message);
							LastResponse = Response;
						}

						if (Response == EAppReturnType::Cancel)
						{
							// The user chose to cancel mid-operation. Break out.
							PackageMigrationImplContext.bWasCanceled = true;
							break;
						}

						if (Response == EAppReturnType::No || Response == EAppReturnType::NoAll)
						{
							bShouldMigrate = false;
						}
					}
				}

				// Avoid a name collision
				if (bIsNameChangeCausedByOtherMigrationPackage)
				{
					// Name collision with another package to migrate (simply add _number to package name)
					int32 Index = 1;
					FString ModifiedNewPackageName;
					// 4 for "_num"
					ModifiedNewPackageName.Reserve(NewPackageName.Len() + 3);
					uint32 ModifiedNewPackageNameHash;

					FString ModifiedNewPackageFilename;
					ModifiedNewPackageFilename.Reserve(NewPackageFilename.Len() + 3);
					FStringView FileExtension = FPathViews::GetExtension(NewPackageFilename, bIncludeDot);
					FStringView NewPackageFilenameWithoutExtension(*NewPackageFilename, NewPackageFilename.Len() - FileExtension.Len());

					do
					{
						++Index;
						ModifiedNewPackageName.Reset();
						ModifiedNewPackageName.Append(NewPackageName);
						ModifiedNewPackageName.AppendChar('_');
						ModifiedNewPackageName.AppendInt(Index);
						ModifiedNewPackageNameHash = GetTypeHash(ModifiedNewPackageName);

						ModifiedNewPackageFilename.Reset();
						ModifiedNewPackageFilename.Append(NewPackageFilenameWithoutExtension);
						ModifiedNewPackageFilename.AppendChar('_');
						ModifiedNewPackageFilename.AppendInt(Index);
						ModifiedNewPackageFilename.Append(FileExtension);
					} while (UsedPackagesName.ContainsByHash(ModifiedNewPackageNameHash, ModifiedNewPackageName)
						|| FileManager.FileExists(*ModifiedNewPackageFilename)
						);

					if (bIsNameChangeCausedByOtherMigrationPackage)
					{
						RenamedPackagesInfo.Add(
							FText::Format(LOCTEXT("MigratePackages_PackageNameChangeByOtherPackage", "({0}) will be named ({1}) because another migrated package is already using that name.")
								, FText::FromString(PackageData.Name)
								, FText::FromString(ModifiedNewPackageName)
							)
						);
					}
					else if (bIsNameChangeCausedByExistingAssetInDestination)
					{
						RenamedPackagesInfo.Add(
							FText::Format(LOCTEXT("MigratePackages_PackageNameChangeByExistingPackageInDestination", "({0}) will be named ({1}) because an file in the destination project is already using that name.")
								, FText::FromString(PackageData.Name)
								, FText::FromString(ModifiedNewPackageName)
							)
						);
					}

					NewPackageNameHash = ModifiedNewPackageNameHash;
					NewPackageName = MoveTemp(ModifiedNewPackageName);
					NewPackageFilename = MoveTemp(ModifiedNewPackageFilename);
				}

				UsedPackagesName.AddByHash(NewPackageNameHash, NewPackageName);

				int32 MigrationPackageDataIndex = PackageMigrationContext.MigrationPackagesData.Emplace(NewPackageName, PackageData.Name, NewPackageFilename);
				FPackageMigrationContext::FMigrationPackageData& MigrationData = PackageMigrationContext.MigrationPackagesData[MigrationPackageDataIndex];

				// Post 5.1 look into detecting which file can be migrated without being loaded
				MigrationData.bNeedInstancedLoad = bShouldMigrate;
				MigrationData.bNeedToBeSaveMigrated = bShouldMigrate;

				ExistingPackageNameToMigrationDataIndex.Add(PackageData.Name, MigrationPackageDataIndex);

				// Check if there is a existing package that need to be moved out the way
				if (UPackage* InTheWayPackage = FindObjectFast<UPackage>(nullptr, *NewPackageName))
				{
					PackageMigrationContext.MoveInTheWayPackage(InTheWayPackage);
				}

				PackageMigrationImplContext.InstancingContext.AddPackageMapping(*PackageData.Name, *NewPackageName);
			}

			if (!RenamedPackagesInfo.IsEmpty())
			{
				MigrateLog.NewPage(LOCTEXT("MigratePackages_PackageNameChange", "Modified Packages Name For Migration"));
				for (const FText& Message : RenamedPackagesInfo)
				{
					MigrateLog.Info(Message);
				}
			}
		}

		static void SetupExternalAssetPackagesMigrationData(const TArray<TPair<const ReportPackageData*, const FAssetData>>& ExternalPackageDatas, TMap<const FStringView, int32>& ExistingPackageNameToMigrationDataIndex, FPackageMigrationContext& PackageMigrationContext, FPackageMigrationImplContext& MigrationImplContext)
		{
			auto GenerateObjectPath = [](const FStringView PackageName, const FStringView AssetName, const FStringView SubPath) -> FString
			{
				FString InstancedPath;

				int32 DelimitersCount = 2;
				if (SubPath.IsEmpty())
				{
					--DelimitersCount;
				}

				InstancedPath.Reserve(PackageName.Len() + AssetName.Len() + SubPath.Len() + DelimitersCount);
				InstancedPath.Append(PackageName);
				InstancedPath.AppendChar(TEXT('.'));
				InstancedPath.Append(AssetName);

				if (!SubPath.IsEmpty())
				{
					InstancedPath.AppendChar(SUBOBJECT_DELIMITER_CHAR);
					InstancedPath.Append(SubPath);
				}

				return InstancedPath;
			};

			// Handle the external packages
			for (const TPair<const ReportPackageData*, const FAssetData>& PackageData : ExternalPackageDatas)
			{
				const FString OuterPath = PackageData.Value.GetOptionalOuterPathName().ToString();

				const FStringView OuterPackageName = FPathViews::GetBaseFilenameWithPath(OuterPath);

				const int32* PtrToMigrationDataIndex = ExistingPackageNameToMigrationDataIndex.Find(OuterPackageName);

				if (!PtrToMigrationDataIndex)
				{
					PackageMigrationContext.AddErrorMigrationMessage(FText::Format(
						LOCTEXT("MigratePackages_ExternalPackageNotExported", "({0}) won't be migrated because the outer of its content ({1}) will not be migrated")
						, FText::FromString(PackageData.Key->Name)
						, FText::FromStringView(OuterPackageName)
					)
					);
					continue;
				}

				FPackageMigrationContext::FMigrationPackageData& OuterMigrationData = PackageMigrationContext.MigrationPackagesData[*PtrToMigrationDataIndex];

				const FSoftObjectPath ExistingObjectPath = PackageData.Value.GetSoftObjectPath();
				FString NewObjectPath = GenerateObjectPath(OuterMigrationData.GetInstancedPackageName(), FPathViews::GetBaseFilename(OuterMigrationData.GetInstancedPackageName()), ExistingObjectPath.GetSubPathString());

				FString NewPackageName;

				if (FWorldPartitionActorDescUtils::IsValidActorDescriptorFromAssetData(PackageData.Value))
				{
					// If the external package is an actor.
					EActorPackagingScheme Scheme = ULevel::GetActorPackagingSchemeFromActorPackageName(PackageData.Key->Name);
					NewPackageName = ULevel::GetActorPackageName(ULevel::GetExternalActorsPath(OuterMigrationData.GetInstancedPackageName()), Scheme, NewObjectPath);
				}
				else
				{
					NewPackageName = FExternalPackageHelper::GetExternalPackageName(OuterMigrationData.GetInstancedPackageName(), NewObjectPath);
				}


				FString NewPackageFilename = PackageMigrationContext.GetDestinationMountPoint().GetMigratedPackageFilename(NewPackageName, FPackageName::GetAssetPackageExtension());

				// Add the external package to the list of migrated asset, even if the asset is not really migrated we still need it for the instance path
				uint32 MigrationPackageDataIndex = PackageMigrationContext.MigrationPackagesData.Emplace(NewPackageName, PackageData.Key->Name, NewPackageFilename);

				FPackageMigrationContext::FMigrationPackageData& MigrationData = PackageMigrationContext.MigrationPackagesData[MigrationPackageDataIndex];

				// Post 5.1 look into detecting which file can be migrated without being loaded
				MigrationData.bNeedInstancedLoad = OuterMigrationData.bNeedInstancedLoad && PackageData.Key->bShouldMigratePackage;
				MigrationData.bNeedToBeSaveMigrated = OuterMigrationData.bNeedToBeSaveMigrated && PackageData.Key->bShouldMigratePackage;

				// Check if there is a existing package that need to be moved out the way
				if (UPackage* InTheWayPackage = FindObjectFast<UPackage>(nullptr, *NewPackageName))
				{
					PackageMigrationContext.MoveInTheWayPackage(InTheWayPackage);
				}

				MigrationImplContext.InstancingContext.AddPackageMapping(*(PackageData.Key->Name), *NewPackageName);
			}
		}

		static void ProcessExcludedDependencies(const TSet<FName>& ExcludedDependencies, UE::AssetTools::FPackageMigrationContext& PackageMigrationContext)
		{
			// Clean the excluded dependencies that are not used by the migrated packages
			if (!ExcludedDependencies.IsEmpty())
			{
				TSet<FName> PackageToMigrate;

				const TArray<UE::AssetTools::FPackageMigrationContext::FMigrationPackageData>& MigrationPackages = PackageMigrationContext.GetMigrationPackagesData();
				PackageToMigrate.Reserve(MigrationPackages.Num());

				for (const UE::AssetTools::FPackageMigrationContext::FMigrationPackageData& PackageData : MigrationPackages)
				{
					if (PackageData.bNeedToBeSaveMigrated)
					{
						PackageToMigrate.Emplace(*PackageData.OriginalPackageName);
					}
				}

				PackageMigrationContext.ExcludedDependencies.Reserve(ExcludedDependencies.Num());

				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
				IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
				TArray<FName> Referencers;

				for (const FName ExcludedDependency : ExcludedDependencies)
				{
					AssetRegistry.GetReferencers(ExcludedDependency, Referencers);

					for (const FName Reference : Referencers)
					{
						if (PackageToMigrate.Contains(Reference))
						{
							PackageMigrationContext.ExcludedDependencies.Add(ExcludedDependency.ToString());
							break;
						}
					}

					Referencers.Reset();
				}
			}
		}

		static void CreatePackagesAndSetupLinkers(FPackageMigrationContext& PackageMigrationContext, FPackageMigrationImplContext& MigrationImplContext, TArray<FName>& PackagesToClean)
		{
			FArchive* ReaderOverride = nullptr;

			// Create the package and their load linker. If an asset try to load a weak dependency the package will load from the right file because of the existing linker.
			for (FPackageMigrationContext::FMigrationPackageData& MigrationPackageData : PackageMigrationContext.MigrationPackagesData)
			{
				if (MigrationImplContext.SlowTask.ShouldCancel() || MigrationImplContext.bWasCanceled)
				{
					MigrationImplContext.bWasCanceled = true;
					PackagesToClean.Add(FName(*MigrationPackageData.GetInstancedPackageName()));
				}
				else
				{
					UPackage* MigrationPackage = CreatePackage(*MigrationPackageData.GetInstancedPackageName());

					/**
					 * Load_Verify tell the linker to not load the package but it will create the linker if the file exist and is valid.
					 * LOAD_NoVerify tell the linker to not check the import of the package (in the editor this will avoid loading the hard dependencies of the package)
					 * 
					 * This will the package with a pre-created linker that has the right instancing context. Ready to loaded by another call to load package that may not have all that info.
					 */
					MigrationPackageData.InstancedPackage = LoadPackage(MigrationPackage, *MigrationPackageData.GetOriginalPackageName(), LOAD_Verify | LOAD_NoVerify, ReaderOverride, &MigrationImplContext.InstancingContext);

					// Add the PKG_NewlyCreated flag here to help the engine code that expect the packages without it to have a file associated with it on disk.
					MigrationPackage->SetPackageFlags(PKG_NewlyCreated);

					PackagesToClean.Add(MigrationPackage->GetFName());
				}
			}
		}

		static void LoadInstancedPackages(FPackageMigrationContext& PackageMigrationContext, FPackageMigrationImplContext& MigrationImplContext)
		{
			const int32 NumOfPackageToLoad = static_cast<int32>(Algo::CountIf(PackageMigrationContext.MigrationPackagesData, [](const FPackageMigrationContext::FMigrationPackageData& MigrationPackageData)
				{
					return MigrationPackageData.bNeedInstancedLoad || MigrationPackageData.bNeedToBeSaveMigrated;
				}));

			// We give 40% of the progress weight to the loading
			const float ProgressPerItemToload = 40.f / static_cast<float>(NumOfPackageToLoad);

			uint32 ProgressCount = 0;

			FArchive* ReaderOverride = nullptr;

			for (FPackageMigrationContext::FMigrationPackageData& MigrationPackageData : PackageMigrationContext.MigrationPackagesData)
			{
				if (MigrationImplContext.SlowTask.ShouldCancel() || MigrationImplContext.bWasCanceled)
				{
					MigrationImplContext.bWasCanceled = true;
					break;
				}

				if (MigrationPackageData.bNeedInstancedLoad || MigrationPackageData.bNeedToBeSaveMigrated)
				{
					MigrationImplContext.SlowTask.EnterProgressFrame(ProgressPerItemToload
						, FText::Format(LOCTEXT("MigratePackages_Load", "Migration: Loading packages {0}/{1}")
							, ProgressCount
							, NumOfPackageToLoad
						)
					);

					++ProgressCount;

					UPackage* LoadedPackage = LoadPackage(MigrationPackageData.GetInstancedPackage(), *MigrationPackageData.GetOriginalPackageName(), LOAD_None, ReaderOverride, &MigrationImplContext.InstancingContext);

					if (IsValid(LoadedPackage))
					{
						UObject* Asset = FPackageMigrationImpl::FindAssetInPackage(LoadedPackage);

						if (Asset && !Asset->HasAnyFlags(RF_Standalone))
						{
							// The external actor or object are sometimes not protected from the garbage collector. This is removed by the purge at the end of the migration
							Asset->AddToRoot();
						}
					}
					else
					{
						MigrationPackageData.bNeedToBeSaveMigrated = false;
						PackageMigrationContext.AddErrorMigrationMessage(FText::Format(LOCTEXT("MigratePackages_LoadFailed", "Couldn't migrate package ({0}) because the load of the asset failed"), FText::FromString(MigrationPackageData.GetOriginalPackageName())));
					}
				}
			}
		}

		static void SaveInstancedPackagesIntoDestination(FPackageMigrationContext& PackageMigrationContext, FPackageMigrationImplContext& MigrationImplContext)
		{
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;

			// Rehydrate the payloads so that we move the virtual payloads also. Add the auto save flag to notify the editor extensions that they shouldn't prompt the user or generate/update some asset on save.
			SaveArgs.SaveFlags |= SAVE_RehydratePayloads | SAVE_FromAutosave;

			// We should look into creating our own log to report the save errors to the user.
			SaveArgs.Error = GWarn;


			uint32 ProgressCount = 0;

			const int32 NumOfPackageToSave = static_cast<int32>(Algo::CountIf(PackageMigrationContext.MigrationPackagesData, [](const FPackageMigrationContext::FMigrationPackageData& MigrationPackageData)
				{
					return MigrationPackageData.bNeedToBeSaveMigrated;
				}));

			// We give 60% of the progress weight to the saving of the assets
			const float ProgressPerItemToSave = 60.f / static_cast<float>(NumOfPackageToSave);

			for (const FPackageMigrationContext::FMigrationPackageData& MigrationPackageData : PackageMigrationContext.MigrationPackagesData)
			{
				if (MigrationImplContext.SlowTask.ShouldCancel() || MigrationImplContext.bWasCanceled)
				{
					MigrationImplContext.bWasCanceled = true;
					break;
				}
				if (MigrationPackageData.bNeedToBeSaveMigrated)
				{

					MigrationImplContext.SlowTask.EnterProgressFrame(ProgressPerItemToSave
						, FText::Format(LOCTEXT("MigratePackages_Save", "Saving the packages into the destination {0}/{1}")
							, ProgressCount
							, NumOfPackageToSave
						)
					);

					++ProgressCount;
					
					UObject* Asset = FPackageMigrationImpl::FindAssetInPackage(MigrationPackageData.GetInstancedPackage());
					if (IsValid(MigrationPackageData.GetInstancedPackage()) && Asset)
					{
						FSavePackageResultStruct SaveResult = GEditor->Save(MigrationPackageData.GetInstancedPackage(), Asset, *MigrationPackageData.GetDestinationFilename(), SaveArgs);

						if (SaveResult.IsSuccessful())
						{
							PackageMigrationContext.AddSucessfullMigrationMessage(FText::Format(LOCTEXT("MigratePackages_SaveSuccess", "Package ({0}) was migrated successfully as ({1}) with the following filename ({2})")\
								, FText::FromString(MigrationPackageData.GetOriginalPackageName())
								, FText::FromString(MigrationPackageData.GetInstancedPackageName())
								, FText::FromString(MigrationPackageData.GetDestinationFilename())
							));
						}
						else
						{
							PackageMigrationContext.AddErrorMigrationMessage(FText::Format(LOCTEXT("MigratePackages_SaveFailed", "Couldn't migrate package ({0}) because the asset save failed. Destination File ({1})")
								, FText::FromString(MigrationPackageData.GetOriginalPackageName())
								, FText::FromString(MigrationPackageData.GetDestinationFilename())
							));
						}
					}
					else if (!IsValid(MigrationPackageData.GetInstancedPackage()))
					{
						PackageMigrationContext.AddErrorMigrationMessage(FText::Format(LOCTEXT("MigratePackages_SaveFailedPackageInvalid", "Couldn't migrate package ({0}) because the asset package is invalid. Destination File ({1})")
							, FText::FromString(MigrationPackageData.GetOriginalPackageName())
							, FText::FromString(MigrationPackageData.GetDestinationFilename())
						));
					}
					else
					{
						PackageMigrationContext.AddErrorMigrationMessage(FText::Format(LOCTEXT("MigratePackages_SaveFailedNoAsset", "Couldn't migrate package ({0}) because the package didn't contain an asset. Destination File ({1})")
							, FText::FromString(MigrationPackageData.GetOriginalPackageName())
							, FText::FromString(MigrationPackageData.GetDestinationFilename())
						));
					}
				}
			}
		}

		static void CleanInstancedPackages(const TArray<FName>& PackagesToClean, FPackageMigrationContext& PackageMigrationContext)
		{
			for (FPackageMigrationContext::FMigrationPackageData& MigrationPackageData : PackageMigrationContext.MigrationPackagesData)
			{
				// Fixes some reference issues when the pending kill flag is off
				MigrationPackageData.InstancedPackage = nullptr;
			}

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

			TSharedRef<TFunction<void(UObject*)>> PurgeObject = MakeShared<TFunction<void(UObject*)>>();
			PurgeObject.Get() = [PurgeObject](UObject* Object)
			{
				if (IsValid(Object))
				{
					if (UPackage* ExternalPackage = Object->GetExternalPackage())
					{
						if (Object != ExternalPackage && IsValid(ExternalPackage))
						{
							(*PurgeObject)(ExternalPackage);
							ForEachObjectWithOuter(ExternalPackage, *PurgeObject);
						}
					}

					if (Object->IsRooted())
					{
						Object->RemoveFromRoot();
					}

					Object->ClearFlags(RF_Public | RF_Standalone);
					Object->MarkAsGarbage();
				}
			};

			// Turn off the components while unloading stuff
			FGlobalComponentReregisterContext ComponentContext;


			TArray<UPackage*> ExistingPackageToClean;
			ExistingPackageToClean.Reserve(PackagesToClean.Num());

			for (const FName PackageName : PackagesToClean)
			{
				if (UPackage* Package = FindObjectFast<UPackage>(nullptr, PackageName, EFindObjectFlags::ExactClass))
				{
					ExistingPackageToClean.Add(Package);
				}
			}

			// 1st Force to clean all the worlds so they can shutdown the subsystems properly
			for (const UPackage* Package : ExistingPackageToClean)
			{
				if (Package->ContainsMap())
				{
					ForEachObjectWithOuter(Package, [](UObject* Object)
						{
							if (UWorld* World = Cast<UWorld>(Object))
							{
								World->CleanupWorld();
							}
						});
				}
			}

			TArray<UObject*> ReferenceToNull;


			// We do the clean pass of the packages in two loop because the PurgeObject can affect the ability to get the main object from another package.
			for (UPackage* Package : ExistingPackageToClean)
			{
				if (UObject* Asset = FPackageMigrationImpl::FindAssetInPackage(Package))
				{
					AssetRegistry.AssetDeleted(Asset);
					ReferenceToNull.Add(Asset);
				}
			}

			for (UPackage* Package : ExistingPackageToClean)
			{
				const ERenameFlags PkgRenameFlags = REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_SkipGeneratedClasses;
				verify(Package->Rename(*MakeUniqueObjectName(nullptr, UPackage::StaticClass(), *FString::Printf(TEXT("%s_DEADFROMMIGRATION"), *Package->GetName())).ToString(), nullptr, PkgRenameFlags));
				(*PurgeObject)(Package);
				ForEachObjectWithOuter(Package, *PurgeObject);
			}

			// Removing assets from memory is complicated
			TArray<UObject*> UDStructToReplace;
			for (int32 Index = 0; Index < ReferenceToNull.Num(); )
			{
				if (UUserDefinedStruct* UDStruct = Cast<UUserDefinedStruct>(ReferenceToNull[Index]))
				{
					ReferenceToNull.RemoveAtSwap(Index);
					UDStructToReplace.Add(UDStruct);
				}
				else
				{
					Index++;
				}
			}

			if (UDStructToReplace.Num())
			{
				ObjectTools::ForceReplaceReferences(GetFallbackStruct(), UDStructToReplace);
			}

			ObjectTools::ForceReplaceReferences(nullptr, ReferenceToNull);

			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}

		static void RestoreInTheWayPackages(FPackageMigrationContext& PackageMigrationContext)
		{
			PackageMigrationContext.TemporalyMovedPackages.Empty();

			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}

		static void BuildAndNotifyLogWindow(FPackageMigrationContext& PackageMigrationContext, FMessageLog& MigrateLog, FPackageMigrationImplContext& MigrationImplContext)
		{
			FText NotificationMessage;

			if (!PackageMigrationContext.MigratedPackageMessages.IsEmpty())
			{
				MigrateLog.NewPage(LOCTEXT("MigratePackages_SuccessPageLabel", "Migrated Packages"));
				for (const FText& InfoMessage : PackageMigrationContext.MigratedPackageMessages)
				{
					MigrateLog.Info(InfoMessage);
				}
			}

			if (!PackageMigrationContext.WarningMessage.IsEmpty())
			{
				MigrateLog.NewPage(LOCTEXT("MigratePackages_WarningPageLabel", "Migration warnings"));
				for (const FText& WarningMessage : PackageMigrationContext.WarningMessage)
				{
					MigrateLog.Warning(WarningMessage);
				}
			}

			if (PackageMigrationContext.ErrorMessages.IsEmpty())
			{
				if (PackageMigrationContext.WarningMessage.IsEmpty())
				{
					NotificationMessage = LOCTEXT("MigratePackages_SuccessNotification", "Content migration completed.");
				}
				else
				{
					NotificationMessage = LOCTEXT("MigratePackages_SuccessNotificationWithWarning", "Content migration completed with some warnings.");
				}
			}
			else
			{
				if (PackageMigrationContext.MigratedPackageMessages.IsEmpty())
				{
					NotificationMessage = LOCTEXT("MigratePackages_FailureNotification", "Content migration failed.");
				}
				else
				{
					NotificationMessage = LOCTEXT("MigratePackages_PartialFailureNotification", "Content migration failed but some asset have been migrated sucessfuly.");
				}

				MigrateLog.NewPage(LOCTEXT("MigratePackages_ErrorsPageLabel", "Migration Errors"));
				for (const FText& ErrorMessage : PackageMigrationContext.ErrorMessages)
				{
					MigrateLog.Error(ErrorMessage);
				}
			}

			if (MigrationImplContext.bWasCanceled)
			{
				if (PackageMigrationContext.MigratedPackageMessages.IsEmpty())
				{
					NotificationMessage = LOCTEXT("MigratePackages_Canceled", "Content migration was canceled.");
				}
				else
				{
					NotificationMessage = LOCTEXT("MigratePackages_CanceledWithMigratedAssets", "Content migration was canceled but some package where already migrated.");
				}
			}

			MigrateLog.Notify(NotificationMessage, EMessageSeverity::Info);
		}
	};
}


IAssetTools& IAssetTools::Get()
{
	FAssetToolsModule& Module = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	return Module.Get();
}

TScriptInterface<IAssetTools> UAssetToolsHelpers::GetAssetTools()
{
	return &UAssetToolsImpl::Get();
}

/** UInterface constructor */
UAssetTools::UAssetTools(const class FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAssetToolsImpl::UAssetToolsImpl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AssetRenameManager(MakeShareable(new FAssetRenameManager))
	, LocalizedAssetTools(MakeShareable(new FLocalizedAssetTools))
	, AssetFixUpRedirectors(MakeShareable(new FAssetFixUpRedirectors))
	, NextUserCategoryBit(EAssetTypeCategories::FirstUser)
	, AssetClassPermissionList_DEPRECATED(MakeShared<FNamePermissionList>())
	, ImportExtensionPermissionList(MakeShared<FNamePermissionList>())
	, FolderPermissionList(MakeShared<FPathPermissionList>())
	, WritableFolderPermissionList(MakeShared<FPathPermissionList>())
	, CreateAssetsAsExternallyReferenceable(true)
{
	TArray<FString> SupportedTypesArray;
	GConfig->GetArray(TEXT("AssetTools"), TEXT("SupportedAssetTypes"), SupportedTypesArray, GEditorIni);
	for (int32 i = 0; i < (int32)EAssetClassAction::AllAssetActions; ++i)
	{
		AssetClassPermissionList.Add(MakeShared<FPathPermissionList>(EPathPermissionListType::ClassPaths));
		for (FString& Type : SupportedTypesArray)
		{
			if (FPackageName::IsShortPackageName(Type))
			{
				FTopLevelAssetPath TypePath = UClass::TryConvertShortTypeNameToPathName<UStruct>(Type, ELogVerbosity::Warning, TEXT("AssetToolsImpl"));
				if (TypePath.IsNull())
				{
					UE_LOGF(LogAssetTools, Warning, "Failed to convert short type name \"%ls\" to path name. Please update SupportedAssetTypes entries in [AssetTools] ini section", *Type);
				}
				else
				{
					Type = TypePath.ToString();
				}
			}
			AssetClassPermissionList[i]->AddAllowListItem("AssetToolsConfigFile", Type);
		}
	}

	TArray<FString> SupportedImportExtensionArray;
	GConfig->GetArray(TEXT("AssetTools"), TEXT("SupportedImportExtensions"), SupportedImportExtensionArray, GEditorIni);
	for (FString& Extension : SupportedImportExtensionArray)
	{
		ImportExtensionPermissionList->AddAllowListItem("AssetToolsConfigFile", FName(*Extension));
	}


	TArray<FString> DenyListedViewPath;
	GConfig->GetArray(TEXT("AssetTools"), TEXT("DenyListAssetPaths"), DenyListedViewPath, GEditorIni);
	for (const FString& Path : DenyListedViewPath)
	{
		FolderPermissionList->AddDenyListItem("AssetToolsConfigFile", Path);
	}

	GConfig->GetArray(TEXT("AssetTools"), TEXT("DenyListContentSubPaths"), SubContentDenyListPaths, GEditorIni);
	TArray<FString> ContentRoots;
	FPackageName::QueryRootContentPaths(ContentRoots);
	for (const FString& ContentRoot : ContentRoots)
	{
		AddSubContentDenyList(ContentRoot);
	}
	FPackageName::OnContentPathMounted().AddUObject(this, &UAssetToolsImpl::OnContentPathMounted);

	// Register the built-in advanced categories
	AllocatedCategoryBits.Add(TEXT("Animation"), FAdvancedAssetCategory(EAssetTypeCategories::Animation, LOCTEXT("AnimationAssetCategory", "Animation")));
	AllocatedCategoryBits.Add(TEXT("Blueprint"), FAdvancedAssetCategory(EAssetTypeCategories::Blueprint, LOCTEXT("BlueprintAssetCategory", "Blueprint")));
	AllocatedCategoryBits.Add(TEXT("Material"), FAdvancedAssetCategory(EAssetTypeCategories::Materials, LOCTEXT("MaterialAssetCategory", "Material")));
	AllocatedCategoryBits.Add(TEXT("Audio"), FAdvancedAssetCategory(EAssetTypeCategories::Sounds, LOCTEXT("SoundAssetCategory", "Audio")));
	AllocatedCategoryBits.Add(TEXT("Physics"), FAdvancedAssetCategory(EAssetTypeCategories::Physics, LOCTEXT("PhysicsAssetCategory", "Physics")));
	AllocatedCategoryBits.Add(TEXT("User Interface"), FAdvancedAssetCategory(EAssetTypeCategories::UI, LOCTEXT("UserInterfaceAssetCategory", "User Interface")));
	AllocatedCategoryBits.Add(TEXT("Misc"), FAdvancedAssetCategory(EAssetTypeCategories::Misc, LOCTEXT("MiscellaneousAssetCategory", "Miscellaneous")));
	AllocatedCategoryBits.Add(TEXT("Gameplay"), FAdvancedAssetCategory(EAssetTypeCategories::Gameplay, LOCTEXT("GameplayAssetCategory", "Gameplay")));
	AllocatedCategoryBits.Add(TEXT("Media"), FAdvancedAssetCategory(EAssetTypeCategories::Media, LOCTEXT("MediaAssetCategory", "Media")));
	AllocatedCategoryBits.Add(TEXT("Texture"), FAdvancedAssetCategory(EAssetTypeCategories::Textures, LOCTEXT("TextureAssetCategory", "Texture")));
	AllocatedCategoryBits.Add(TEXT("World"), FAdvancedAssetCategory(EAssetTypeCategories::World, LOCTEXT("WorldAssetCategory", "World")));
	AllocatedCategoryBits.Add(TEXT("FX"), FAdvancedAssetCategory(EAssetTypeCategories::FX, LOCTEXT("FXAssetCategory", "FX")));

	// Register the built-in asset type actions
	RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_AnimBoneCompressionSettings));
	RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_AnimCurveCompressionSettings));
	RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_VariableFrameStrippingSettings));
	RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_DeletedObjectPlaceholder));

	// Note: Please don't add any more actions here!  They belong in an editor-only module that is more tightly
	// coupled to your new system, and you should not create a dependency on your new system from AssetTools.
}

void UAssetToolsImpl::RemoveAssetTypeActionBySupportedClass(const UClass* SupportedClass)
{
	// Swap and pop the removed with the end, fixing up the lookup map if required
	if (const int* IndexPtr = AssetTypeActionsLookup.Find(SupportedClass))
	{
		const int LastIndex = AssetTypeActionsList.Num() - 1;
		const int Index = *IndexPtr;

		AssetTypeActionsLookup.Remove(SupportedClass);
		
		if (LastIndex != Index)
		{
			TSharedRef<IAssetTypeActions>& Last = AssetTypeActionsList.Last();
			AssetTypeActionsLookup.Add(Last->GetSupportedClass(), Index);
			AssetTypeActionsList[Index] = MoveTemp(Last);
		}

		AssetTypeActionsList.Pop();
	}

	verify(AssetTypeActionsLookup.Num() == AssetTypeActionsList.Num());
}


void UAssetToolsImpl::RegisterAssetTypeActions(const TSharedRef<IAssetTypeActions>& NewActions)
{
	UClass* SupportedClass = NewActions->GetSupportedClass();

	// If there is a duplicate AssetTypeActions registered, remove it
	// only the most recent one will be used
	RemoveAssetTypeActionBySupportedClass(SupportedClass);
	
	const int Index = AssetTypeActionsList.Num();
	AssetTypeActionsLookup.Add(SupportedClass, Index);
	AssetTypeActionsList.Add(NewActions);

	if (!NewActions->IsAssetDefinitionInDisguise())
	{
		UAssetDefinition_AssetTypeActionsProxy* Proxy = NewObject<UAssetDefinition_AssetTypeActionsProxy>();
		Proxy->Initialize(NewActions);
		
		UAssetDefinitionRegistry::Get()->RegisterAssetDefinition(Proxy);
	}
}

void UAssetToolsImpl::SyncAssetTypesToAssetDefinitions()
{
	uint64 Version = UAssetDefinitionRegistry::Get()->GetAssetDefinitionVersion();
	if (CachedVersion == 0 || CachedVersion != Version)
	{
		CachedVersion = Version;
		
		TArray<TObjectPtr<UAssetDefinition>> AssetDefinitions = UAssetDefinitionRegistry::Get()->GetAllAssetDefinitions();
		
		for (UAssetDefinition* AssetDefinition : AssetDefinitions)
		{
			if (!AssetDefinition->IsA<UAssetDefinition_AssetTypeActionsProxy>())
			{
				const TSoftClassPtr<UObject> AssetClass = AssetDefinition->GetAssetClass();
				const bool ActionDefinitionAlreadyProxied = 
					Algo::AnyOf(AssetTypeActionsList, [AssetClass](const TSharedRef<IAssetTypeActions>& Actions){ return Actions->GetSupportedClass() == AssetClass; });

				// If an asset definition is already registered check that the new one doesn't take precedence over the old one
				if (ActionDefinitionAlreadyProxied)
				{
					if (const int* Index = AssetTypeActionsLookup.Find(AssetClass.Get()))
					{
						TWeakPtr<IAssetTypeActions> TypeActions = AssetTypeActionsList[*Index];
						if (const TSharedPtr<IAssetTypeActions>& ActionPtr = TypeActions.Pin())
						{
							if (ActionPtr->IsAssetDefinitionInDisguise())
							{
								const TSharedPtr<FAssetDefinitionProxy>& ProxyAction = StaticCastSharedPtr<FAssetDefinitionProxy>(ActionPtr);
								if (const UAssetDefinition* ProxyAssetDefinition = ProxyAction->GetAssetDefinition())
								{
									// If the old one can't be registered statically, but the new one can, we should remove the old in favor of the new one
									if (!ProxyAssetDefinition->CanRegisterStatically() && AssetDefinition->CanRegisterStatically())
									{
										// Unregister the old one in favor of the one that can be registered statically
										UnregisterAssetTypeActions(ProxyAction.ToSharedRef());
										TSharedRef<FAssetDefinitionProxy> Proxy = MakeShared<FAssetDefinitionProxy>(AssetDefinition);
										// Cache the asset definition proxy.
										RegisterAssetTypeActions(Proxy);
									}
								}
							}
						}
					}	
				}
				else
				{
					TSharedRef<FAssetDefinitionProxy> Proxy = MakeShared<FAssetDefinitionProxy>(AssetDefinition);
					// Cache the asset definition proxy.
					RegisterAssetTypeActions(Proxy);
				}
			}
		}

		// Remove any AssetDefinitionProxy objects that don't have an associated AssetDefinition
		for (int32 Index = AssetTypeActionsList.Num() - 1, Last = 0; Index >= Last; --Index)
		{
			const TSharedRef<IAssetTypeActions>& AssetTypeActions = AssetTypeActionsList[Index];
			if (AssetTypeActions->IsAssetDefinitionInDisguise())
			{
				UClass* AssetTypeActionsSupportedClass = AssetTypeActions->GetSupportedClass();
				
				bool bFound = false;
				// Find it in the AssetDefinitions List
				for (UAssetDefinition* AssetDefinition : AssetDefinitions)
				{
					if (!AssetDefinition->IsA<UAssetDefinition_AssetTypeActionsProxy>())
					{
						const TSoftClassPtr<UObject> AssetClass = AssetDefinition->GetAssetClass();
						if (AssetClass == AssetTypeActionsSupportedClass)
						{
							bFound = true;
							break;
						}
					}
				}
				if (!bFound)
				{
					RemoveAssetTypeActionBySupportedClass(AssetTypeActionsSupportedClass);
				}
			}
		}		
	}
}

void UAssetToolsImpl::UnregisterAssetTypeActions(const TSharedRef<IAssetTypeActions>& ActionsToRemove)
{	
	const UClass* SupportedClass = ActionsToRemove->GetSupportedClass();
	RemoveAssetTypeActionBySupportedClass(SupportedClass);

	// Now also remove the proxy AssetDefinition we registered for this type.
	for (UAssetDefinition* AssetDefinition : UAssetDefinitionRegistry::Get()->GetAllAssetDefinitions())
	{
		if (UAssetDefinition_AssetTypeActionsProxy* Proxy = Cast<UAssetDefinition_AssetTypeActionsProxy>(AssetDefinition))
		{
			if (Proxy->GetAssetType() == ActionsToRemove)
			{
				UAssetDefinitionRegistry::Get()->UnregisterAssetDefinition(AssetDefinition);
				break;
			}
		}
	}
}

void UAssetToolsImpl::GetAssetTypeActionsList( TArray<TWeakPtr<IAssetTypeActions>>& OutAssetTypeActionsList ) const
{
	const_cast<UAssetToolsImpl*>(this)->SyncAssetTypesToAssetDefinitions();
	for (auto ActionsIt = AssetTypeActionsList.CreateConstIterator(); ActionsIt; ++ActionsIt)
	{
		OutAssetTypeActionsList.Add(*ActionsIt);
	}
}

TWeakPtr<IAssetTypeActions> UAssetToolsImpl::GetAssetTypeActionsForClass(const UClass* Class) const
{
	const_cast<UAssetToolsImpl*>(this)->SyncAssetTypesToAssetDefinitions();

	const UClass* CandidateClass = Class;
	
	while (CandidateClass)
	{
		const int* Index = AssetTypeActionsLookup.Find(CandidateClass);
		if (Index)
		{
			TWeakPtr<IAssetTypeActions> TypeActions = AssetTypeActionsList[*Index];
			return TypeActions;
		}

		// Walk up the class hierarchy until we find the most derived ClassTypeAction
		CandidateClass = CandidateClass->GetSuperClass();
	}

	return nullptr;
}

TSharedPtr<ILocalizedAssetTools> UAssetToolsImpl::GetLocalizedAssetTools() const
{
	return LocalizedAssetTools;
}

bool UAssetToolsImpl::CanLocalize(const UClass* Class) const
{
	return LocalizedAssetTools->CanLocalize(Class);
}

TOptional<FLinearColor> UAssetToolsImpl::GetTypeColor(const UClass* Class) const
{
	if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(Class))
	{
		return AssetDefinition->GetAssetColor();
	}
	else
	{
		if (TSharedPtr<IAssetTypeActions> AssetActions = GetAssetTypeActionsForClass(Class).Pin())
		{
			return FLinearColor(AssetActions->GetTypeColor());
		}
	}

	return TOptional<FLinearColor>();
}

TArray<TWeakPtr<IAssetTypeActions>> UAssetToolsImpl::GetAssetTypeActionsListForClass(const UClass* Class) const
{
	TArray<TWeakPtr<IAssetTypeActions>> ResultAssetTypeActionsList;

	const_cast<UAssetToolsImpl*>(this)->SyncAssetTypesToAssetDefinitions();
	for (int32 TypeActionsIdx = 0; TypeActionsIdx < AssetTypeActionsList.Num(); ++TypeActionsIdx)
	{
		TSharedRef<IAssetTypeActions> TypeActions = AssetTypeActionsList[TypeActionsIdx];
		UClass* SupportedClass = TypeActions->GetSupportedClass();

		if (Class->IsChildOf(SupportedClass))
		{
			ResultAssetTypeActionsList.Add(TypeActions);
		}
	}

	return ResultAssetTypeActionsList;
}

EAssetTypeCategories::Type UAssetToolsImpl::RegisterAdvancedAssetCategory(FName CategoryKey, FText CategoryDisplayName)
{
	EAssetTypeCategories::Type Result = FindAdvancedAssetCategory(CategoryKey);
	if (Result == EAssetTypeCategories::Misc)
	{
		if (NextUserCategoryBit != 0)
		{
			// Register the category
			Result = (EAssetTypeCategories::Type)NextUserCategoryBit;
			AllocatedCategoryBits.Add(CategoryKey, FAdvancedAssetCategory(Result, CategoryDisplayName));

			UE_LOGF(LogAssetTools, Verbose, "RegisterAssetTypeCategory(\"%ls\", \"%ls\") used up bit 0x%x, Offset: %d", *CategoryKey.ToString(), *CategoryDisplayName.ToString(), NextUserCategoryBit, FMath::FloorLog2(Result));

			// Advance to the next bit, or store that we're out
			if (NextUserCategoryBit == EAssetTypeCategories::LastUser)
			{
				NextUserCategoryBit = 0;
			}
			else
			{
				NextUserCategoryBit = NextUserCategoryBit << 1;
			}
		}
		else
		{
			UE_LOGF(LogAssetTools, Warning, "RegisterAssetTypeCategory(\"%ls\", \"%ls\") failed as all user bits have been exhausted (placing into the Misc category instead)", *CategoryKey.ToString(), *CategoryDisplayName.ToString());
		}
	}

	return Result;
}

EAssetTypeCategories::Type UAssetToolsImpl::FindAdvancedAssetCategory(FName CategoryKey) const
{
	if (const FAdvancedAssetCategory* ExistingCategory = AllocatedCategoryBits.Find(CategoryKey))
	{
		return ExistingCategory->CategoryType;
	}
	else
	{
		return EAssetTypeCategories::Misc;
	}
}

void UAssetToolsImpl::GetAllAdvancedAssetCategories(TArray<FAdvancedAssetCategory>& OutCategoryList) const
{
	AllocatedCategoryBits.GenerateValueArray(OutCategoryList);
}

void UAssetToolsImpl::RegisterClassTypeActions(const TSharedRef<IClassTypeActions>& NewActions)
{
	ClassTypeActionsList.Add(NewActions);
}

void UAssetToolsImpl::UnregisterClassTypeActions(const TSharedRef<IClassTypeActions>& ActionsToRemove)
{
	ClassTypeActionsList.Remove(ActionsToRemove);
}

void UAssetToolsImpl::GetClassTypeActionsList( TArray<TWeakPtr<IClassTypeActions>>& OutClassTypeActionsList ) const
{
	for (auto ActionsIt = ClassTypeActionsList.CreateConstIterator(); ActionsIt; ++ActionsIt)
	{
		OutClassTypeActionsList.Add(*ActionsIt);
	}
}

TWeakPtr<IClassTypeActions> UAssetToolsImpl::GetClassTypeActionsForClass( UClass* Class ) const
{
	TSharedPtr<IClassTypeActions> MostDerivedClassTypeActions;

	for (int32 TypeActionsIdx = 0; TypeActionsIdx < ClassTypeActionsList.Num(); ++TypeActionsIdx)
	{
		TSharedRef<IClassTypeActions> TypeActions = ClassTypeActionsList[TypeActionsIdx];
		UClass* SupportedClass = TypeActions->GetSupportedClass();

		if ( Class->IsChildOf(SupportedClass) )
		{
			if ( !MostDerivedClassTypeActions.IsValid() || SupportedClass->IsChildOf( MostDerivedClassTypeActions->GetSupportedClass() ) )
			{
				MostDerivedClassTypeActions = TypeActions;
			}
		}
	}

	return MostDerivedClassTypeActions;
}

UObject* UAssetToolsImpl::CreateAsset(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FName CallingContext, bool bOverwriteExisting)
{
	FGCObjectScopeGuard DontGCFactory(Factory);

	// Verify the factory class
	if ( !ensure(AssetClass || Factory) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("MustSupplyClassOrFactory", "The new asset wasn't created due to a problem finding the appropriate factory or class for the new asset.") );
		return nullptr;
	}

	// This is the class default and this should not fail for all current factories.
	if (Factory && !EnumHasAnyFlags(Factory->GetSupportedWorkflows(), EFactoryCreateWorkflow::Default))
	{
		FMessageDialog::Open(EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
		return nullptr;
	}

	// Verify the factory supports the asset class
	if (AssetClass && Factory && 
		((Factory->SupportedClass != nullptr && !ensure(AssetClass->IsChildOf(Factory->GetSupportedClass()))) || 
		 (Factory->SupportedClass == nullptr && !ensure(Factory->DoesSupportClass(AssetClass)))) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
		return nullptr;
	}

	const FString PackageName = UPackageTools::SanitizePackageName(PackagePath + TEXT("/") + AssetName);

	// Make sure we can create the asset without conflicts
	if ( !CanCreateAsset(AssetName, PackageName, LOCTEXT("CreateANewObject", "Create a new object"), bOverwriteExisting) )
	{
		return nullptr;
	}

	UClass* ClassToUse = AssetClass ? AssetClass : (Factory ? Factory->GetSupportedClass() : nullptr);

	UPackage* Pkg = CreatePackage(*PackageName);
	UObject* NewObj = nullptr;
	EObjectFlags Flags = RF_Public|RF_Standalone|RF_Transactional;
	if ( Factory )
	{
		NewObj = Factory->FactoryCreateNew(ClassToUse, Pkg, FName( *AssetName ), Flags, nullptr, GWarn, CallingContext);
	}
	else if ( AssetClass )
	{
		NewObj = NewObject<UObject>(Pkg, ClassToUse, FName(*AssetName), Flags);
	}

	if( NewObj )
	{
		Pkg->SetAssetAccessSpecifier(ShouldCreateAssetsAsExternallyReferenceableForPath(PackageName) ? EAssetAccessSpecifier::Public : EAssetAccessSpecifier::Private);

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(NewObj);

		// analytics create record
		UAssetToolsImpl::OnNewCreateRecord(AssetClass, false);

		// Mark the package dirty...
		Pkg->MarkPackageDirty();
	}

	return NewObj;
}

bool UAssetToolsImpl::CreateAssetAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled, FName CallingContext, bool bOverwriteExisting)
{
	// Verify the factory class
	if (!ensure(AssetClass && Factory))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MustSupplyClassOrFactory", "The new asset wasn't created due to a problem finding the appropriate factory or class for the new asset."));
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	if (!EnumHasAnyFlags(Factory->GetSupportedWorkflows(), EFactoryCreateWorkflow::Asynchronous))
	{
		FMessageDialog::Open(EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	// Verify the factory supports the asset class
	if (((Factory->SupportedClass != nullptr && !ensure(AssetClass->IsChildOf(Factory->GetSupportedClass()))) ||
			(Factory->SupportedClass == nullptr && !ensure(Factory->DoesSupportClass(AssetClass)))))
	{
		FMessageDialog::Open(EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	const FString PackageName = UPackageTools::SanitizePackageName(PackagePath + TEXT("/") + AssetName);

	// Make sure we can create the asset without conflicts
	if (!CanCreateAsset(AssetName, PackageName, LOCTEXT("CreateANewObject", "Create a new object"), bOverwriteExisting))
	{
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	UClass* ClassToUse = AssetClass ? AssetClass : (Factory ? Factory->GetSupportedClass() : nullptr);

	UPackage* Pkg = CreatePackage(*PackageName);
	EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;

	FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(Factory);

	TStrongObjectPtr<UPackage> DontGCPackage(Pkg);

	return Factory->ConfigurePropertiesAsync(
		FOnFactoryConfigurePropertiesAsyncComplete::CreateUObject(this, &UAssetToolsImpl::CreateAssetAsync_OnComplete, DontGCPackage, ClassToUse, 
			AssetName, Flags, CallingContext, PackageName, OnComplete, OnCancelled),
		FOnFactoryConfigurePropertiesAsyncCancelled()
	);
}

void UAssetToolsImpl::CreateAssetsFrom(TConstArrayView<UObject*> SourceObjects, UClass* CreateAssetType, const FString& DefaultSuffix, TFunctionRef<UFactory*(UObject*)> FactoryConstructor, FName CallingContext)
{
	if ( SourceObjects.Num() == 1 )
	{
		if (UObject* SourceObject = SourceObjects[0])
		{
			if (UFactory* Factory = FactoryConstructor(SourceObject))
			{
				// Create an appropriate and unique name 
				TOptional<FString> DefaultAssetName = GetDefaultAssetNameForClass(CreateAssetType, Factory, SourceObject);

				const FString BasePackageName = DefaultAssetName.Get(SourceObject->GetName());
				const FString Suffix = DefaultAssetName.IsSet() ? FString() : DefaultSuffix;

				const FString PackagePath = FPackageName::GetLongPackagePath(SourceObject->GetPackage()->GetName());

				FString Name;
				FString PackageName;
				CreateUniqueAssetName(PackagePath / BasePackageName, Suffix, PackageName, Name);

				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackageName), CreateAssetType, Factory);
			}
		}
	}
	else
	{
		TArray<UObject*> ObjectsToSync;
		for (UObject* SourceObject : SourceObjects)
		{
			if ( SourceObject )
			{
				// Create the factory used to generate the asset
				if (UFactory* Factory = FactoryConstructor(SourceObject))
				{
					// Determine an appropriate name
					TOptional<FString> DefaultAssetName = GetDefaultAssetNameForClass(CreateAssetType, Factory, SourceObject);

					const FString BasePackageName = DefaultAssetName.Get(SourceObject->GetName());
					const FString Suffix = DefaultAssetName.IsSet() ? FString() : DefaultSuffix;

					const FString PackagePath = FPackageName::GetLongPackagePath(SourceObject->GetPackage()->GetName());

					FString Name;
					FString PackageName;
					CreateUniqueAssetName(PackagePath / BasePackageName, Suffix, PackageName, Name);
					
					if (UObject* NewAsset = CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), CreateAssetType, Factory, CallingContext))
					{
						ObjectsToSync.Add(NewAsset);
					}
				}
			}
		}

		if ( ObjectsToSync.Num() > 0 )
		{
			SyncBrowserToAssets(ObjectsToSync);
		}
	}
}

void UAssetToolsImpl::CreateAssetAsync_OnComplete(UFactory* Factory, TStrongObjectPtr<UPackage> Package, UClass* AssetClass, FString AssetName,
	EObjectFlags Flags, FName CallingContext, FString PackageName, FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled)
{
	UPackage* Pkg = Package.Get();
	UObject* NewObj = Factory->FactoryCreateNew(AssetClass, Pkg, FName(*AssetName), Flags, nullptr, GWarn, CallingContext);

	if (NewObj)
	{
		Pkg->SetIsExternallyReferenceable(ShouldCreateAssetsAsExternallyReferenceableForPath(PackageName));

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(NewObj);

		// analytics create record
		UAssetToolsImpl::OnNewCreateRecord(AssetClass, false);

		// Mark the package dirty...
		Pkg->MarkPackageDirty();

		OnComplete.ExecuteIfBound(Factory, NewObj);
	}
	else
	{
		OnCancelled.ExecuteIfBound(Factory);
	}
}

UObject* UAssetToolsImpl::CreateAssetWithDialog(UClass* AssetClass, UFactory* Factory, FName CallingContext)
{
	if (Factory != nullptr)
	{
		// Determine the starting path. Try to use the most recently used directory
		FString AssetPath;

		const FString DefaultFilesystemDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		if (DefaultFilesystemDirectory.IsEmpty() || !FPackageName::TryConvertFilenameToLongPackageName(DefaultFilesystemDirectory, AssetPath))
		{
			// No saved path, just use the game content root
			AssetPath = TEXT("/Game");
		}

		// Determine an appropriate name
		TOptional<FString> DefaultAssetName = GetDefaultAssetNameForClass(AssetClass, Factory, nullptr);
		const FString BaseAssetName = DefaultAssetName.Get(Factory->GetDefaultNewAssetName());

		FString PackageName;
		FString AssetName;
		CreateUniqueAssetName(AssetPath / BaseAssetName, TEXT(""), PackageName, AssetName);

		return CreateAssetWithDialog(AssetName, AssetPath, AssetClass, Factory, CallingContext);
	}

	return nullptr;
}


UObject* UAssetToolsImpl::CreateAssetWithDialog(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FName CallingContext, const bool bCallConfigureProperties)
{
	FGCObjectScopeGuard DontGCFactory(Factory);

	if(Factory)
	{
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
		SaveAssetDialogConfig.DefaultPath = PackagePath;
		SaveAssetDialogConfig.DefaultAssetName = AssetName;
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		if (Factory->GetSupportedClass() != nullptr)
		{
			SaveAssetDialogConfig.AssetClassNames.Add(Factory->GetSupportedClass()->GetClassPathName());
		}
		else if(AssetClass && Factory->DoesSupportClass(AssetClass))
		{
			SaveAssetDialogConfig.AssetClassNames.Add(AssetClass->GetClassPathName());
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
			return nullptr;
		}		

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

		if (!SaveObjectPath.IsEmpty())
		{
			const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			const FString SavePackagePath = FPaths::GetPath(SavePackageName);
			const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, SavePackagePath);
			
			bool bCreateAsset = true;
			if (bCallConfigureProperties)
			{
				FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(Factory);
				bCreateAsset = Factory->ConfigureProperties();
			}

			if (bCreateAsset)
			{
				return CreateAsset(SaveAssetName, SavePackagePath, AssetClass, Factory, CallingContext);
			}
		}
	}

	return nullptr;
}

bool UAssetToolsImpl::CreateAssetWithDialogAsync(UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete,
	FAssetCreateCancelled OnCancelled, FName CallingContext)
{
	if (!Factory)
	{
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	// Determine the starting path. Try to use the most recently used directory
	FString AssetPath;

	const FString DefaultFilesystemDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
	if (DefaultFilesystemDirectory.IsEmpty() || !FPackageName::TryConvertFilenameToLongPackageName(DefaultFilesystemDirectory, AssetPath))
	{
		// No saved path, just use the game content root
		AssetPath = TEXT("/Game");
	}

	// Determine an appropriate name
	const TOptional<FString> DefaultAssetName = GetDefaultAssetNameForClass(AssetClass, Factory, nullptr);
	const FString BaseAssetName = DefaultAssetName.Get(Factory->GetDefaultNewAssetName());

	FString PackageName;
	FString AssetName;
	CreateUniqueAssetName(AssetPath / BaseAssetName, TEXT(""), PackageName, AssetName);

	return CreateAssetWithDialogAsync(AssetName, AssetPath, AssetClass, Factory, OnComplete, OnCancelled, CallingContext);
}

bool UAssetToolsImpl::CreateAssetWithDialogAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory,
	FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled, FName CallingContext)
{
	if (!Factory)
	{
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	if (!EnumHasAnyFlags(Factory->GetSupportedWorkflows(), EFactoryCreateWorkflow::Asynchronous))
	{
		FMessageDialog::Open(EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PackagePath;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

	if (Factory->GetSupportedClass() != nullptr)
	{
		SaveAssetDialogConfig.AssetClassNames.Add(Factory->GetSupportedClass()->GetClassPathName());
	}
	else if (AssetClass && Factory->DoesSupportClass(AssetClass))
	{
		SaveAssetDialogConfig.AssetClassNames.Add(AssetClass->GetClassPathName());
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, UE::Private::GetInvalidFactoryMessage());
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		OnCancelled.ExecuteIfBound(Factory);
		return false;
	}

	FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(Factory);

	return Factory->ConfigurePropertiesAsync(
		FOnFactoryConfigurePropertiesAsyncComplete::CreateUObject(this, &UAssetToolsImpl::CreateAssetWithDialogAsync_OnComplete,
			AssetClass, CallingContext, SaveObjectPath, OnComplete, OnCancelled),
		FOnFactoryConfigurePropertiesAsyncCancelled()
	);
}

void UAssetToolsImpl::CreateAssetWithDialogAsync_OnComplete(UFactory* Factory, UClass* AssetClass, FName CallingContext, FString SaveObjectPath,
	FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled)
{
	const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
	const FString SavePackagePath = FPaths::GetPath(SavePackageName);
	const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, SavePackagePath);

	if (UObject* NewObj = CreateAsset(SaveAssetName, SavePackagePath, AssetClass, Factory, CallingContext))
	{
		OnComplete.ExecuteIfBound(Factory, NewObj);
	}
	else
	{
		OnCancelled.ExecuteIfBound(Factory);
	}
}

UObject* UAssetToolsImpl::DuplicateAssetWithDialog(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject)
{
	return DuplicateAssetWithDialogAndTitle(AssetName, PackagePath, OriginalObject, LOCTEXT("DuplicateAssetDialogTitle", "Duplicate Asset As"));
}

UObject* UAssetToolsImpl::DuplicateAssetWithDialogAndTitle(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject, FText DialogTitle)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = DialogTitle;
	SaveAssetDialogConfig.DefaultPath = PackagePath;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	if (!SaveObjectPath.IsEmpty())
	{
		const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		const FString SavePackagePath = FPaths::GetPath(SavePackageName);
		const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, SavePackagePath);

		return PerformDuplicateAsset(SaveAssetName, SavePackagePath, OriginalObject, true);
	}

	return nullptr;
}

UObject* UAssetToolsImpl::DuplicateAsset(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject)
{
	return PerformDuplicateAsset(AssetName, PackagePath, OriginalObject, false);
}

UObject* UAssetToolsImpl::PerformDuplicateAsset(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject, bool bWithDialog)
{
	// Verify the source object
	if ( !OriginalObject )
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidSourceObject", "The new asset wasn't created due to a problem finding the object to duplicate.") );
		return nullptr;
	}

	const FString PackageName = PackagePath / AssetName;

	// Make sure we can create the asset without conflicts
	if ( !CanCreateAsset(AssetName, PackageName, LOCTEXT("DuplicateAnObject", "Duplicate an object"), false) )
	{
		return nullptr;
	}

	ObjectTools::FPackageGroupName PGN;
	PGN.PackageName = PackageName;
	PGN.GroupName = TEXT("");
	PGN.ObjectName = AssetName;

	TSet<UPackage*> ObjectsUserRefusedToFullyLoad;
	bool bPromtToOverwrite = bWithDialog;
	UObject* NewObject = ObjectTools::DuplicateSingleObject(OriginalObject, PGN, ObjectsUserRefusedToFullyLoad, bPromtToOverwrite);
	if(NewObject != nullptr)
	{
		// Assets must have RF_Public and RF_Standalone
		const bool bIsAsset = NewObject->IsAsset();
		NewObject->SetFlags(RF_Public | RF_Standalone);

		if (!bIsAsset && NewObject->IsAsset())
		{
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(NewObject);
		}

		if ( ISourceControlModule::Get().IsEnabled() )
		{
			// Save package here if SCC is enabled because the user can use SCC to revert a change
			TArray<UPackage*> OutermostPackagesToSave;
			OutermostPackagesToSave.Add(NewObject->GetOutermost());

			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			FEditorFileUtils::PromptForCheckoutAndSave(OutermostPackagesToSave, bCheckDirty, bPromptToSave);

			// now attempt to branch, we can do this now as we should have a file on disk
			SourceControlHelpers::CopyPackage(NewObject->GetOutermost(), OriginalObject->GetOutermost());
		}

		// analytics create record
		UAssetToolsImpl::OnNewCreateRecord(NewObject->GetClass(), true);
	}

	return NewObject;
}

void UAssetToolsImpl::SetCreateAssetsAsExternallyReferenceable(bool bValue)
{
	CreateAssetsAsExternallyReferenceable = bValue;
}

bool UAssetToolsImpl::GetCreateAssetsAsExternallyReferenceable()
{
	return CreateAssetsAsExternallyReferenceable;
}

bool UAssetToolsImpl::ShouldCreateAssetsAsExternallyReferenceableForPath(const FStringView AssetPath) const
{
	// Allow other systems to control the result
	TOptional<bool> bExternallyReferenceable;
	CreateAssetsAsExternallyReferenceableForPathDelegate.Broadcast(AssetPath, bExternallyReferenceable);
	if (bExternallyReferenceable.IsSet())
	{
		return bExternallyReferenceable.GetValue();
	}

	// If no other systems set a result, then use the global setting applied to all assets
	return CreateAssetsAsExternallyReferenceable;
}

UE::AssetTools::FShouldCreateAssetsAsExternallyReferenceableForPath& UAssetToolsImpl::GetOnShouldCreateAssetsAsExternallyReferenceableForPath()
{
	return CreateAssetsAsExternallyReferenceableForPathDelegate;
}

void UAssetToolsImpl::GenerateAdvancedCopyDestinations(FAdvancedCopyParams& InParams, const TArray<FName>& InPackageNamesToCopy, const UAdvancedCopyCustomization* CopyCustomization, TMap<FString, FString>& OutPackagesAndDestinations) const
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	FString DestinationFolder = InParams.GetDropLocationForAdvancedCopy();

	if (ensure(DesktopPlatform))
	{
		FPaths::NormalizeFilename(DestinationFolder);
	}
	else
	{
		// Not on a platform that supports desktop functionality
		return;
	}

	bool bGenerateRelativePaths = CopyCustomization->GetShouldGenerateRelativePaths();

	for (auto PackageNameIt = InPackageNamesToCopy.CreateConstIterator(); PackageNameIt; ++PackageNameIt)
	{
		FName PackageName = *PackageNameIt;

		const FString& PackageNameString = PackageName.ToString();
		FString SrcFilename;
		if (FPackageName::DoesPackageExist(PackageNameString, &SrcFilename))
		{
			bool bFileOKToCopy = true;

			FString DestFilename = DestinationFolder;

			FString SubFolder;
			if (SrcFilename.Split(TEXT("/Content/"), nullptr, &SubFolder))
			{
				DestFilename += *SubFolder;
			}
			else
			{
				// Couldn't find Content folder in source path
				bFileOKToCopy = false;
			}

			if (bFileOKToCopy)
			{
				FString Parent = FString();
				if (bGenerateRelativePaths)
				{

					FString RootFolder = CopyCustomization->GetPackageThatInitiatedCopy();
					if (RootFolder != PackageNameString)
					{
						FString BaseParent = FString();
						int32 MinLength = RootFolder.Len() < PackageNameString.Len() ? RootFolder.Len() : PackageNameString.Len();
						for (int Char = 0; Char < MinLength; Char++)
						{
							if (RootFolder[Char] == PackageNameString[Char])
							{
								BaseParent += RootFolder[Char];
							}
							else
							{
								break;
							}
						}

						// If we are in the root content folder, don't break down the folder string
						if (BaseParent == TEXT("/Game"))
						{
							Parent = BaseParent;
						}
						else
						{
							BaseParent.Split(TEXT("/"), &Parent, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
						}
					}
				}

				const FString DestinationPackageName = UAssetToolsImpl::GenerateAdvancedCopyDestinationPackageName(PackageNameString, Parent, DestinationFolder);
				OutPackagesAndDestinations.Add(PackageNameString, DestinationPackageName);
			}
		}
	}
}

bool UAssetToolsImpl::CanShowDialogs()
{
	return !FApp::IsUnattended() && FSlateApplication::IsInitialized() && !GIsRunningUnattendedScript && !IsRunningCommandlet();
}

FString UAssetToolsImpl::GenerateAdvancedCopyDestinationPackageName(const FString& SourcePackage, const FString& SourcePath, const FString& DestinationFolder)
{
	FString DestinationPackageName;

	const bool bIsRelativeOperation = SourcePath.Len() && DestinationFolder.Len() && SourcePackage.StartsWith(SourcePath);
	if (bIsRelativeOperation)
	{
		// Folder copy/move.

		// Collect the relative path then use it to determine the new location
		// For example, if SourcePath = /Game/MyPath and SourcePackage = /Game/MyPath/MySubPath/MyAsset
		//     /Game/MyPath/MySubPath/MyAsset -> /MySubPath/

		const int32 ShortPackageNameLen = FPackageName::GetShortName(SourcePackage).Len();
		const int32 RelativePathLen = SourcePackage.Len() - ShortPackageNameLen - SourcePath.Len();
		const FString RelativeDestPath = SourcePackage.Mid(SourcePath.Len(), RelativePathLen);

		DestinationPackageName = DestinationFolder + RelativeDestPath + FPackageName::GetShortName(SourcePackage);
	}
	else if (DestinationFolder.Len())
	{
		// Use the passed in default path
		// Normal path
		// 
		// NOTE: We cannot use a shortened path from SourcePackage because there may be assets that share names.
		//       To prevent assets from copying over eachother we need a unique destination path.

		// A little nicety: don't expose '/Game/' to users -- instead create a subfolder using the project's name
		if (SourcePackage.StartsWith(TEXT("/Game/")))
		{
			FString ThrowawayRoot, SubPath, PkgName;
			FPackageName::SplitLongPackageName(SourcePackage, ThrowawayRoot, SubPath, PkgName);

			DestinationPackageName = DestinationFolder / FApp::GetProjectName() / SubPath / PkgName;
		}
		else // Other non-'/Game/' paths (ones from plugins, etc.)...
		{
			DestinationPackageName = DestinationFolder / SourcePackage;
		}
	}
	else
	{
		// Use the path from the old package
		DestinationPackageName = SourcePackage;
	}

	return DestinationPackageName;
}

bool UAssetToolsImpl::FlattenAdvancedCopyDestinations(const TArray<TMap<FString, FString>>& PackagesAndDestinations, TMap<FString, FString>& FlattenedPackagesAndDestinations) const
{
	FString CopyErrors;
	for (const TMap<FString, FString>& PackageAndDestinationMap : PackagesAndDestinations)
	{
		for (auto It = PackageAndDestinationMap.CreateConstIterator(); It; ++It)
		{
			const FString& PackageName = It.Key();
			const FString& DestFilename = It.Value();

			if (FlattenedPackagesAndDestinations.Contains(PackageName))
			{
				const FString* ExistingDestinationPtr = FlattenedPackagesAndDestinations.Find(PackageName);
				const FString ExistingDestination = *ExistingDestinationPtr;
				if (ExistingDestination != DestFilename)
				{
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AdvancedCopy_DuplicateDestinations", "Advanced Copy failed because {0} was being duplicated in two locations, {1} and {2}."),
						FText::FromString(PackageName),
						FText::FromString(FPaths::GetPath(ExistingDestination)),
						FText::FromString(FPaths::GetPath(DestFilename))));
					return false;
				}
			}
			
			// File passed all error conditions above, add it to valid flattened list
			FlattenedPackagesAndDestinations.Add(PackageName, DestFilename);
		}
	}
	// All files passed all validation tests
	return true;
}

bool UAssetToolsImpl::ValidateFlattenedAdvancedCopyDestinations(const TMap<FString, FString>& FlattenedPackagesAndDestinations) const
{
	FString CopyErrors;

	for (auto It = FlattenedPackagesAndDestinations.CreateConstIterator(); It; ++It)
	{
		const FString& PackageName = It.Key();
		const FString& DestFilename = It.Value();

		// Check for source/destination collisions
		if (PackageName == FPaths::GetPath(DestFilename))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AdvancedCopy_DuplicatedSource", "Advanced Copy failed because {0} was being copied over itself."), FText::FromString(PackageName)));
			return false;
		}
		else if (FlattenedPackagesAndDestinations.Contains(FPaths::GetPath(DestFilename)))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AdvancedCopy_DestinationEqualsSource", "Advanced Copy failed because {0} was being copied over the source file {1}."),
				FText::FromString(PackageName),
				FText::FromString(FPaths::GetPath(DestFilename))));
			return false;
		}

		// Check for valid copy locations
		FString SrcFilename;
		if (!FPackageName::DoesPackageExist(PackageName, &SrcFilename))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AdvancedCopyPackages_PackageMissing", "{0} does not exist on disk."), FText::FromString(PackageName)));
			return false;
		}
		else if (SrcFilename.Contains(FPaths::EngineContentDir()))
		{
			const FString LeafName = SrcFilename.Replace(*FPaths::EngineContentDir(), TEXT("Engine/"));
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AdvancedCopyPackages_EngineContent", "Unable to copy Engine asset {0}. Engine assets cannot be copied using Advanced Copy."),
				FText::FromString(LeafName)));
			return false;
		}
	}
	
	// All files passed all validation tests
	return true;
}

void UAssetToolsImpl::GetAllAdvancedCopySources(FName SelectedPackage, FAdvancedCopyParams& CopyParams, TArray<FName>& OutPackageNamesToCopy, TMap<FName, FName>& DependencyMap, const UAdvancedCopyCustomization* CopyCustomization) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FString CurrentRoot = FString();
	TArray<FName> CurrentDependencies;
	if (!OutPackageNamesToCopy.Contains(SelectedPackage))
	{
		TArray<FAssetData> SourceAssetData;
		AssetRegistry.GetAssetsByPackageName(SelectedPackage, SourceAssetData);
		// Check if this is a folder before using the filter to exclude assets
		bool bIsFolder = SourceAssetData.Num() == 0;
		FARCompiledFilter CompiledExclusionFilter;
		AssetRegistry.CompileFilter(CopyCustomization->GetARFilter(), CompiledExclusionFilter);
		AssetRegistry.UseFilterToExcludeAssets(SourceAssetData, CompiledExclusionFilter);
		// If this is a valid asset
		if (SourceAssetData.Num() > 0 || bIsFolder)
		{
			CurrentDependencies.Add(SelectedPackage);
		}

		// If we should check for dependencies OR we are currently checking a folder
		// Folders should ALWAYS get checked for assets and subfolders
		if ((CopyParams.bShouldCheckForDependencies && SourceAssetData.Num() > 0) || bIsFolder)
		{
			RecursiveGetDependenciesAdvanced(SelectedPackage, CopyParams, CurrentDependencies, DependencyMap, CompiledExclusionFilter, SourceAssetData);
		}
		OutPackageNamesToCopy.Append(CurrentDependencies);
	}
}

bool UAssetToolsImpl::AdvancedCopyPackages(
	const TMap<FString, FString>& SourceAndDestPackages,
	const bool bForceAutosave,
	const bool bCopyOverAllDestinationOverlaps,
	FDuplicatedObjects* OutDuplicatedObjects,
	EMessageSeverity::Type NotificationSeverityFilter) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AdvancedCopyPackages);

	if (ValidateFlattenedAdvancedCopyDestinations(SourceAndDestPackages))
	{
		TArray<FString> SuccessfullyCopiedDestinationFiles;
		TArray<FName> SuccessfullyCopiedSourcePackages;
		TArray<UPackage*> SuccessfullyCopiedDestinationPackages;

		FDuplicatedObjects DuplicatedObjectsLocal;
		FDuplicatedObjects& DuplicatedObjectsForEachPackage = OutDuplicatedObjects ? *OutDuplicatedObjects : DuplicatedObjectsLocal;

		TSet<UObject*> ExistingObjectSet;
		TSet<UObject*> NewObjectSet;
		TSet<FName> CopiedWorldPartitionMaps;
		FString CopyErrors;

		SuccessfullyCopiedDestinationFiles.Reserve(SourceAndDestPackages.Num());
		SuccessfullyCopiedSourcePackages.Reserve(SourceAndDestPackages.Num());
		SuccessfullyCopiedDestinationPackages.Reserve(SourceAndDestPackages.Num());
		DuplicatedObjectsForEachPackage.Reserve(SourceAndDestPackages.Num());
		ExistingObjectSet.Reserve(SourceAndDestPackages.Num());
		NewObjectSet.Reserve(SourceAndDestPackages.Num());

		TUniquePtr<FScopedSlowTask> LoopProgress;

		if (UE::AssetTools::Private::bEnableHeaderPatching)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AdvancedCopyPackages.HeaderPatching);

			FAssetHeaderPatcher::FContext PatcherContext(SourceAndDestPackages);
			FAssetHeaderPatcher Patcher(MoveTemp(PatcherContext));

			int32 NumFilesToPatch = 0;
			int32 PatchAssetsCompletedCount = 0;
			FAssetHeaderPatcher::FPatchAsyncParams PatchParams;
			PatchParams.OutNumFilesToPatch = &NumFilesToPatch;
			PatchParams.OutNumFilesPatched = &PatchAssetsCompletedCount;
			PatchParams.TaskPriority = UE::Tasks::ETaskPriority::BackgroundHigh;

			UE::Tasks::FTask PatchAssetsCompletionTask = Patcher.PatchAsync(PatchParams);
		
			LoopProgress = MakeUnique<FScopedSlowTask>(static_cast<float>(NumFilesToPatch), LOCTEXT("AdvancedCopyPackages.ReplacingAssetReferences", "Replacing Asset References..."));
			LoopProgress->MakeDialog();

			while (!PatchAssetsCompletionTask.Wait(FTimespan::FromSeconds(0.5)))
			{
				// We don't support C++20 in all modules and platforms yet and avoid using atomic_ref as a result
				LoopProgress->CompletedWork = (float)FPlatformAtomics::AtomicRead_Relaxed((volatile int32*)&PatchAssetsCompletedCount);
				LoopProgress->TickProgress();
			}

			// Sorting out the results.
			// And reporting to the user
			FMessageLog AdvancedCopyLog("AssetTools");

			bool bHasErrors = Patcher.HasErrors();
			TMap<FString, FAssetHeaderPatcher::EResult> ErroredFiles;
			if (bHasErrors)
			{
				ErroredFiles = Patcher.GetErrorFiles();
				AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_SourceControlErrorsListPage", "Revision Control Errors"));
			}

			// reporting files with copy errors and filtering successful ones
			for (const auto& FilenameAndResult : ErroredFiles)
			{			
				AdvancedCopyLog.Error(FText::Format(LOCTEXT("AdvancedCopyPackages_CouldNotProcessError", "{0} could not be processed: {1}"), FText::FromString(FilenameAndResult.Key), FText::FromString(*LexToString(FilenameAndResult.Value))));
			}

			if (SuccessfullyCopiedDestinationFiles.Num() > 0
			&& GetDefault<UEditorLoadingSavingSettings>()->bSCCAutoAddNewFiles
			&& ISourceControlModule::Get().IsEnabled())
			{
				// attempt to add files to source control (this can quite easily fail, but if it works it is very useful)
				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
				if (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), SourceControlHelpers::AbsoluteFilenames(SuccessfullyCopiedDestinationFiles)) == ECommandResult::Failed)
				{
					for (const FString& Filename : SuccessfullyCopiedDestinationFiles)
					{
						if (!SourceControlProvider.GetState(*Filename, EStateCacheUsage::Use)->IsAdded())
						{
							if (!bHasErrors)
							{
								bHasErrors = true;
								AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_SourceControlErrorsListPage", "Revision Control Errors"));
							}

							AdvancedCopyLog.Error(FText::Format(LOCTEXT("AdvancedCopyPackages_SourceControlError", "{0} could not be added to revision control"), FText::FromString(*Filename)));
						}
					}
				}
			}

			// Report the result to the user
			FText LogMessage = FText::FromString(TEXT("Advanced content copy completed successfully!"));
			EMessageSeverity::Type Severity = EMessageSeverity::Info;
			FString ErrorsString;
			if (bHasErrors)
			{
				Severity = EMessageSeverity::Error;

				FString ErrorMessage = LOCTEXT("AdvancedCopyPackages_SourceControlErrorsList", "Some files reported revision control errors.").ToString();

				if (SuccessfullyCopiedSourcePackages.Num() > 0)
				{
					AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_CopyErrorsSuccesslistPage", "Copied Successfully"));
					for (const FString& Filename : SuccessfullyCopiedDestinationFiles)
					{
						AdvancedCopyLog.Info(FText::FromString(Filename));
					}

					ErrorMessage += LINE_TERMINATOR;
					ErrorMessage += LOCTEXT("AdvancedCopyPackages_CopyErrorsSuccesslist", "Some files were copied successfully.").ToString();
				}
				LogMessage = FText::FromString(ErrorMessage);
			}
			else
			{
				AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_CompletePage", "Advanced content copy completed successfully!"));
				for (const FString& Filename : SuccessfullyCopiedDestinationFiles)
				{
					AdvancedCopyLog.Info(FText::FromString(Filename));
				}
			}
			// @note Using the bForce param because the InSeverityFilter param is only checked against logs in the last page
			AdvancedCopyLog.Notify(LogMessage, /*InSeverityFilter=*/EMessageSeverity::Error, /*bForce=*/(Severity <= NotificationSeverityFilter));
			return true;
		}
		else
		{
			LoopProgress = MakeUnique<FScopedSlowTask>(static_cast<float>(SourceAndDestPackages.Num()), LOCTEXT("AdvancedCopyPackages.CopyingFilesAndDependencies", "Copying Files and Dependencies..."));
			LoopProgress->MakeDialog();

			for (const auto& Package : SourceAndDestPackages)
			{
				const FString& PackageName = Package.Key;
				const FString& DestPackage = Package.Value;
				FString SrcFilename;

				if (FPackageName::DoesPackageExist(PackageName, &SrcFilename))
				{
					LoopProgress->EnterProgressFrame();

					UPackage* Pkg = LoadPackage(nullptr, *PackageName, LOAD_None);
					if (Pkg)
					{
						FString Name = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(SrcFilename));
						UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), Pkg, *Name);
						if (ExistingObject)
						{
							TSet<UPackage*> ObjectsUserRefusedToFullyLoad;
							ObjectTools::FPackageGroupName PGN;
							PGN.GroupName = TEXT("");
							PGN.ObjectName = FPaths::GetBaseFilename(DestPackage);
							PGN.PackageName = DestPackage;
							const bool bShouldPromptForDestinationConflict = !bCopyOverAllDestinationOverlaps;
							TMap<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>> DuplicatedObjects;
							FName PackageFName(*PackageName);


							// Temp fix for some codepaths that allows advanced copy of world packages. For partitioned worlds, this can only be supported for worlds with
							// streaming disabled and this code should be removed once the callers switch to the same codepath as editor save as.
							if (UWorld* World = Cast<UWorld>(ExistingObject))
							{
								if (UWorldPartition* WorldPartition = World->GetWorldPartition())
								{
									if (!WorldPartition->IsInitialized())
									{
										WorldPartition->Initialize(World, FTransform::Identity);
									}
									CopiedWorldPartitionMaps.Add(PackageFName);
								}
							}

							if (UObject* NewObject = ObjectTools::DuplicateSingleObject(ExistingObject, PGN, ObjectsUserRefusedToFullyLoad, bShouldPromptForDestinationConflict, &DuplicatedObjects))
							{
								ExistingObjectSet.Add(ExistingObject);
								NewObjectSet.Add(NewObject);
								DuplicatedObjectsForEachPackage.Add(MoveTemp(DuplicatedObjects));
								SuccessfullyCopiedSourcePackages.Add(PackageFName);
								SuccessfullyCopiedDestinationFiles.Add(DestPackage);
								SuccessfullyCopiedDestinationPackages.Add(NewObject->GetPackage());
							}
						}
					}
				}
			}
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		TSet<UObject*> ObjectsAndSubObjectsToReplaceWithin;
		ObjectTools::GatherSubObjectsForReferenceReplacement(NewObjectSet, ExistingObjectSet, ObjectsAndSubObjectsToReplaceWithin);

		LoopProgress.Reset(); // Note: Reset first as FScopedSlowTask asserts about out-of-order scoping if assigning over an existing FScopedSlowTask
		LoopProgress = MakeUnique<FScopedSlowTask>(static_cast<float>(SuccessfullyCopiedSourcePackages.Num()), LOCTEXT("AdvancedCopyPackages.ReplacingAssetReferences", "Replacing Asset References..."));
		LoopProgress->MakeDialog();

		TMap<UObject*, TArray<UObject*, TInlineAllocator<1>>> Consolidations;
		TArray<ObjectTools::FReplaceRequest> Requests;
		auto ConsolidatePendingObjects = [&Consolidations, &Requests, &ObjectsAndSubObjectsToReplaceWithin, &ExistingObjectSet]()
		{
			check(Requests.Num() == 0);

			if (Consolidations.Num() > 0)
			{
				Requests.Reserve(Consolidations.Num());
				for (TPair<UObject*, TArray<UObject*, TInlineAllocator<1>>>&Consolidation : Consolidations)
				{
					Requests.Add(ObjectTools::FReplaceRequest{ Consolidation.Key, Consolidation.Value });
				}
				ObjectTools::ConsolidateObjects(Requests, ObjectsAndSubObjectsToReplaceWithin, ExistingObjectSet, false);

				Consolidations.Reset();
				Requests.Reset();
			}
		};

		TArray<FName> Dependencies;
		for (FName SuccessfullyCopiedPackage : SuccessfullyCopiedSourcePackages)
		{
			LoopProgress->EnterProgressFrame();

			Dependencies.Reset();
			AssetRegistryModule.Get().GetDependencies(SuccessfullyCopiedPackage, Dependencies);

			// Temp fix for some codepaths that allows advanced copy of world packages.
			// if the map is a world partition map, add dependencies of the actor packages as well
			if (CopiedWorldPartitionMaps.Contains(SuccessfullyCopiedPackage))
			{
				TArray<FName> ExternalObjectsPaths;
				Algo::Transform(ULevel::GetExternalObjectsPaths(SuccessfullyCopiedPackage.ToString()), ExternalObjectsPaths, [](const FString& Path) { return FName(*Path); });
				TArray<FAssetData> ExternalActors;
				AssetRegistryModule.Get().GetAssetsByPaths(ExternalObjectsPaths, ExternalActors, true, true);
				for (const FAssetData& AssetData : ExternalActors)
				{
					AssetRegistryModule.Get().GetDependencies(AssetData.PackageName, Dependencies);
				}
			}

			for (FName Dependency : Dependencies)
			{
				const int32 DependencyIndex = SuccessfullyCopiedSourcePackages.IndexOfByKey(Dependency);
				if (DependencyIndex != INDEX_NONE)
				{
					Consolidations.Reserve(Consolidations.Num() + DuplicatedObjectsForEachPackage[DependencyIndex].Num());
					for (const TPair<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>>& Duplication : DuplicatedObjectsForEachPackage[DependencyIndex])
					{
						UObject* SourceObject = Duplication.Key.Get();
						UObject* NewObject = Duplication.Value.Get();
						if (SourceObject && NewObject)
						{
							Consolidations.FindOrAdd(NewObject).AddUnique(SourceObject);
						}
					}
				}
			}
		}

		LoopProgress.Reset();

		ConsolidatePendingObjects();
		check(Consolidations.Num() == 0);

		ObjectTools::CompileBlueprintsAfterRefUpdate(NewObjectSet.Array());

		// The default value for save packages is true if SCC is enabled because the user can use SCC to revert a change
		// The save needs to happen before FMarkForAdd is called on the files.
		bool bSavePackages = ISourceControlModule::Get().IsEnabled() || bForceAutosave;
		if (bSavePackages)
		{
			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			FEditorFileUtils::PromptForCheckoutAndSave(SuccessfullyCopiedDestinationPackages, bCheckDirty, bPromptToSave);
		}

		FString SourceControlErrors;

		if (SuccessfullyCopiedDestinationFiles.Num() > 0)
		{
			// attempt to add files to source control (this can quite easily fail, but if it works it is very useful)
			if (GetDefault<UEditorLoadingSavingSettings>()->bSCCAutoAddNewFiles)
			{
				if (ISourceControlModule::Get().IsEnabled())
				{
					ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
					if (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), SourceControlHelpers::PackageFilenames(SuccessfullyCopiedDestinationFiles)) == ECommandResult::Failed)
					{
						for (auto FileIt(SuccessfullyCopiedDestinationFiles.CreateConstIterator()); FileIt; FileIt++)
						{
							if (!SourceControlProvider.GetState(*FileIt, EStateCacheUsage::Use)->IsAdded())
							{
								SourceControlErrors += FText::Format(LOCTEXT("AdvancedCopyPackages_SourceControlError", "{0} could not be added to revision control"), FText::FromString(*FileIt)).ToString();
								SourceControlErrors += LINE_TERMINATOR;
							}
						}
					}
				}
			}
		}

		for (FName CopiedWorldPartitionMap : CopiedWorldPartitionMaps)
		{
			UPackage* CopiedWorldPartitionPackage = FindPackage(nullptr, *CopiedWorldPartitionMap.ToString());
			UWorld* World = UWorld::FindWorldInPackage(CopiedWorldPartitionPackage);
			UWorldPartition* WorldPartition = World->GetWorldPartition();
			WorldPartition->Uninitialize();
		}

		FMessageLog AdvancedCopyLog("AssetTools");
		FText LogMessage = FText::FromString(TEXT("Advanced content copy completed successfully!"));
		EMessageSeverity::Type Severity = EMessageSeverity::Info;
		if (SourceControlErrors.Len() > 0)
		{
			Severity = EMessageSeverity::Error;

			AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_SourceControlErrorsListPage", "Revision Control Errors"));
			AdvancedCopyLog.Error(FText::FromString(*SourceControlErrors));

			FString ErrorMessage = LOCTEXT("AdvancedCopyPackages_SourceControlErrorsList", "Some files reported revision control errors.").ToString();
			
			if (SuccessfullyCopiedSourcePackages.Num() > 0)
			{
				AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_CopyErrorsSuccesslistPage", "Copied Successfully"));
				for (auto FileIt = SuccessfullyCopiedSourcePackages.CreateConstIterator(); FileIt; ++FileIt)
				{
					if (!FileIt->IsNone())
					{
						AdvancedCopyLog.Info(FText::FromName(*FileIt));
					}
				}

				ErrorMessage += LINE_TERMINATOR;
				ErrorMessage += LOCTEXT("AdvancedCopyPackages_CopyErrorsSuccesslist", "Some files were copied successfully.").ToString();
			}
			LogMessage = FText::FromString(ErrorMessage);
		}
		else
		{
			AdvancedCopyLog.NewPage(LOCTEXT("AdvancedCopyPackages_CompletePage", "Advanced content copy completed successfully!"));
			for (auto FileIt = SuccessfullyCopiedSourcePackages.CreateConstIterator(); FileIt; ++FileIt)
			{
				if (!FileIt->IsNone())
				{
					AdvancedCopyLog.Info(FText::FromName(*FileIt));
				}
			}
		}
		// @note Using the bForce param because the InSeverityFilter param is only checked against logs in the last page
		AdvancedCopyLog.Notify(LogMessage, /*InSeverityFilter=*/EMessageSeverity::Error, /*bForce=*/(Severity <= NotificationSeverityFilter));
		return true;
	}
	return false;
}

bool UAssetToolsImpl::AdvancedCopyPackages(const FAdvancedCopyParams& CopyParams, const TArray<TMap<FString, FString>>& PackagesAndDestinations) const
{
	bool bResult = false;
	FDuplicatedObjects DuplicatedObjects;
	
	TMap<FString, FString> FlattenedDestinationMap;
	if (FlattenAdvancedCopyDestinations(PackagesAndDestinations, FlattenedDestinationMap))
	{
		bResult = AdvancedCopyPackages(FlattenedDestinationMap, CopyParams.bShouldForceSave, CopyParams.bCopyOverAllDestinationOverlaps, &DuplicatedObjects, EMessageSeverity::Info);
	}

	if (CopyParams.OnCopyComplete.IsBound())
	{
		TArray<FAssetRenameData> AllCopiedAssets;
		for (const TMap<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>>& PerPackageCopies : DuplicatedObjects)
		{
			for (const TPair<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>>& AssetSourceAndCopy : PerPackageCopies)
			{
				const TSoftObjectPtr<UObject>& CopiedAsset = AssetSourceAndCopy.Value;
				FAssetRenameData& AssetRenameData = AllCopiedAssets.Emplace_GetRef(TWeakObjectPtr(CopiedAsset.Get()), CopiedAsset.GetLongPackageName(), CopiedAsset.GetAssetName());
				AssetRenameData.OldObjectPath = AssetSourceAndCopy.Key.ToSoftObjectPath();
				AssetRenameData.NewObjectPath = CopiedAsset.ToSoftObjectPath();
			}
		}
		CopyParams.OnCopyComplete.Execute(bResult, AllCopiedAssets);
	}
	
	return bResult;
}

bool UAssetToolsImpl::IsDiscoveringAssetsInProgress() const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	return AssetRegistryModule.Get().IsLoadingAssets();
}

void UAssetToolsImpl::OpenDiscoveringAssetsDialog(const FOnAssetsDiscovered& InOnAssetsDiscovered)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		SDiscoveringAssetsDialog::OpenDiscoveringAssetsDialog(InOnAssetsDiscovered);
	}
}

bool UAssetToolsImpl::RenameAssets(const TArray<FAssetRenameData>& AssetsAndNames)
{
	return AssetRenameManager->RenameAssets(AssetsAndNames);
}

EAssetRenameResult UAssetToolsImpl::RenameAssetsWithDialog(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout)
{
	return AssetRenameManager->RenameAssetsWithDialog(AssetsAndNames, bAutoCheckout);
}

void UAssetToolsImpl::FindSoftReferencesToObject(FSoftObjectPath TargetObject, TArray<UObject*>& ReferencingObjects)
{
	AssetRenameManager->FindSoftReferencesToObject(TargetObject, ReferencingObjects);
}

void UAssetToolsImpl::FindSoftReferencesToObjects(const TArray<FSoftObjectPath>& TargetObjects, TMap<FSoftObjectPath, TArray<UObject*>>& ReferencingObjects)
{
	AssetRenameManager->FindSoftReferencesToObjects(TargetObjects, ReferencingObjects);
}

void UAssetToolsImpl::RenameReferencingSoftObjectPaths(const TArray<UPackage *> PackagesToCheck, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap)
{
	AssetRenameManager->RenameReferencingSoftObjectPaths(PackagesToCheck, AssetRedirectorMap);
}

TArray<UObject*> UAssetToolsImpl::ImportAssetsWithDialog(const FString& DestinationPath)
{
	const bool bAllowAsyncImport = false;
	return ImportAssetsWithDialogImplementation(DestinationPath, bAllowAsyncImport);
}

void UAssetToolsImpl::ImportAssetsWithDialogAsync(const FString& DestinationPath)
{
	const bool bAllowAsyncImport = true;
	ImportAssetsWithDialogImplementation(DestinationPath, bAllowAsyncImport);
}

TArray<UObject*> UAssetToolsImpl::ImportAssetsAutomated(const UAutomatedAssetImportData* ImportData)
{
	check(ImportData);

	FAssetImportParams Params;

	Params.bAutomated = true;
	Params.bForceOverrideExisting = ImportData->bReplaceExisting;
	Params.bSyncToBrowser = false;
	Params.SpecifiedFactory = TStrongObjectPtr<UFactory>(ImportData->Factory);
	Params.ImportData = ImportData;

	return ImportAssetsInternal(ImportData->Filenames, ImportData->DestinationPath, nullptr, Params);
}

void UAssetToolsImpl::ImportAssetTasks(const TArray<UAssetImportTask*>& ImportTasks)
{
	FScopedSlowTask SlowTask(static_cast<float>(ImportTasks.Num()), LOCTEXT("ImportSlowTask", "Importing"));
	SlowTask.MakeDialog();

	FAssetImportParams Params;
	Params.bSyncToBrowser = false;

	TArray<FString> Filenames;
	Filenames.Add(TEXT(""));
	TArray<UPackage*> PackagesToSave;
	for (UAssetImportTask* ImportTask : ImportTasks)
	{
		if (!ImportTask)
		{
			UE_LOGF(LogAssetTools, Warning, "ImportAssetTasks() supplied an empty task");
			continue;
		}

		SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("Import_ImportingFile", "Importing \"{0}\"..."), FText::FromString(FPaths::GetBaseFilename(ImportTask->Filename))));

		Params.AssetImportTask = ImportTask;
		Params.bAllowAsyncImport = ImportTask->bAsync;
		Params.bForceOverrideExisting = ImportTask->bReplaceExisting;
		Params.bAutomated = ImportTask->bAutomated;
		Params.SpecifiedFactory = TStrongObjectPtr<UFactory>(ImportTask->Factory);
		Filenames[0] = ImportTask->Filename;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ImportTask->Result = ImportAssetsInternal(Filenames, ImportTask->DestinationPath, nullptr, Params);

		PackagesToSave.Reset(ImportTask->Result.Num());
		for (UObject* Object : ImportTask->Result)
		{
			ImportTask->ImportedObjectPaths.Add(Object->GetPathName());
			if (ImportTask->bSave)
			{
				PackagesToSave.AddUnique(Object->GetOutermost());
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// If this wasn't an async import through Interchange (and hence does not have valid AsyncResults),
		// save imported packages here if required.
		if (!ImportTask->AsyncResults.IsValid() && ImportTask->bSave)
		{
			UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true);
		}
	}
}

void UAssetToolsImpl::ExportAssets(const TArray<FString>& AssetsToExport, const FString& ExportPath)
{
	TArray<UObject*> AssetObjectsToExport;
	AssetObjectsToExport.Reserve(AssetsToExport.Num());

	for (const FString& AssetStr : AssetsToExport)
	{
		UObject* Asset = LoadObject<UObject>(nullptr, *AssetStr);
		if (Asset)
		{
			AssetObjectsToExport.Add(Asset);
		}
		else
		{
			UE_LOGF(LogAssetTools, Error, "Could not load asset '%ls' to export it", *AssetStr);
		}
	}

	const bool bPromptIndividualFilenames = false;
	const bool bExportAsCleanFileName = false;
	ExportAssetsInternal(AssetObjectsToExport, bPromptIndividualFilenames, bExportAsCleanFileName, ExportPath);
}

void UAssetToolsImpl::ExportAssets(const TArray<UObject*>& AssetsToExport, const FString& ExportPath) const
{
	const bool bPromptIndividualFilenames = false;
	const bool bSaveAsCleanFilename = false;
	ExportAssetsInternal(AssetsToExport, bPromptIndividualFilenames, bSaveAsCleanFilename, ExportPath);
}

void UAssetToolsImpl::ExportAssetsWithCleanFilename(const TArray<UObject*>& AssetsToExport, const FString& ExportPath) const
{
	const bool bPromptIndividualFilenames = false;
	const bool bSaveAsCleanFilename = true;
	ExportAssetsInternal(AssetsToExport, bPromptIndividualFilenames, bSaveAsCleanFilename, ExportPath);
}
 
void UAssetToolsImpl::ExportAssetsWithDialog(const TArray<UObject*>& AssetsToExport, bool bPromptForIndividualFilenames)
{
	const bool bSaveAsCleanFilename = false;
	ExportAssetsInternal(AssetsToExport, bPromptForIndividualFilenames, bSaveAsCleanFilename, TEXT(""));
}

void UAssetToolsImpl::ExportAssetsWithDialog(const TArray<FString>& AssetsToExport, bool bPromptForIndividualFilenames)
{
	TArray<UObject*> AssetObjectsToExport;
	AssetObjectsToExport.Reserve(AssetsToExport.Num());

	for (const FString& AssetStr : AssetsToExport)
	{
		UObject* Asset = LoadObject<UObject>(nullptr, *AssetStr);
		if (Asset)
		{
			AssetObjectsToExport.Add(Asset);
		}
		else
		{
			UE_LOGF(LogAssetTools, Error, "Could not load asset '%ls' to export it", *AssetStr);
		}
	}

	const bool bSaveAsCleanFilename = false;
	ExportAssetsInternal(AssetObjectsToExport, bPromptForIndividualFilenames, bSaveAsCleanFilename, TEXT(""));
}

void UAssetToolsImpl::ExpandDirectories(const TArray<FString>& Files, const FString& DestinationPath, TArray<TPair<FString, FString>>& FilesAndDestinations) const
{
	// Iterate through all files in the list, if any folders are found, recurse and expand them.
	for ( int32 FileIdx = 0; FileIdx < Files.Num(); ++FileIdx )
	{
		const FString& Filename = Files[FileIdx];

		// If the file being imported is a directory, just include all sub-files and skip the directory.
		if ( IFileManager::Get().DirectoryExists(*Filename) )
		{
			FString FolderName = FPaths::GetCleanFilename(Filename);

			// Get all files & folders in the folder.
			FString SearchPath = Filename / FString(TEXT("*"));
			TArray<FString> SubFiles;
			IFileManager::Get().FindFiles(SubFiles, *SearchPath, true, true);

			// FindFiles just returns file and directory names, so we need to tack on the root path to get the full path.
			TArray<FString> FullPathItems;
			for ( FString& SubFile : SubFiles )
			{
				FullPathItems.Add(Filename / SubFile);
			}

			// Expand any sub directories found.
			FString NewSubDestination = DestinationPath / FolderName;
			ExpandDirectories(FullPathItems, NewSubDestination, FilesAndDestinations);
		}
		else
		{
			// Add any files and their destination path.
			FilesAndDestinations.Emplace(Filename, DestinationPath);
		}
	}
}
TArray<UObject*> UAssetToolsImpl::ImportAssets(const TArray<FString>& Files, const FString& DestinationPath, UFactory* ChosenFactory, bool bSyncToBrowser /* = true */, TArray<TPair<FString, FString>>* FilesAndDestinations /* = nullptr */, bool bAllowAsyncImport /* = false */, bool bSceneImport /*= false*/) const
{
	const bool bForceOverrideExisting = false;

	FAssetImportParams Params;

	Params.bAutomated = false;
	Params.bForceOverrideExisting = false;
	Params.bSyncToBrowser = bSyncToBrowser;
	Params.SpecifiedFactory = TStrongObjectPtr<UFactory>(ChosenFactory);
	Params.bAllowAsyncImport = bAllowAsyncImport;
	Params.bSceneImport = bSceneImport;

	return ImportAssetsInternal(Files, DestinationPath, FilesAndDestinations, Params);
}

void UAssetToolsImpl::CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName)
{
	const FString SanitizedBasePackageName = UPackageTools::SanitizePackageName(InBasePackageName);

	const FString PackagePath = FPackageName::GetLongPackagePath(SanitizedBasePackageName);
	const FString BaseAssetNameWithSuffix = FPackageName::GetLongPackageAssetName(SanitizedBasePackageName) + InSuffix;
	const FString SanitizedBaseAssetName = ObjectTools::SanitizeObjectName(BaseAssetNameWithSuffix);

	int32 IntSuffix = 0;
	bool bObjectExists = false;

	int32 CharIndex = SanitizedBaseAssetName.Len() - 1;
	while (CharIndex >= 0 && SanitizedBaseAssetName[CharIndex] >= TEXT('0') && SanitizedBaseAssetName[CharIndex] <= TEXT('9'))
	{
		--CharIndex;
	}
	FString TrailingInteger;
	FString TrimmedBaseAssetName = SanitizedBaseAssetName;
	if (SanitizedBaseAssetName.Len() > 0 && CharIndex == -1)
	{
		// This is the all numeric name, in this case we'd like to append _number, because just adding a number isn't great
		TrimmedBaseAssetName += TEXT("_");
		IntSuffix = 2;
	}
	if (CharIndex >= 0 && CharIndex < SanitizedBaseAssetName.Len() - 1)
	{
		TrailingInteger = SanitizedBaseAssetName.RightChop(CharIndex + 1);
		TrimmedBaseAssetName = SanitizedBaseAssetName.Left(CharIndex + 1);
		IntSuffix = FCString::Atoi(*TrailingInteger);
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	do
	{
		bObjectExists = false;
		if ( IntSuffix < 1 )
		{
			OutAssetName = SanitizedBaseAssetName;
		}
		else
		{
			FString Suffix = FString::Printf(TEXT("%d"), IntSuffix);
			while (Suffix.Len() < TrailingInteger.Len())
			{
				Suffix = TEXT("0") + Suffix;
			}
			OutAssetName = FString::Printf(TEXT("%s%s"), *TrimmedBaseAssetName, *Suffix);
		}
	
		OutPackageName = PackagePath + TEXT("/") + OutAssetName;
		FString ObjectPath = OutPackageName + TEXT(".") + OutAssetName;

		// Use the asset registry if possible to find existing assets without loading them
		if ( !AssetRegistryModule.Get().IsLoadingAssets() )
		{
			FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
			if(AssetData.IsValid())
			{
				bObjectExists = true;
			}
		}
		else
		{
			bObjectExists = LoadObject<UObject>(nullptr, *ObjectPath, nullptr, LOAD_NoWarn | LOAD_NoRedirects) != nullptr;
		}
		IntSuffix++;
	}
	while (bObjectExists != false);
}

bool UAssetToolsImpl::AssetUsesGenericThumbnail( const FAssetData& AssetData ) const
{
	if ( !AssetData.IsValid() )
	{
		// Invalid asset, assume it does not use a shared thumbnail
		return false;
	}

	if( AssetData.IsAssetLoaded() )
	{
		// Loaded asset, see if there is a rendering info for it
		UObject* Asset = AssetData.GetAsset();
		FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo( Asset );
		return !RenderInfo || !RenderInfo->Renderer;
	}

	if ( AssetData.AssetClassPath == UBlueprint::StaticClass()->GetClassPathName() )
	{
		// Unloaded blueprint asset
		// It would be more correct here to find the rendering info for the generated class,
		// but instead we are simply seeing if there is a thumbnail saved on disk for this asset
		FString PackageFilename;
		if ( FPackageName::DoesPackageExist(AssetData.PackageName.ToString(), &PackageFilename) )
		{
			TSet<FName> ObjectFullNames;
			FThumbnailMap ThumbnailMap;

			FName ObjectFullName = FName(*AssetData.GetFullName());
			ObjectFullNames.Add(ObjectFullName);

			ThumbnailTools::LoadThumbnailsFromPackage(PackageFilename, ObjectFullNames, ThumbnailMap);

			FObjectThumbnail* ThumbnailPtr = ThumbnailMap.Find(ObjectFullName);
			if (ThumbnailPtr)
			{
				return (*ThumbnailPtr).IsEmpty();
			}

			return true;
		}
	}
	else
	{
		// Unloaded non-blueprint asset. See if the class has a rendering info.
		UClass* Class = FindObject<UClass>(AssetData.AssetClassPath);

		UObject* ClassCDO = nullptr;
		if (Class != nullptr)
		{
			ClassCDO = Class->GetDefaultObject();
		}

		// Get the rendering info for this object
		FThumbnailRenderingInfo* RenderInfo = nullptr;
		if (ClassCDO != nullptr)
		{
			RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo( ClassCDO );
		}

		return !RenderInfo || !RenderInfo->Renderer;
	}

	return false;
}

void UAssetToolsImpl::DiffAgainstDepot( UObject* InObject, const FString& InPackagePath, const FString& InPackageName ) const
{
	check( InObject );

	// Make sure our history is up to date
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	SourceControlProvider.Execute(UpdateStatusOperation, SourceControlHelpers::PackageFilename(InPackagePath));

	// Get the SCC state
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SourceControlHelpers::PackageFilename(InPackagePath), EStateCacheUsage::Use);

	// If we have an asset and its in SCC..
	if( SourceControlState.IsValid() && InObject != nullptr && SourceControlState->IsSourceControlled() )
	{
		// Get the file name of package
		FString RelativeFileName;
		if(FPackageName::DoesPackageExist(InPackagePath, &RelativeFileName))
		{
			if(SourceControlState->GetHistorySize() > 0)
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(0);
				check(Revision.IsValid());

				// Get the head revision of this package from source control
				FString AbsoluteFileName = FPaths::ConvertRelativePathToFull(RelativeFileName);
				FString TempFileName;
				if(UPackage* TempPackage = DiffUtils::LoadPackageForDiff(Revision))
				{
					// Grab the old asset from that old package
					UObject* OldObject = FindObject<UObject>(TempPackage, *InPackageName);

					// Recovery for package names that don't match
					if (OldObject == nullptr)
					{
						OldObject = TempPackage->FindAssetInPackage();
					}

					if(OldObject != nullptr)
					{
						/* Set the revision information*/
						FRevisionInfo OldRevision;
						OldRevision.Changelist = Revision->GetCheckInIdentifier();
						OldRevision.Date = Revision->GetDate();
						OldRevision.Revision = Revision->GetRevision();

						FRevisionInfo NewRevision; 
						NewRevision.Revision = TEXT("");
						DiffAssets(OldObject, InObject, OldRevision, NewRevision);
					}
				}
			}
		} 
	}
}

void UAssetToolsImpl::DiffAssets(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const
{
	if (OldAsset == nullptr && NewAsset == nullptr)
	{
		UE_LOGF(LogAssetTools, Warning, "DiffAssets: Both of the supplied assets were nullptr.");
		return;
	}

	// Get class of both assets
	const auto AssetClassFallback = [NewAsset, OldAsset]()->UClass*
	{
		if (OldAsset)
		{
			return OldAsset->GetClass();
		}
		if (NewAsset)
		{
			return NewAsset->GetClass();
		}
		check(false); // this should never happen
		return nullptr;
	};
	
	const UClass* OldClass = OldAsset ? OldAsset->GetClass() : AssetClassFallback();
	const UClass* NewClass = NewAsset ? NewAsset->GetClass() : AssetClassFallback();
	
	// If same class..
	if(OldClass == NewClass)
	{
		// Get class-specific actions
		TWeakPtr<IAssetTypeActions> Actions = GetAssetTypeActionsForClass( NewClass );
		if(Actions.IsValid())
		{
			// And use that to perform the Diff
			Actions.Pin()->PerformAssetDiff(OldAsset, NewAsset, OldRevision, NewRevision);
		}
	}
	else
	{
		UE_LOGF(LogAssetTools, Warning, "DiffAssets: Classes were not the same.");
	}
}

FString UAssetToolsImpl::DumpAssetToTempFile(UObject* Asset) const
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	// Export asset to archive
	if (Asset)
	{
		UExporter::ExportToOutputDevice(&Context, Asset, nullptr,
			Archive, TEXT("copy"), 0,
			PPF_ExportsNotFullyQualified|PPF_Copy|PPF_Delimited|PPF_ForDiff,
			false, Asset->GetOuter());
	}
	// Used to generate unique file names during a run
	static int TempFileNum = 0;

	// Build name for temp text file
	FString AssetName = Asset? Asset->GetName() : TEXT("empty");
	FString RelTempFileName = FString::Printf(TEXT("%sText%s-%d.txt"), *FPaths::DiffDir(), *AssetName, TempFileNum++);
	FString AbsoluteTempFileName = FPaths::ConvertRelativePathToFull(RelTempFileName);

	// Save text into temp file
	if( !FFileHelper::SaveStringToFile( Archive, *AbsoluteTempFileName ) )
	{
		//UE_LOGF(LogAssetTools, Warning, "DiffAssets: Could not write %ls", *AbsoluteTempFileName);
		return TEXT("");
	}
	else
	{
		return AbsoluteTempFileName;
	}
}

FString WrapArgument(const FString& Argument)
{
	// Wrap the passed in argument so it changes from Argument to "Argument"
	return FString::Printf(TEXT("%s%s%s"),	(Argument.StartsWith("\"")) ? TEXT(""): TEXT("\""),
											*Argument,
											(Argument.EndsWith("\"")) ? TEXT(""): TEXT("\""));
}

bool UAssetToolsImpl::CreateDiffProcess(const FString& DiffCommand,  const FString& OldTextFilename,  const FString& NewTextFilename, const FString& DiffArgs) const
{
	// Construct Arguments
	FString Arguments = FString::Printf( TEXT("%s %s %s"),*WrapArgument(OldTextFilename), *WrapArgument(NewTextFilename), *DiffArgs );

	bool bTryRunDiff = true;
	FString NewDiffCommand = DiffCommand;

	while (bTryRunDiff)
	{
		// Fire process
		if (FPlatformProcess::CreateProc(*NewDiffCommand, *Arguments, true, false, false, nullptr, 0, nullptr, nullptr).IsValid())
		{
			return true;
		}
		else
		{
			const FText Message = FText::Format(NSLOCTEXT("AssetTools", "DiffFail", "The currently set diff tool '{0}' could not be run. Would you like to set a new diff tool?"), FText::FromString(DiffCommand));
			EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo, Message);
			if (Response == EAppReturnType::No)
			{
				bTryRunDiff = false;
			}
			else
			{
				IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
				check(DesktopPlatform);

				const FText FileFilterType = NSLOCTEXT("AssetTools", "Executables", "Executables");
#if PLATFORM_WINDOWS
				const FString FileFilterText = FString::Printf(TEXT("%s (*.exe)|*.exe"), *FileFilterType.ToString());
#elif PLATFORM_MAC
				const FString FileFilterText = FString::Printf(TEXT("%s (*.app)|*.app"), *FileFilterType.ToString());
#else
				const FString FileFilterText = FString::Printf(TEXT("%s"), *FileFilterType.ToString());
#endif

				TArray<FString> OutFiles;
				if (DesktopPlatform->OpenFileDialog(
					nullptr,
					NSLOCTEXT("AssetTools", "ChooseDiffTool", "Choose Diff Tool").ToString(),
					TEXT(""),
					TEXT(""),
					FileFilterText,
					EFileDialogFlags::None,
					OutFiles))
				{
					UEditorLoadingSavingSettings& Settings = *GetMutableDefault<UEditorLoadingSavingSettings>();
					Settings.TextDiffToolPath.FilePath = OutFiles[0];
					Settings.SaveConfig();
					NewDiffCommand = OutFiles[0];
				}
			}
		}
	}

	return false;
}

void UAssetToolsImpl::MigratePackages(const TArray<FName>& PackageNamesToMigrate) const
{
	FMigrationOptions Options = FMigrationOptions();
	Options.bPrompt = true;
	MigratePackages(PackageNamesToMigrate, FString(), Options);
}

void UAssetToolsImpl::MigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& DestinationPath, const struct FMigrationOptions& Options) const
{
	MigratePackages(PackageNamesToMigrate, DestinationPath, TSet<FName>(), TSet<FName>(), Options);
}

void UAssetToolsImpl::MigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& DestinationPath, TSet<FName> AllDependenciesToMigrate, TSet<FName> ExcludedDependencies, const struct FMigrationOptions& Options) const
{
	// Packages must be saved for the migration to work
	const bool bPromptUserToSave = CanShowDialogs() && Options.bPrompt;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages))
	{
		auto RunMigratePackages = [this, PackageNamesToMigrate, DestinationPath, Options, AllDependenciesToMigrate = MoveTemp(AllDependenciesToMigrate), ExcludedDependencies = MoveTemp(ExcludedDependencies)]() mutable
			{
				bool bSanitizeInputs = true;
				if (AllDependenciesToMigrate.IsEmpty() && ExcludedDependencies.IsEmpty())
				{
					BuildMigrateDependencies(PackageNamesToMigrate, AllDependenciesToMigrate, ExcludedDependencies);
					bSanitizeInputs = false;
				}
				PerformMigratePackages(PackageNamesToMigrate, DestinationPath, Options, AllDependenciesToMigrate, ExcludedDependencies, bSanitizeInputs);
			};
		
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			// Open a dialog asking the user to wait while assets are being discovered
			SDiscoveringAssetsDialog::OpenDiscoveringAssetsDialog(
				SDiscoveringAssetsDialog::FOnAssetsDiscovered::CreateLambda(RunMigratePackages)
			);
		}
		else
		{
			// Assets are already discovered, perform the migration now
			RunMigratePackages();
		}
	}
}

UE::AssetTools::FOnPackageMigration& UAssetToolsImpl::GetOnPackageMigration()
{
	return OnPackageMigration;
}

void UAssetToolsImpl::OnNewImportRecord(UClass* AssetType, const FString& FileExtension, bool bSucceeded, bool bWasCancelled, const FDateTime& StartTime)
{
	// Don't attempt to report usage stats if analytics isn't available
	if(AssetType != nullptr && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attribs;
		Attribs.Add(FAnalyticsEventAttribute(TEXT("AssetType"), AssetType->GetName()));
		Attribs.Add(FAnalyticsEventAttribute(TEXT("FileExtension"), FileExtension));
		Attribs.Add(FAnalyticsEventAttribute(TEXT("Outcome"), bSucceeded ? TEXT("Success") : (bWasCancelled ? TEXT("Cancelled") : TEXT("Failed"))));
		FTimespan TimeTaken = FDateTime::UtcNow() - StartTime;
		Attribs.Add(FAnalyticsEventAttribute(TEXT("TimeTaken.Seconds"), (float)TimeTaken.GetTotalSeconds()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ImportAsset"), Attribs);
	}
}

void UAssetToolsImpl::OnNewCreateRecord(UClass* AssetType, bool bDuplicated)
{
	// Don't attempt to report usage stats if analytics isn't available
	if(AssetType != nullptr && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attribs;
		Attribs.Add(FAnalyticsEventAttribute(TEXT("AssetType"), AssetType->GetName()));
		Attribs.Add(FAnalyticsEventAttribute(TEXT("Duplicated"), bDuplicated? TEXT("Yes") : TEXT("No")));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.CreateAsset"), Attribs);
	}
}

TArray<UObject*> UAssetToolsImpl::ImportAssetsInternal(const TArray<FString>& Files, const FString& RootDestinationPath, TArray<TPair<FString, FString>> *FilesAndDestinationsPtr, const FAssetImportParams& Params) const
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, GIsRunningUnattendedScript || Params.bAutomated);

	UFactory* SpecifiedFactory = Params.SpecifiedFactory.Get();
	const bool bForceOverrideExisting = Params.bForceOverrideExisting;
	const bool bSyncToBrowser = Params.bSyncToBrowser;
	const bool bAutomatedImport = Params.bAutomated || GIsAutomationTesting;

	TArray<UObject*> ReturnObjects;
	TArray<FString> ValidFiles;
	ValidFiles.Reserve(Files.Num());
	for (int32 FileIndex = 0; FileIndex < Files.Num(); ++FileIndex)
	{
		if (!Files[FileIndex].IsEmpty())
		{
			FString InputFile = Files[FileIndex];
			FPaths::NormalizeDirectoryName(InputFile);
			ValidFiles.Add(InputFile);
		}
	}
	TMap< FString, TArray<UFactory*> > ExtensionToFactoriesMap;
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

	// Use Interchange if it's enabled and we weren't asked to use a specific UFactory
	const bool bUseInterchangeFramework = [SpecifiedFactory, &InterchangeManager]()
	{
		return UInterchangeManager::IsInterchangeImportEnabled() && (SpecifiedFactory == nullptr);
	}();

	TArray<TPair<FString, FString>> FilesAndDestinations;
	if (FilesAndDestinationsPtr == nullptr)
	{
		ExpandDirectories(ValidFiles, RootDestinationPath, FilesAndDestinations);
	}
	else
	{
		FilesAndDestinations = (*FilesAndDestinationsPtr);
	}

	{
		FString UnallowedFilesString;
		// Check for non allowed extensions
		FilesAndDestinations.RemoveAll([&UnallowedFilesString, this](const TPair<FString, FString>& InValue)
			{
				FStringView Extension = FPathViews::GetExtension(InValue.Key);
				if (!IsImportExtensionAllowed(Extension))
				{
					if (!UnallowedFilesString.IsEmpty())
					{
						UnallowedFilesString += TEXT("\n");
					}
					UnallowedFilesString += InValue.Key;

					return true;
				}

				return false;
			});

		FText ErrorMsg;
		if (!UnallowedFilesString.IsEmpty())
		{
			ErrorMsg = FText::Format(LOCTEXT("UnsupportedFileImport", "Unsupported file format to import:\n{0}"), FText::FromString(UnallowedFilesString));
		}

		if (!ErrorMsg.IsEmpty())
		{
			AssetViewUtils::ShowErrorNotifcation(ErrorMsg);
		}
	}

	if(SpecifiedFactory == nullptr)
	{
		// First instantiate one factory for each file extension encountered that supports the extension
		// @todo import: gmp: show dialog in case of multiple matching factories
		for(TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			if(!(*ClassIt)->IsChildOf(UFactory::StaticClass()) || ((*ClassIt)->HasAnyClassFlags(CLASS_Abstract)) || (*ClassIt)->IsChildOf(USceneImportFactory::StaticClass()))
			{
				continue;
			}

			UFactory* Factory = Cast<UFactory>((*ClassIt)->GetDefaultObject());

			if(!Factory->bEditorImport)
			{
				continue;
			}

			TArray<FString> FactoryExtensions;
			Factory->GetSupportedFileExtensions(FactoryExtensions);

			for(const TPair<FString, FString>& FileDest : FilesAndDestinations)
			{
				const FString FileExtension = FPaths::GetExtension(FileDest.Key);

				// Case insensitive string compare with supported formats of this factory
				if(FactoryExtensions.Contains(FileExtension))
				{
					TArray<UFactory*>& ExistingFactories = ExtensionToFactoriesMap.FindOrAdd(FileExtension);

					// Do not remap extensions, just reuse the existing UFactory.
					// There may be multiple UFactories, so we will keep track of all of them
					bool bFactoryAlreadyInMap = false;
					for(auto FoundFactoryIt = ExistingFactories.CreateConstIterator(); FoundFactoryIt; ++FoundFactoryIt)
					{
						if((*FoundFactoryIt)->GetClass() == Factory->GetClass())
						{
							bFactoryAlreadyInMap = true;
							break;
						}
					}

					if(!bFactoryAlreadyInMap)
					{
						// We found a factory for this file, it can be imported!
						// Create a new factory of the same class and make sure it doesn't get GCed.
						// The object will be removed from the root set at the end of this function.
						UFactory* NewFactory = NewObject<UFactory>(GetTransientPackage(), Factory->GetClass());
						if(NewFactory->ConfigureProperties())
						{
							NewFactory->AddToRoot();
							ExistingFactories.Add(NewFactory);
						}
					}
				}
			}
		}
	}
	else if(SpecifiedFactory->bEditorImport && !bAutomatedImport) 
	{

		TArray<FString> FactoryExtensions;
		SpecifiedFactory->GetSupportedFileExtensions(FactoryExtensions);

		for(auto FileIt = ValidFiles.CreateConstIterator(); FileIt; ++FileIt)
		{
			const FString FileExtension = FPaths::GetExtension(*FileIt);

			// Case insensitive string compare with supported formats of this factory
			if(!FactoryExtensions.Contains(FileExtension))
			{
				continue;
			}

			TArray<UFactory*>& ExistingFactories = ExtensionToFactoriesMap.FindOrAdd(FileExtension);

			// Do not remap extensions, just reuse the existing UFactory.
			// There may be multiple UFactories, so we will keep track of all of them
			bool bFactoryAlreadyInMap = false;
			for(auto FoundFactoryIt = ExistingFactories.CreateConstIterator(); FoundFactoryIt; ++FoundFactoryIt)
			{
				if((*FoundFactoryIt)->GetClass() == SpecifiedFactory->GetClass())
				{
					bFactoryAlreadyInMap = true;
					break;
				}
			}

			if(!bFactoryAlreadyInMap)
			{
				// We found a factory for this file, it can be imported!
				// Create a new factory of the same class and make sure it doesnt get GCed.
				// The object will be removed from the root set at the end of this function.
				UFactory* NewFactory = NewObject<UFactory>(GetTransientPackage(), SpecifiedFactory->GetClass());
				if(NewFactory->ConfigureProperties())
				{
					NewFactory->AddToRoot();
					ExistingFactories.Add(NewFactory);
				}
			}
		}
	}

	// We need to sort the factories so that they get tested in priority order
	for(TPair<FString, TArray<UFactory*>>& ExtensionToFactories : ExtensionToFactoriesMap)
	{
		ExtensionToFactories.Value.Sort(&UFactory::SortFactoriesByPriority);
	}

	// Some flags to keep track of what the user decided when asked about overwriting or replacing
	bool bOverwriteAll = false;
	bool bReplaceAll = false;
	bool bDontOverwriteAny = false;
	bool bDontReplaceAny = false;
	if (bAutomatedImport)
	{
		bOverwriteAll = bReplaceAll = bForceOverrideExisting;
		bDontOverwriteAny = bDontReplaceAny = !bForceOverrideExisting;
	}

	TArray<UFactory*> UsedFactories;
	bool bImportWasCancelled = false;
	bool bShowSlowTask = !bUseInterchangeFramework;
	bool bOnlyInterchangeImport = bUseInterchangeFramework;
	if(bUseInterchangeFramework)
	{
		for (int32 FileIdx = 0; FileIdx < FilesAndDestinations.Num(); ++FileIdx)
		{
			// Filename will need to get sanitized before we create an asset out of them as they
			// can be created out of sources that contain spaces and other invalid characters. Filename cannot be sanitized
			// until other checks are done that rely on looking at the actual source file so sanitation is delayed.
			const FString& Filename = FilesAndDestinations[FileIdx].Key;
			{
				UE::Interchange::FScopedSourceData ScopedSourceData(Filename);

				if (!InterchangeManager.CanTranslateSourceData(ScopedSourceData.GetSourceData(), Params.bSceneImport))
				{
					bOnlyInterchangeImport = false;
					break;
				}
			}
		}

		if (!bOnlyInterchangeImport)
		{
			if (Files.Num() > 1)
			{	
				//Always allow user to cancel the import task if they are importing multiple files.
				//If we're importing a single file, then the factory policy will dictate if the import if cancelable.
				bShowSlowTask = true;
			}
		}
		else
		{
			//Do not show the slow task if we are only using the Interchange Manager for imports
			bShowSlowTask = false;
		}
	}

	// Create a slow task to document progress
	FScopedSlowTask SlowTask(static_cast<float>(ValidFiles.Num()), LOCTEXT("ImportSlowTask", "Importing"), bShowSlowTask);

	if (bShowSlowTask && ValidFiles.Num() > 1)
	{
		//Always allow user to cancel the import task if they are importing multiple ValidFiles.
		//If we're importing a single file, then the factory policy will dictate if the import if cancelable.
		SlowTask.MakeDialog(true);
	}
	
	struct FInterchangeImportStatus
	{
		explicit FInterchangeImportStatus(int32 NumFiles)
			: InterchangeResultsContainer(NewObject<UInterchangeResultsContainer>(GetTransientPackage())),
			  ImportCount(NumFiles)
		{}

		TStrongObjectPtr<UInterchangeResultsContainer> InterchangeResultsContainer;
		TArray<TWeakObjectPtr<UObject>> ImportedObjects;
		std::atomic<int32> ImportCount;
	};

	TSharedPtr<FInterchangeImportStatus, ESPMode::ThreadSafe> ImportStatus = MakeShared<FInterchangeImportStatus>(FilesAndDestinations.Num());

	// Now iterate over the input files and use the same factory object for each file with the same extension
	for(int32 FileIdx = 0; FileIdx < FilesAndDestinations.Num() && !bImportWasCancelled; ++FileIdx)
	{
		// Filename and DestinationPath will need to get santized before we create an asset out of them as they
		// can be created out of sources that contain spaces and other invalid characters. Filename cannot be sanitized
		// until other checks are done that rely on looking at the actual source file so sanitation is delayed.
		const FString& Filename = FilesAndDestinations[FileIdx].Key;

		FString DestinationPath;
		FString ErrorMsg;
		if (!FPackageName::TryConvertFilenameToLongPackageName(ObjectTools::SanitizeObjectPath(FilesAndDestinations[FileIdx].Value), DestinationPath, &ErrorMsg))
		{
			const FText Message = FText::Format(LOCTEXT("CannotConvertDestinationPath", "Can't import the file '{0}' because the destination path '{1}' cannot be converted to a package path."), FText::FromString(Filename), FText::FromString(DestinationPath));
			if (!bAutomatedImport)
			{
				FMessageDialog::Open(EAppMsgType::Ok, Message);
			}

			UE_LOGF(LogAssetTools, Warning, "%ls", *ErrorMsg);
			UE_LOGF(LogAssetTools, Warning, "%ls", *Message.ToString());

			continue;
		}

		// TryConvertFilenameToLongPackageName doesn't check if the package path is mounted but IsValidPath does it.
		if (!FPackageName::IsValidPath(DestinationPath))
		{
			const FText Message = FText::Format(LOCTEXT("InvalidDestinationPath", "Can't import the file '{0}' because the destination path '{1}' is not a valid package path for the current project."), FText::FromString(Filename), FText::FromString(DestinationPath));
			if (!bAutomatedImport)
			{
				FMessageDialog::Open(EAppMsgType::Ok, Message);
			}

			UE_LOGF(LogAssetTools, Warning, "%ls", *ErrorMsg);
			UE_LOGF(LogAssetTools, Warning, "%ls", *Message.ToString());

			continue;
		}

		if (bUseInterchangeFramework)
		{
			UE::Interchange::FScopedSourceData ScopedSourceData(Filename);

			if (InterchangeManager.CanTranslateSourceData(ScopedSourceData.GetSourceData(), Params.bSceneImport))
			{
				FImportAssetParameters ImportAssetParameters;
				ImportAssetParameters.bIsAutomated = bAutomatedImport;
				ImportAssetParameters.bFollowRedirectors = UE::AssetTools::Private::bFollowRedirectorsWhenImporting;
				ImportAssetParameters.ReimportAsset = nullptr;
				ImportAssetParameters.bReplaceExisting = bForceOverrideExisting;
				if (Params.AssetImportTask && !Params.AssetImportTask->DestinationName.IsEmpty())
				{
					ImportAssetParameters.DestinationName = Params.AssetImportTask->DestinationName;
				}

				TFunction<void(UE::Interchange::FImportResult&)> AppendImportResult =
					// Note: ImportStatus captured by value so that the lambda keeps the shared ptr alive
					[ImportStatus, ImportTask = Params.AssetImportTask](UE::Interchange::FImportResult& Result)
					{
						ImportStatus->InterchangeResultsContainer->Append(Result.GetResults());
						ImportStatus->ImportedObjects.Append(Result.GetImportedObjects());

						// If imported through an AssetImportTask object, and it's set to save after import, do this here.
						if (ImportTask && ImportTask->bSave)
						{
							TArray<UPackage*> PackagesToSave;
							for (UObject* Object : Result.GetImportedObjects())
							{
								PackagesToSave.Add(Object->GetOutermost());
							}

							constexpr bool bDirtyOnly = true;
							UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, bDirtyOnly);
						}
					};
				
				TFunction<void(UE::Interchange::FImportResult&)> AppendAndBroadcastImportResultIfNeeded =
					// Note: ImportStatus captured by value so that the lambda keeps the shared ptr alive
					[bSceneImport = Params.bSceneImport, ImportStatus, AppendImportResult, bSyncToBrowser](UE::Interchange::FImportResult& Result)
					{
						AppendImportResult(Result);

						if (--ImportStatus->ImportCount == 0)
						{
							UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
							InterchangeManager.OnBatchImportComplete.Broadcast(ImportStatus->InterchangeResultsContainer);

							TArray<UObject*> MainAssets;
							if (bSceneImport)
							{
								for (const TWeakObjectPtr<UObject>& WeakObject : ImportStatus->ImportedObjects)
								{
									if (WeakObject.IsValid() && WeakObject->IsA<UInterchangeSceneImportAsset>())
									{
										MainAssets.Add(WeakObject.Get());
										// There should be only one anyway
										break;
									}
								}
							}
							else
							{
								for (const TWeakObjectPtr<UObject>& WeakObject : ImportStatus->ImportedObjects)
								{
									if (WeakObject.IsValid() && WeakObject->IsAsset())
									{
										MainAssets.Add(WeakObject.Get());
									}
								}
							}
							for (UObject* Object : MainAssets)
							{
								if (Object->IsAsset())
								{
									FAssetRegistryModule::AssetCreated(Object);
									GEditor->BroadcastObjectReimported(Object);
								}
							}

							const IConsoleVariable* Var = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.SyncToBrowser")); // Declared in InterchangeEditorModule.cpp

							// Content Browser sync is editor-only behavior and must stay disabled in commandlets.
                            // In non-commandlet contexts, the CVar overrides the passed value when available.
                            const bool bResolvedSyncToBrowser = IsRunningCommandlet() ? false : (Var ? Var->GetBool() : bSyncToBrowser);
							
							if (bResolvedSyncToBrowser)
							{
								UAssetToolsImpl::Get().SyncBrowserToAssets(MainAssets);
							}
						}
					};

				if (Params.bSceneImport || (Params.SpecifiedFactory && Params.SpecifiedFactory->GetClass()->IsChildOf(USceneImportFactory::StaticClass())))
				{
					TPair<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef> InterchangeResults =
						InterchangeManager.ImportSceneAsync(DestinationPath, ScopedSourceData.GetSourceData(), ImportAssetParameters);

					// If we have an ImportTask, fill out the asynchronous results object here so the caller can see when the results are ready
					if ( Params.AssetImportTask)
					{
						 Params.AssetImportTask->AsyncResults = InterchangeResults.Get<0>();
					}

					InterchangeResults.Get<0>()->OnDone(AppendImportResult);;
					InterchangeResults.Get<1>()->OnDone(AppendAndBroadcastImportResultIfNeeded);

					if (!Params.bAllowAsyncImport)
					{
						InterchangeResults.Get<0>()->WaitUntilDone();
						InterchangeResults.Get<1>()->WaitUntilDone();
					}
				}
				else
				{
					if (Params.AssetImportTask)
					{
						//Add the override pipelines if we have a task that specify valid interchange override pipelines
						if (UInterchangePipelineStackOverride* PipelineStackOverride = Cast<UInterchangePipelineStackOverride>(Params.AssetImportTask->Options))
						{
							for (const FSoftObjectPath& OverridePipelinePath : PipelineStackOverride->OverridePipelines)
							{
								ImportAssetParameters.OverridePipelines.Add(OverridePipelinePath);
							}
						}
						else
						{
							InterchangeManager.ConvertImportData(Params.AssetImportTask->Options, ImportAssetParameters);
						}
					}
					UE::Interchange::FAssetImportResultRef InterchangeResult = Params.bAllowAsyncImport
						? (InterchangeManager.ImportAssetAsync(DestinationPath, ScopedSourceData.GetSourceData(), ImportAssetParameters))
						: (InterchangeManager.ImportAssetWithResult(DestinationPath, ScopedSourceData.GetSourceData(), ImportAssetParameters));

					InterchangeResult->OnDone(AppendAndBroadcastImportResultIfNeeded);

					// If we have an ImportTask, fill out the asynchronous results object here so the caller can see when the results are ready
					if ( Params.AssetImportTask)
					{
						 Params.AssetImportTask->AsyncResults = InterchangeResult;
					}
				}

				if (!Params.bAllowAsyncImport)
				{
					ReturnObjects.Reserve(ImportStatus->ImportedObjects.Num());

					for (const TWeakObjectPtr<UObject>& ImportedObject : ImportStatus->ImportedObjects)
					{
						ReturnObjects.Add(ImportedObject.Get());
					}
				}

				//Import done, iterate the next file and destination
				
				//If we do not import only interchange file, update the progress for each interchange task
				if (!bOnlyInterchangeImport)
				{
					SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("Import_ImportingFile", "Importing \"{0}\"..."), FText::FromString(FPaths::GetBaseFilename(Filename))));
				}
				continue;
			}
		}
		FString FileExtension = FPaths::GetExtension(Filename);
		const TArray<UFactory*>* FactoriesPtr = ExtensionToFactoriesMap.Find(FileExtension);
		UFactory* Factory = nullptr;
		SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("Import_ImportingFile", "Importing \"{0}\"..."), FText::FromString(FPaths::GetBaseFilename(Filename))));

		// Assume that for automated import, the user knows exactly what factory to use if it exists
		if(bAutomatedImport && SpecifiedFactory && SpecifiedFactory->FactoryCanImport(Filename))
		{
			Factory = SpecifiedFactory;
		}
		else if(FactoriesPtr)
		{
			const TArray<UFactory*>& Factories = *FactoriesPtr;

			if(Factories.Num() > 0)
			{
				// Handle the potential of multiple factories being found
				//	Factories was previously sorted by ImportPriority
				//  and filtered by file extension
				//  but FactoryCanImport has not been checked yet

				for(auto FactoryIt = Factories.CreateConstIterator(); FactoryIt; ++FactoryIt)
				{
					UFactory* TestFactory = *FactoryIt;
					if(TestFactory->FactoryCanImport(Filename))
					{
						if ( Factory != nullptr )
						{
							// we found a second factory for this type
							if ( Factory->ImportPriority == TestFactory->ImportPriority )
							{
								UE_LOGF(LogAssetTools, Warning, "Two factories registered with same priority : %ls and %ls", *Factory->GetName(), *TestFactory->GetName() );
							}
							break;
						}

						Factory = TestFactory;
						//found one, continue so we can check for multiple importers 
						//break;
					}
				}

				if ( Factory == nullptr )
				{
					// no factories passed FactoryCanImport()
					//  this seems wrong, but it preserves old behavior
					// ??
					
					Factory = Factories[0];
					
					UE_LOGF(LogAssetTools, Warning, "Factory did not pass FactoryCanImport, trying anyway : %ls", *Factory->GetName());
				}
			}
		}
		else
		{
			if(FEngineAnalytics::IsAvailable())
			{
				TArray<FAnalyticsEventAttribute> Attribs;
				Attribs.Add(FAnalyticsEventAttribute(TEXT("FileExtension"), FileExtension));

				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ImportFailed"), Attribs);
			}

			const FText Message = FText::Format(LOCTEXT("ImportFailed_UnknownExtension", "Failed to import '{0}'. Unknown extension '{1}'."), FText::FromString(Filename), FText::FromString(FileExtension));
			FNotificationInfo Info(Message);
			Info.ExpireDuration = 3.0f;
			Info.bUseLargeFont = false;
			Info.bFireAndForget = true;
			Info.bUseSuccessFailIcons = true;
			FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);

			UE_LOGF(LogAssetTools, Warning, "%ls", *Message.ToString());
		}

		if(Factory != nullptr)
		{
			if (FilesAndDestinations.Num() == 1)
			{
				SlowTask.MakeDialog(Factory->CanImportBeCanceled());
			}

			// Reset the 'Do you want to overwrite the existing object?' Yes to All / No to All prompt, to make sure the
			// user gets a chance to select something when the factory is first used during this import
			if (!UsedFactories.Contains(Factory))
			{
				Factory->ResetState();
				UsedFactories.AddUnique(Factory);
			}

			UClass* ImportAssetType = Factory->SupportedClass;
			bool bImportSucceeded = false;
			FDateTime ImportStartTime = FDateTime::UtcNow();

			FString Name;
			if (Params.AssetImportTask && !Params.AssetImportTask->DestinationName.IsEmpty())
			{
				Name = Params.AssetImportTask->DestinationName;
			}
			else
			{
				Name = FPaths::GetBaseFilename(Filename);
			}
			Name = ObjectTools::SanitizeObjectName(Name);

			FString PackageName = ObjectTools::SanitizeInvalidChars(FPaths::Combine(*DestinationPath, *Name), INVALID_LONGPACKAGE_CHARACTERS);

			// We can not create assets that share the name of a map file in the same location
			if(FEditorFileUtils::IsMapPackageAsset(PackageName))
			{
				const FText Message = FText::Format(LOCTEXT("AssetNameInUseByMap", "You can not create an asset named '{0}' because there is already a map file with this name in this folder."), FText::FromString(Name));
				if(!bAutomatedImport)
				{
					FMessageDialog::Open(EAppMsgType::Ok, Message);
				}
				UE_LOGF(LogAssetTools, Warning, "%ls", *Message.ToString());
				OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
				continue;
			}

			UPackage* Pkg = CreatePackage( *PackageName);
			if(!ensure(Pkg))
			{
				// Failed to create the package to hold this asset for some reason
				OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
				continue;
			}

			// Make sure the destination package is loaded
			Pkg->FullyLoad();

			// Check for an existing object
			UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), Pkg, *Name);

			if (UE::AssetTools::Private::bFollowRedirectorsWhenImporting)
			{
				if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(ExistingObject))
				{
					if (Redirector->DestinationObject)
					{
						ExistingObject = Redirector->DestinationObject;
						Pkg = Redirector->DestinationObject->GetPackage();
						if (FPackageName::GetLongPackageAssetName(PackageName) == Name)
						{
							Name = FPackageName::GetLongPackageAssetName(Pkg->GetName());
						}
						PackageName = Pkg->GetName();
					}
				}
			}

			if(ExistingObject != nullptr)
			{
				// If the existing object is one of the imports we've just created we can't replace or overwrite it
				if(ReturnObjects.Contains(ExistingObject))
				{
					if(ImportAssetType == nullptr)
					{
						// The factory probably supports multiple types and cant be determined yet without asking the user or actually loading it
						// We just need to generate an unused name so object should do fine.
						ImportAssetType = UObject::StaticClass();
					}
					// generate a unique name for this import
					Name = MakeUniqueObjectName(Pkg, ImportAssetType, *Name).ToString();
				}
				else
				{
					// If the object is supported by the factory we are using, ask if we want to overwrite the asset
					// Otherwise, prompt to replace the object
					if(Factory->DoesSupportClass(ExistingObject->GetClass()))
					{
						// The factory can overwrite this object, ask if that is okay, unless "Yes To All" or "No To All" was already selected
						EAppReturnType::Type UserResponse;

						if(bForceOverrideExisting || bOverwriteAll || GIsAutomationTesting)
						{
							UserResponse = EAppReturnType::YesAll;
						}
						else if(bDontOverwriteAny)
						{
							UserResponse = EAppReturnType::NoAll;
						}
						else
						{
							UserResponse = FMessageDialog::Open(
								EAppMsgType::YesNoYesAllNoAll,
								FText::Format(LOCTEXT("ImportObjectAlreadyExists_SameClass", "Do you want to overwrite the existing asset?\n\nAn asset already exists at the import location: {0}"), FText::FromString(PackageName)));

							bOverwriteAll = UserResponse == EAppReturnType::YesAll;
							bDontOverwriteAny = UserResponse == EAppReturnType::NoAll;
						}

						const bool bWantOverwrite = UserResponse == EAppReturnType::Yes || UserResponse == EAppReturnType::YesAll;

						if(!bWantOverwrite)
						{
							// User chose not to replace the package
							bImportWasCancelled = true;
							OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
							continue;
						}
					}
					else if(!bAutomatedImport)
					{
						// The factory can't overwrite this asset, ask if we should delete the object then import the new one. Only do this if "Yes To All" or "No To All" was not already selected.
						EAppReturnType::Type UserResponse;

						if(bReplaceAll)
						{
							UserResponse = EAppReturnType::YesAll;
						}
						else if(bDontReplaceAny)
						{
							UserResponse = EAppReturnType::NoAll;
						}
						else
						{
							UserResponse = FMessageDialog::Open(
								EAppMsgType::YesNoYesAllNoAll,
								FText::Format(LOCTEXT("ImportObjectAlreadyExists_DifferentClass", "Do you want to replace the existing asset?\n\nAn asset already exists at the import location: {0}"), FText::FromString(PackageName)));

							bReplaceAll = UserResponse == EAppReturnType::YesAll;
							bDontReplaceAny = UserResponse == EAppReturnType::NoAll;
						}

						const bool bWantReplace = UserResponse == EAppReturnType::Yes || UserResponse == EAppReturnType::YesAll;

						if(bWantReplace)
						{
							// Delete the existing object
							int32 NumObjectsDeleted = 0;
							TArray< UObject* > ObjectsToDelete;
							ObjectsToDelete.Add(ExistingObject);

							// If the user forcefully deletes the package, all sorts of things could become invalidated,
							// the Pkg pointer might be killed even though it was added to the root.
							TWeakObjectPtr<UPackage> WeakPkg(Pkg);

							// Dont let the package get garbage collected (just in case we are deleting the last asset in the package)
							Pkg->AddToRoot();
							NumObjectsDeleted = ObjectTools::DeleteObjects(ObjectsToDelete, /*bShowConfirmation=*/false);

							// If the weak package ptr is still valid, it should then be safe to remove it from the root.
							if(WeakPkg.IsValid())
							{
								Pkg->RemoveFromRoot();
							}

							const FString QualifiedName = PackageName + TEXT(".") + Name;
							FText Reason;
							if(NumObjectsDeleted == 0 || !IsGloballyUniqueObjectName(*QualifiedName, &Reason))
							{
								// Original object couldn't be deleted
								const FText Message = FText::Format(LOCTEXT("ImportDeleteFailed", "Failed to delete '{0}'. The asset is referenced by other content."), FText::FromString(PackageName));
								FMessageDialog::Open(EAppMsgType::Ok, Message);
								UE_LOGF(LogAssetTools, Warning, "%ls", *Message.ToString());
								OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
								continue;
							}
							else
							{
								// succeed, recreate package since it has been deleted
								Pkg = CreatePackage( *PackageName);
								Pkg->MarkAsFullyLoaded();
							}
						}
						else
						{
							// User chose not to replace the package
							bImportWasCancelled = true;
							OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
							continue;
						}
					}
				}
			}

			// Check for a package that was marked for delete in source control
			if(!CheckForDeletedPackage(Pkg))
			{
				OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
				continue;
			}

			Pkg->SetAssetAccessSpecifier(ShouldCreateAssetsAsExternallyReferenceableForPath(PackageName) ? EAssetAccessSpecifier::Public : EAssetAccessSpecifier::Private);

			Factory->SetAutomatedAssetImportData(Params.ImportData);
			Factory->SetAssetImportTask(Params.AssetImportTask);

			ImportAssetType = Factory->ResolveSupportedClass();
			UObject* Result = Factory->ImportObject(ImportAssetType, Pkg, FName(*Name), RF_Public | RF_Standalone | RF_Transactional, Filename, nullptr, bImportWasCancelled);

			Factory->SetAutomatedAssetImportData(nullptr);
			Factory->SetAssetImportTask(nullptr);

			// Do not report any error if the operation was canceled.
			if(!bImportWasCancelled)
			{
				if(Result)
				{
					ReturnObjects.Add(Result);

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(Result);
					GEditor->BroadcastObjectReimported(Result);

					for (UObject* AdditionalResult : Factory->GetAdditionalImportedObjects())
					{
						ReturnObjects.Add(AdditionalResult);
					}

					bImportSucceeded = true;
				}
				else
				{
					const FText Message = FText::Format(LOCTEXT("ImportFailed_Generic", "Failed to import '{0}'. Failed to create asset '{1}'.\nPlease see Output Log for details."), FText::FromString(Filename), FText::FromString(PackageName));
					if(!bAutomatedImport)
					{
						FMessageDialog::Open(EAppMsgType::Ok, Message);
					}
					UE_LOGF(LogAssetTools, Warning, "%ls", *Message.ToString());
				}
			}

			// Refresh the supported class.  Some factories (e.g. FBX) only resolve their type after reading the file
			ImportAssetType = Factory->ResolveSupportedClass();
			OnNewImportRecord(ImportAssetType, FileExtension, bImportSucceeded, bImportWasCancelled, ImportStartTime);
		}
		else
		{
			// A factory or extension was not found. The extension warning is above. If a factory was not found, the user likely canceled a factory configuration dialog.
		}

		bImportWasCancelled |= SlowTask.ShouldCancel();
		if (bImportWasCancelled)
		{
			UE_LOGF(LogAssetTools, Log, "The import task was canceled.");
		}
	}

	// Clean up and remove the factories we created from the root set
	for(TMap<FString, TArray<UFactory*>>::TConstIterator ExtensionIt = ExtensionToFactoriesMap.CreateConstIterator(); ExtensionIt; ++ExtensionIt)
	{
		for(TArray<UFactory*>::TConstIterator FactoryIt = ExtensionIt.Value().CreateConstIterator(); FactoryIt; ++FactoryIt)
		{
			(*FactoryIt)->CleanUp();
			(*FactoryIt)->RemoveFromRoot();
		}
	}

	// Sync content browser to the newly created assets
	if(ReturnObjects.Num() && bSyncToBrowser)
	{
		UAssetToolsImpl::Get().SyncBrowserToAssets(ReturnObjects);
	}

	return ReturnObjects;
}

void UAssetToolsImpl::ExportAssetsInternal(const TArray<UObject*>& ObjectsToExport, bool bPromptIndividualFilenames, bool bSaveAsCleanFilename, const FString& ExportPath) const
{
	FString LastExportPath = !ExportPath.IsEmpty() ? ExportPath : FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT);

	if (ObjectsToExport.Num() == 0)
	{
		return;
	}

	FString SelectedExportPath;
	if (!bPromptIndividualFilenames)
	{
		if (ExportPath.IsEmpty())
		{
			// If not prompting individual files, prompt the user to select a target directory.
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			if (DesktopPlatform)
			{
				FString FolderName;
				const FString Title = NSLOCTEXT("UnrealEd", "ChooseADirectory", "Choose A Directory").ToString();
				const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					Title,
					LastExportPath,
					FolderName
				);

				if (bFolderSelected)
				{
					SelectedExportPath = FolderName;
				}
			}
		}
		else
		{
			SelectedExportPath = ExportPath;
		}

		// Copy off the selected path for future export operations.
		LastExportPath = SelectedExportPath;
	}

	GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "Exporting", "Exporting"), true);

	// Create an array of all available exporters.
	TArray<UExporter*> Exporters;
	ObjectTools::AssembleListOfExporters(Exporters);

	//Array to control the batch mode and the show options for the exporters that will be use by the selected assets
	TArray<UExporter*> UsedExporters;

	// Export the objects.
	bool bAnyObjectMissingSourceData = false;

	// Permission filter based on class
	TSharedPtr<FPathPermissionList> ExportClassPermissionList = GetAssetClassPathPermissionList(EAssetClassAction::ExportAsset);
	const bool bExportClassPermissionListHasFiltering = ExportClassPermissionList && ExportClassPermissionList->HasFiltering();
	// Use TMap as optimization because FPathPermissionList performs slow TArray<FString> search
	TMap<UClass*, bool> ClassPassesFilterResults;

	for (int32 Index = 0; Index < ObjectsToExport.Num(); Index++)
	{
		GWarn->StatusUpdate(Index, ObjectsToExport.Num(), FText::Format(NSLOCTEXT("UnrealEd", "Exportingf", "Exporting ({0} of {1})"), FText::AsNumber(Index), FText::AsNumber(ObjectsToExport.Num())));

		UObject* ObjectToExport = ObjectsToExport[Index];
		if (!ObjectToExport)
		{
			continue;
		}

		if (ObjectToExport->GetOutermost()->HasAnyPackageFlags(PKG_DisallowExport))
		{
			continue;
		}

		// Look at export class permission filter
		if (bExportClassPermissionListHasFiltering)
		{
			// Reuse results as optimization
			const bool* FoundPassesFilterResult = ClassPassesFilterResults.Find(ObjectToExport->GetClass());
			if (FoundPassesFilterResult)
			{
				if (*FoundPassesFilterResult == false)
				{
					continue;
				}
			}
			else
			{
				FNameBuilder AssetClassPath;
				ObjectToExport->GetClass()->GetPathName(nullptr, AssetClassPath);
				const bool bPassesFilterResult = ExportClassPermissionList->PassesFilter(AssetClassPath.ToView());

				// Save result to be reused as optimization
				ClassPassesFilterResults.Add(ObjectToExport->GetClass(), bPassesFilterResult);

				if (bPassesFilterResult == false)
				{
					continue;
				}
			}
		}

		// Find all the exporters that can export this type of object and construct an export file dialog.
		TArray<FString> AllFileTypes;
		TArray<FString> AllExtensions;
		TArray<FString> PreferredExtensions;

		// Iterate in reverse so the most relevant file formats are considered first.
		for (int32 ExporterIndex = Exporters.Num() - 1; ExporterIndex >= 0; --ExporterIndex)
		{
			UExporter* Exporter = Exporters[ExporterIndex];
			if (Exporter->SupportedClass)
			{
				const bool bObjectIsSupported = Exporter->SupportsObject(ObjectToExport);
				if (bObjectIsSupported)
				{
					// Get a string representing of the exportable types.
					check(Exporter->FormatExtension.Num() == Exporter->FormatDescription.Num());
					check(Exporter->FormatExtension.IsValidIndex(Exporter->PreferredFormatIndex));
					for (int32 FormatIndex = Exporter->FormatExtension.Num() - 1; FormatIndex >= 0; --FormatIndex)
					{
						const FString& FormatExtension = Exporter->FormatExtension[FormatIndex];
						const FString& FormatDescription = Exporter->FormatDescription[FormatIndex];

						if (FormatIndex == Exporter->PreferredFormatIndex)
						{
							PreferredExtensions.Add(FormatExtension);
						}
						AllFileTypes.Add(FString::Printf(TEXT("%s (*.%s)|*.%s"), *FormatDescription, *FormatExtension, *FormatExtension));
						AllExtensions.Add(FString::Printf(TEXT("*.%s"), *FormatExtension));
					}
				}
			}
		}

		// Skip this object if no exporter found for this resource type.
		if (PreferredExtensions.Num() == 0)
		{
			continue;
		}

		// If FBX is listed, make that the most preferred option
		//   also prioritize PNG and EXR ahead of other image formats
		// @todo provide a general purpose way of prioritizing export formats, don't hard-code FBX here
		const TCHAR * TopPriorityExtensions[] =
		{
			TEXT("FBX"),
			TEXT("PNG"),
			TEXT("EXR")
		};
		const int TopPriorityExtensionsCount = 3;

		for(int TopPriorityExtensionsIndex=0;TopPriorityExtensionsIndex<TopPriorityExtensionsCount;TopPriorityExtensionsIndex++)
		{
			FString ThisExtension = TopPriorityExtensions[TopPriorityExtensionsIndex];
			// FString.Find is case insensitive
			int32 ExtIndex = PreferredExtensions.Find(ThisExtension);
			if (ExtIndex != INDEX_NONE)
			{
				PreferredExtensions.RemoveAt(ExtIndex);
				PreferredExtensions.Insert(ThisExtension, 0);
				// only do for first one found:
				break;
			}
		}

		// if there are multiple exporters, we arbitrarily choose [0] to go first
		//	@todo sort by alpha or priority or something to make this order consistent
		FString FirstExtension = PreferredExtensions[0];

		// If TopPriorityExtension is listed, make that the first option here too, then compile them all into one string
		check(AllFileTypes.Num() == AllExtensions.Num())

		for (int ExtIndex = 1; ExtIndex < AllFileTypes.Num(); ++ExtIndex)
		{
			const FString FileType = AllFileTypes[ExtIndex];
			if (FileType.Contains(FirstExtension))
			{
				AllFileTypes.RemoveAt(ExtIndex);
				AllFileTypes.Insert(FileType, 0);

				const FString Extension = AllExtensions[ExtIndex];
				AllExtensions.RemoveAt(ExtIndex);
				AllExtensions.Insert(Extension, 0);
			}
		}

		FString FileTypes;
		FString Extensions;
		for (int ExtIndex = 0; ExtIndex < AllFileTypes.Num(); ++ExtIndex)
		{
			if (FileTypes.Len())
			{
				FileTypes += TEXT("|");
			}
			FileTypes += AllFileTypes[ExtIndex];

			if (Extensions.Len())
			{
				Extensions += TEXT(";");
			}
			Extensions += AllExtensions[ExtIndex];
		}

		FString SaveFileName;
		if (bPromptIndividualFilenames)
		{
			TArray<FString> SaveFilenames;
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			bool bSave = false;
			if (DesktopPlatform)
			{
				bSave = DesktopPlatform->SaveFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					FText::Format(NSLOCTEXT("UnrealEd", "Save_F", "Save: {0}"), FText::FromString(ObjectToExport->GetName())).ToString(),
					*LastExportPath,
					*ObjectToExport->GetName(),
					*FileTypes,
					EFileDialogFlags::None,
					SaveFilenames
				);
			}

			if (!bSave)
			{
				int32 NumObjectsLeftToExport = ObjectsToExport.Num() - Index - 1;
				if (NumObjectsLeftToExport > 0)
				{
					const FText ConfirmText = FText::Format(NSLOCTEXT("UnrealEd", "AssetTools_ExportObjects_CancelRemaining", "Would you like to cancel exporting the next {0} files as well?"), FText::AsNumber(NumObjectsLeftToExport));
					if (EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, ConfirmText))
					{
						break;
					}
				}
				continue;
			}
			SaveFileName = FString(SaveFilenames[0]);

			// Copy off the selected path for future export operations.
			LastExportPath = SaveFileName;
		}
		else
		{
			// Assemble a filename from the export directory and the object path.
			SaveFileName = SelectedExportPath;

			if (!FPackageName::IsShortPackageName(ObjectToExport->GetOutermost()->GetFName()))
			{
				// Determine the save file name from the long package name
				FString PackageName = ObjectToExport->GetOutermost()->GetName();
				if (PackageName.Left(1) == TEXT("/"))
				{
					// Trim the leading slash so the file manager doesn't get confused
					PackageName.MidInline(1, MAX_int32, EAllowShrinking::No);
				}

				FPaths::NormalizeFilename(PackageName);
				
				SaveFileName /= bSaveAsCleanFilename ? FPaths::GetCleanFilename(PackageName) : PackageName;
			}
			else
			{
				// Assemble the path from the package name.
				SaveFileName /= ObjectToExport->GetOutermost()->GetName();
				SaveFileName /= ObjectToExport->GetName();
			}
			SaveFileName += FString::Printf(TEXT(".%s"), *FirstExtension);
			UE_LOGF(LogAssetTools, Log, "Exporting \"%ls\" to \"%ls\"", *ObjectToExport->GetPathName(), *SaveFileName);
		}

		// Create the path, then make sure the target file is not read-only.
		const FString ObjectExportPath(FPaths::GetPath(SaveFileName));
		const bool bFileInSubdirectory = ObjectExportPath.Contains(TEXT("/"));
		if (bFileInSubdirectory && (!IFileManager::Get().MakeDirectory(*ObjectExportPath, true)))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_FailedToMakeDirectory", "Failed to make directory {0}"), FText::FromString(ObjectExportPath)));
		}
		else if (IFileManager::Get().IsReadOnly(*SaveFileName))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_CouldntWriteToFile_F", "Couldn't write to file '{0}'. Maybe file is read-only?"), FText::FromString(SaveFileName)));
		}
		else
		{
			// We have a writeable file.  Now go through that list of exporters again and find the right exporter and use it.
			TArray<UExporter*>	ValidExporters;

			for (int32 ExporterIndex = 0; ExporterIndex < Exporters.Num(); ++ExporterIndex)
			{
				UExporter* Exporter = Exporters[ExporterIndex];
				if (Exporter->SupportsObject(ObjectToExport))
				{
					check(Exporter->FormatExtension.Num() == Exporter->FormatDescription.Num());
					for (int32 FormatIndex = 0; FormatIndex < Exporter->FormatExtension.Num(); ++FormatIndex)
					{
						const FString& FormatExtension = Exporter->FormatExtension[FormatIndex];
						if (FCString::Stricmp(*FormatExtension, *FPaths::GetExtension(SaveFileName)) == 0 ||
							FCString::Stricmp(*FormatExtension, TEXT("*")) == 0)
						{
							ValidExporters.Add(Exporter);
							break;
						}
					}
				}
			}

			// Handle the potential of multiple exporters being found
			UExporter* ExporterToUse = NULL;
			if (ValidExporters.Num() == 1)
			{
				ExporterToUse = ValidExporters[0];
			}
			else if (ValidExporters.Num() > 1)
			{
				// Set up the first one as default
				ExporterToUse = ValidExporters[0];

				// ...but search for a better match if available
				for (int32 ExporterIdx = 0; ExporterIdx < ValidExporters.Num(); ExporterIdx++)
				{
					if (ValidExporters[ExporterIdx]->GetClass()->GetFName() == ObjectToExport->GetExporterName())
					{
						ExporterToUse = ValidExporters[ExporterIdx];
						break;
					}
				}
			}

			// If an exporter was found, use it.
			if (ExporterToUse)
			{
				const FScopedBusyCursor BusyCursor;

				if (!UsedExporters.Contains(ExporterToUse))
				{
					ExporterToUse->SetBatchMode(ObjectsToExport.Num() > 1 && !bPromptIndividualFilenames);
					ExporterToUse->SetCancelBatch(false);
					ExporterToUse->SetShowExportOption(true);
					ExporterToUse->AddToRoot();
					UsedExporters.Add(ExporterToUse);
				}

				UAssetExportTask* ExportTask = NewObject<UAssetExportTask>();
				FGCObjectScopeGuard ExportTaskGuard(ExportTask);
				ExportTask->Object = ObjectToExport;
				ExportTask->Exporter = ExporterToUse;
				ExportTask->Filename = SaveFileName;
				ExportTask->bSelected = false;
				ExportTask->bReplaceIdentical = true;
				ExportTask->bPrompt = false;
				ExportTask->bUseFileArchive = ObjectToExport->IsA(UPackage::StaticClass());
				ExportTask->bWriteEmptyFiles = false;

				UExporter::RunAssetExportTask(ExportTask);

				if (ExporterToUse->GetBatchMode() && ExporterToUse->GetCancelBatch())
				{
					//Exit the export file loop when there is a cancel all
					break;
				}
			}
		}
	}

	//Set back the default value for the all used exporters
	for (UExporter* UsedExporter : UsedExporters)
	{
		UsedExporter->SetBatchMode(false);
		UsedExporter->SetCancelBatch(false);
		UsedExporter->SetShowExportOption(true);
		UsedExporter->RemoveFromRoot();
	}
	UsedExporters.Empty();

	if (bAnyObjectMissingSourceData)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Exporter_Error_SourceDataUnavailable", "No source data available for some objects.  See the log for details."));
	}

	GWarn->EndSlowTask();

	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, LastExportPath);
}

bool UAssetToolsImpl::CanExportAssets(const TArray<FAssetData>& AssetsToExport) const
{
	const TSharedPtr<FPathPermissionList> PermissionList = GetAssetClassPathPermissionList(EAssetClassAction::ExportAsset);
	if (PermissionList && PermissionList->HasFiltering())
	{
		// Use TSet as optimization because FPathPermissionList performs slow TArray<FString> search
		TSet<FTopLevelAssetPath> AlreadyPassed;
		for (const FAssetData& AssetData : AssetsToExport)
		{
			if (AssetData.HasAnyPackageFlags(EPackageFlags::PKG_DisallowExport))
			{
				return false;
			}

			if (!AlreadyPassed.Contains(AssetData.AssetClassPath))
			{
				FNameBuilder ClassPath;
				AssetData.AssetClassPath.AppendString(ClassPath);
				if (!PermissionList->PassesFilter(ClassPath.ToView()))
				{
					return false;
				}

				AlreadyPassed.Add(AssetData.AssetClassPath);
			}
		}
	}
	else
	{
		for (const FAssetData& AssetData : AssetsToExport)
		{
			if (AssetData.HasAnyPackageFlags(EPackageFlags::PKG_DisallowExport))
			{
				return false;
			}
		}
	}

	return true;
}

UAssetToolsImpl& UAssetToolsImpl::Get()
{
	FAssetToolsModule& Module = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
	return static_cast<UAssetToolsImpl&>(Module.Get());
}

void UAssetToolsImpl::SyncBrowserToAssets(const TArray<UObject*>& AssetsToSync)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets( AssetsToSync, /*bAllowLockedBrowsers=*/true );
}

void UAssetToolsImpl::SyncBrowserToAssets(const TArray<FAssetData>& AssetsToSync)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets( AssetsToSync, /*bAllowLockedBrowsers=*/true );
}

bool UAssetToolsImpl::CheckForDeletedPackage(const UPackage* Package) const
{
	if ( ISourceControlModule::Get().IsEnabled() )
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if ( SourceControlProvider.IsAvailable() )
		{
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::ForceUpdate);
			if ( SourceControlState.IsValid() && SourceControlState->IsDeleted() )
			{
				// Creating an asset in a package that is marked for delete - revert the delete and check out the package
				if (!SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), Package))
				{
					// Failed to revert file which was marked for delete
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RevertDeletedFileFailed", "Failed to revert package which was marked for delete."));
					return false;
				}

				if (!SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), Package))
				{
					// Failed to check out file
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("CheckOutFileFailed", "Failed to check out package"));
					return false;
				}
			}
		}
		else
		{
			FMessageLog EditorErrors("EditorErrors");
			EditorErrors.Warning(LOCTEXT( "DeletingNoSCCConnection", "Could not check for deleted file. No connection to revision control available!"));
			EditorErrors.Notify();
		}
	}

	return true;
}

bool UAssetToolsImpl::CanCreateAsset(const FString& AssetName, const FString& PackageName, const FText& OperationText, bool bOverwriteExisting) const
{
	// @todo: These 'reason' messages are not localized strings!
	FText Reason;
	if (!FName(*AssetName).IsValidObjectName( Reason )
		|| !FPackageName::IsValidLongPackageName( PackageName, /*bIncludeReadOnlyRoots=*/false, &Reason ) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, Reason );
		return false;
	}

	// We can not create assets that share the name of a map file in the same location
	if ( FEditorFileUtils::IsMapPackageAsset(PackageName) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("AssetNameInUseByMap", "You can not create an asset named '{0}' because there is already a map file with this name in this folder."), FText::FromString( AssetName ) ) );
		return false;
	}

	// Check for delegates that disallow the particular path and name
	if (!IsObjectPathAllowed(PackageName, &Reason))
	{
		FMessageDialog::Open(EAppMsgType::Ok, Reason);
		return false;
	}

	// Find (or create!) the desired package for this object
	UPackage* Pkg = CreatePackage(*PackageName);

	// Handle fully loading packages before creating new objects.
	TArray<UPackage*> TopLevelPackages;
	TopLevelPackages.Add( Pkg );
	if( !UPackageTools::HandleFullyLoadingPackages( TopLevelPackages, OperationText ) )
	{
		// User aborted.
		return false;
	}

	// We need to test again after fully loading.
	if (!FName(*AssetName).IsValidObjectName( Reason )
		||	!FPackageName::IsValidLongPackageName( PackageName, /*bIncludeReadOnlyRoots=*/false, &Reason ) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, Reason );
		return false;
	}

	// Check for an existing object
	UObject* ExistingObject = StaticFindObject( UObject::StaticClass(), Pkg, *AssetName );
	if( ExistingObject != nullptr )
	{
		// Object already exists in either the specified package or another package.

		if (!CanShowDialogs() && !bOverwriteExisting)
		{
			UE_LOGF(LogAssetTools, Error, "The asset '%ls' already exists in package '%ls'. CanCreateAsset cannot ask the user as the application is running unattended and will return false", *AssetName, *PackageName);
			return false;
		}
		
		bool bWantReplace = bOverwriteExisting;
		if (!bWantReplace)
		{
			// Check to see if the user wants to replace the object.
			TSharedRef<SMessageDialog> ConfirmDialog = SNew(SMessageDialog)
				.Icon(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
				.Title(FText(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_Title", "Overwrite Existing Object")))
				.Message(FText::Format(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_Message", "An object already exists with this name.\n\n\tName: {0}\n\tClass: {1}\n\tAsset path: {2}\n\nOverwrite the existing object?"),
					FText::FromString(AssetName),
					FText::FromString(ExistingObject->GetClass()->GetName()),
					FText::FromString(PackageName)))
				.Buttons({
					SCustomDialog::FButton(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_ButtonOverwrite", "Overwrite")).SetPrimary(true),
					SCustomDialog::FButton(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_ButtonCancel", "Cancel")),
					})
					.ContentMinWidth(300.0f);
			uint32 ConfirmationResult = ConfirmDialog->ShowModal();

			bWantReplace = ConfirmationResult == 0;
		}

		if( bWantReplace )
		{
			// Replacing an object.  Here we go!
			// Delete the existing object
			bool bDeleteSucceeded = ObjectTools::DeleteSingleObject( ExistingObject );

			if (bDeleteSucceeded)
			{
				// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
				CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

				// Old package will be GC'ed... create a new one here
				Pkg = CreatePackage(*PackageName);
				Pkg->MarkAsFullyLoaded();
			}
			else
			{
				// Notify the user that the operation failed b/c the existing asset couldn't be deleted
				FMessageDialog::Open( EAppMsgType::Ok,	FText::Format( NSLOCTEXT("DlgNewGeneric", "ContentBrowser_CannotDeleteReferenced", "{0} wasn't created.\n\nThe asset is referenced by other content."), FText::FromString( AssetName ) ) );
			}

			if( !bDeleteSucceeded || !IsUniqueObjectName( *AssetName, Pkg, Reason ) )
			{
				// Original object couldn't be deleted
				return false;
			}
		}
		else
		{
			// User chose not to replace the object; they'll need to enter a new name
			return false;
		}
	}

	// Check for a package that was marked for delete in source control
	if ( !CheckForDeletedPackage(Pkg) )
	{
		return false;
	}

	return true;
}

void UAssetToolsImpl::BuildMigrateDependencies(const TArray<FName>& PackageNamesToMigrate, TSet<FName>& OutAllDependenciesToMigrate, TSet<FName>& OutExcludedDependencies) const
{
	// Form a full list of packages to move by including the dependencies of the supplied packages
	OutAllDependenciesToMigrate.Reset();
	TSet<FString> ExternalObjectsPaths;
	OutExcludedDependencies.Reset();
	{
		FScopedSlowTask SlowTask(static_cast<float>(PackageNamesToMigrate.Num()), LOCTEXT( "MigratePackages_GatheringDependencies", "Gathering Dependencies..." ) );
		SlowTask.MakeDialog();

		TFunction<bool (FName)> ShouldExcludePackage = [this](FName PackageName) { return !CanMigratePackage(PackageName); };

		for ( auto PackageIt = PackageNamesToMigrate.CreateConstIterator(); PackageIt; ++PackageIt )
		{
			SlowTask.EnterProgressFrame();

			if (!OutAllDependenciesToMigrate.Contains(*PackageIt) )
			{
				OutAllDependenciesToMigrate.Add(*PackageIt);
				RecursiveGetDependencies(*PackageIt, OutAllDependenciesToMigrate, ExternalObjectsPaths, OutExcludedDependencies, ShouldExcludePackage);
			}
		}
	}
}

void UAssetToolsImpl::PerformMigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& DestinationPath, const FMigrationOptions& Options, TSet<FName> AllDependenciesToMigrate, TSet<FName> ExcludedDependencies, bool bSanitizeInputs) const
{
	if (bSanitizeInputs)
	{
		TSet<FName> VerifiedAllPackageNamesToMove;
		TSet<FName> VerifiedExcludedDependencies;
		BuildMigrateDependencies(PackageNamesToMigrate, VerifiedAllPackageNamesToMove, VerifiedExcludedDependencies);
		
		// Don't allow migration of packages that are not in the allowed list of verified packages 
		for (auto It = AllDependenciesToMigrate.CreateIterator(); It; ++It)
		{
			if (!VerifiedAllPackageNamesToMove.Contains(*It))
			{
				It.RemoveCurrent();
			}
		}
		
		// Don't allow migration of dependencies that have been excluded in verified list
		for (const FName& Ex : VerifiedExcludedDependencies)
		{
			if (!ExcludedDependencies.Contains(Ex))
			{
				ExcludedDependencies.Add(Ex);
			}
		}
	}
	
	// Fetch the enabled plugins and their mount points
	TMap<FName, EPluginLoadedFrom> EnabledPluginToLoadedFrom;
	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPluginsWithContent();
	for (const TSharedRef<IPlugin>& EnabledPlugin : EnabledPlugins)
	{
		EnabledPluginToLoadedFrom.Add(FName(EnabledPlugin->GetMountedAssetPath()), EnabledPlugin->GetLoadedFrom());
	}

	// Find assets in non-Project Plugins
	TSet<FName> ShouldMigratePackage;
	bool bShouldShowEngineContent = GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();
	TMap<FName, TSet<FName>> PackageToExternalObjectPackages;
	TMap<FName, FName> ExternalObjectPackageToParentPackage;
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		const bool bOnlyIncludeOnDiskAssets = true;

		// This is the new list to prompt for migration
		TSet<FName> FilteredPackageNamesToMove;

		for (const FName& PackageName : AllDependenciesToMigrate)
		{
			// Associate External Packages to Level Packages
			FString PackageNameStr = PackageName.ToString();
			if (PackageNameStr.Contains(FPackagePath::GetExternalActorsFolderName()) || PackageNameStr.Contains(FPackagePath::GetExternalObjectsFolderName()))
			{
				TArray<FAssetData> Assets;
				if (AssetRegistryModule.Get().GetAssetsByPackageName(PackageName, Assets, bOnlyIncludeOnDiskAssets))
				{
					for (const FAssetData& AssetData : Assets)
					{
						if (!AssetData.GetOptionalOuterPathName().IsNone())
						{
							const FName OuterPackageName = FSoftObjectPath(AssetData.GetOptionalOuterPathName().ToString()).GetLongPackageFName();
							PackageToExternalObjectPackages.FindOrAdd(OuterPackageName).Add(PackageName);
							
							if (bSanitizeInputs)
							{
								ExternalObjectPackageToParentPackage.Add(PackageName, OuterPackageName);
							}
						}
					}
				}
				// Avoid adding external object packages to the FilteredPackageNamesToMove because we don't want them to show up in the Migrate dialog. Instead they are going to be migrated if their outer package gets migrated.
				continue;
			}

			FName PackageMountPoint = FPackageName::GetPackageMountPoint(PackageName.ToString(), false);
			EPluginLoadedFrom* Found = EnabledPluginToLoadedFrom.Find(PackageMountPoint);

			bool bShouldMigratePackage = true;
			if (Found)
			 {
				// plugin content, decide if it's appropriate to migrate
				switch (*Found)
				{
				case EPluginLoadedFrom::Engine:
					if (!bShouldShowEngineContent)
					{
						continue;
					}
					bShouldMigratePackage = false;
					break;

				case EPluginLoadedFrom::Project:
					bShouldMigratePackage = true;
					break;
				 
				default:
					bShouldMigratePackage = false;
					break;
				 }
			 }
			 else
			 {
				// this is not plugin content
				if (PackageName.ToString().StartsWith(TEXT("/Engine")))
				{
					// Engine content
					if (!bShouldShowEngineContent)
					{
						continue;
					}
					bShouldMigratePackage = false;
				}
				else
				{
					// Game content
					bShouldMigratePackage = true;
				}
			}


			// Ignore dependencies and only migrate the given assets
			if (Options.bIgnoreDependencies)
			{
				if (PackageNamesToMigrate.Contains(PackageName))
				{
					bShouldMigratePackage = true;
				}
				else
				{
					bShouldMigratePackage = false;
				}
			}


			FilteredPackageNamesToMove.Add(PackageName);

			if (bShouldMigratePackage)
			{
				ShouldMigratePackage.Add(PackageName);
			}
		}

		AllDependenciesToMigrate = FilteredPackageNamesToMove;
	}

	// Confirm that there is at least one package to move 
	if (AllDependenciesToMigrate.Num() == 0)
	{
		if (CanShowDialogs())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MigratePackages_NoFilesToMove", "No files were found to move"));
		}
		return;
	}

	TSet<FName> SelectedParents;
	TSharedPtr<TArray<ReportPackageData>> ReportPackages = MakeShareable(new TArray<ReportPackageData>);
	for (auto PackageIt = AllDependenciesToMigrate.CreateConstIterator(); PackageIt; ++PackageIt)
	{
		const FName PackageName = *PackageIt;
		bool bShouldMigratePackage = ShouldMigratePackage.Find(PackageName) != nullptr;
		
		if (bSanitizeInputs && bShouldMigratePackage)
		{
			// For each external object/actor if their parent is not being migrated, then they shouldn't either
			if (const FName* ParentPackageName = ExternalObjectPackageToParentPackage.Find(PackageName))
			{
				if (!ShouldMigratePackage.Contains(*ParentPackageName))
				{
					bShouldMigratePackage = false;
				}
			}
		}
		
		ReportPackages.Get()->Add({ PackageName.ToString(), bShouldMigratePackage });
	}
	
	if (bSanitizeInputs)
	{
		// Clean PackageToExternalObjectPackages to keep only the selected Package parents
		for (auto It = PackageToExternalObjectPackages.CreateIterator(); It; ++It)
		{
			if (!ShouldMigratePackage.Contains(It.Key()))
			{
				It.RemoveCurrent();
			}
		}
	}
	
	// Prompt the user displaying all assets that are going to be migrated
	if(CanShowDialogs() && Options.bPrompt)
	{
		const FText ReportMessage = LOCTEXT("MigratePackagesReportTitle", "The following assets will be migrated to another content folder.");
		SPackageReportDialog::FOnReportConfirmed OnReportConfirmed = SPackageReportDialog::FOnReportConfirmed::CreateUObject(this, &UAssetToolsImpl::MigratePackages_ReportConfirmed, ReportPackages, DestinationPath, MoveTemp(ExcludedDependencies), MoveTemp(PackageToExternalObjectPackages), Options);
		SPackageReportDialog::OpenPackageReportDialog(ReportMessage, *ReportPackages.Get(), OnReportConfirmed);
	}
	else
	{
		UAssetToolsImpl::MigratePackages_ReportConfirmed(ReportPackages, DestinationPath, MoveTemp(ExcludedDependencies), MoveTemp(PackageToExternalObjectPackages), Options);
	}
}

void UAssetToolsImpl::MigratePackages_ReportConfirmed(TSharedPtr<TArray<ReportPackageData>> PackageDataToMigrate, const FString DestinationPath, TSet<FName> ExcludedDependencies, TMap<FName, TSet<FName>> PackageToExternalObjectPackages, const FMigrationOptions Options) const
{
	FString DestinationFolder;
	if (!CanShowDialogs() || !DestinationPath.IsEmpty())
	{
		if(DestinationPath.IsEmpty())
		{
			UE_LOGF(LogAssetTools, Error, "Migration Destination path cannot be empty.");
			return;
		}

		DestinationFolder = DestinationPath;
		FPaths::NormalizeDirectoryName(DestinationFolder);
		DestinationFolder += TEXT("/");
	}
	else
	{
		// Choose a destination folder
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (ensure(DesktopPlatform))
		{
			const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

			const FString Title = LOCTEXT("MigrateToFolderTitle", "Choose a destination Content folder").ToString();
			bool bFolderAccepted = false;
			while (!bFolderAccepted)
			{
				const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
					ParentWindowWindowHandle,
					Title,
					FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT),
					DestinationFolder
				);

				if (!bFolderSelected)
				{
					// User canceled, return
					return;
				}

				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, DestinationFolder);
				FPaths::NormalizeFilename(DestinationFolder);
				if (!DestinationFolder.EndsWith(TEXT("/")))
				{
					DestinationFolder += TEXT("/");
				}

				// Verify that it is a content folder
				if (!DestinationFolder.EndsWith(TEXT("/Content/")))
				{
					// The user chose a non-content folder. Let them know they cannot do that.
					const FText Message = FText::Format(LOCTEXT("MigratePackages_NonContentFolder", "{0} does not appear to be a Content folder. Migrated content only work if placed in a Content folder. Select a Content folder."), FText::FromString(DestinationFolder));
					EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::OkCancel, Message);
					if (Response == EAppReturnType::Cancel)
					{
						return;
					}

					continue;
				}


				FString RootPath = UE::AssetTools::Private::FPackageMigrationImpl::GetMountPointRootPath(DestinationFolder);

				if (RootPath.IsEmpty())
				{
					const FText Message = FText::Format(LOCTEXT("MigratePackages_CannotIdentifyMountPoint", "{0} does not appear to be a game Content folder or a Plugin folder. The tool didn't found the associated uproject or uplugin file used to determine the unreal path of destination folder. Migrated content only work if placed in a Project or a Plugin Content folder. Select a Content folder."), FText::FromString(DestinationFolder));
					EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::OkCancel, Message);
					if (Response == EAppReturnType::Cancel)
					{
						return;
					}
					continue;
				}

				if (WritableFolderPermissionList->HasFiltering() && FPathViews::IsParentPathOf(FPaths::ProjectDir(), DestinationFolder))
				{
					// Make sure the path is not in the deny list
					if (!WritableFolderPermissionList->PassesFilter(RootPath))
					{
						const FText Message = FText::Format(LOCTEXT("MigratePackages_CannotMoveContentHere", "{0} is not in a writable folder. Chose another Content folder."), FText::FromString(DestinationFolder));
						EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::OkCancel, Message);
						if (Response == EAppReturnType::Cancel)
						{
							return;
						}
						continue;
					}
				}
				
			

				bFolderAccepted = true;
			}
		}
		else
		{
			// Not on a platform that supports desktop functionality
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoDesktopPlatform", "Error: This platform does not support a file dialog."));
			return;
		}
	}

	// Build a list of packages to handle
	TSet<FName> AllPackageNamesToMove;
	TArray<ReportPackageData> ExternalPackagesToMigrate;
	for (auto PackageDataIt = PackageDataToMigrate->CreateConstIterator(); PackageDataIt; ++PackageDataIt)
	{
		if (PackageDataIt->bShouldMigratePackage)
		{
			FName PackageName(PackageDataIt->Name);
			AllPackageNamesToMove.Add(PackageName);

			if (TSet<FName>* ExternalObjectPackages = PackageToExternalObjectPackages.Find(PackageName))
			{
				for (const FName& ExternalObjectPackage : *ExternalObjectPackages)
				{
					AllPackageNamesToMove.Add(ExternalObjectPackage);
					ExternalPackagesToMigrate.Add({ ExternalObjectPackage.ToString(), true });
				}
			}
		}
	}
	PackageDataToMigrate->Append(MoveTemp(ExternalPackagesToMigrate));

	FMessageLog MigrateLog("AssetTools");

	if (UE::AssetTools::Private::bUseNewPackageMigration)
	{
		using namespace UE::AssetTools::Private;

		FString MountPointRootPath = FPackageMigrationImpl::GetMountPointRootPath(DestinationFolder);

		if (!MountPointRootPath.IsEmpty())
		{
			FPackageMigrationImpl::FTemporaryWritableFolderPermission ScopedFolderPermission(WritableFolderPermissionList);

			// Add a temporary permission if needed
			if (WritableFolderPermissionList->HasFiltering())
			{
				if (FPathViews::IsParentPathOf(FPaths::ProjectDir(), DestinationFolder))
				{
					if (!WritableFolderPermissionList->PassesFilter(MountPointRootPath))
					{
						UE_LOGF(LogAssetTools, Error, "Migration Destination path is not a writable folder.");
						return;
					}
				}
				else
				{
					ScopedFolderPermission.AddTemporaryWriteFolderPermission(MountPointRootPath);
				}
			}

			FPackageMigrationImpl::FPackageMigrationImplContext MigrationImplContext;
			MigrationImplContext.Options = Options;

			// 1) Create the package migration state. It will create the mount point also if needed
			UE::AssetTools::FPackageMigrationContext PackageMigrationContext(UE::AssetTools::FPackageMigrationContext::FScopedMountPoint(MoveTemp(MountPointRootPath), MoveTemp(DestinationFolder)));

			PackageMigrationContext.CurrentStep =  UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::BeginMigration;
			OnPackageMigration.Broadcast(PackageMigrationContext);

			// 2) Set up the instancing context so that the packages are load in the destination mount point and setup the packages migration data
			{
				PackageMigrationContext.MigrationPackagesData.Reserve(PackageDataToMigrate->Num());

				TArray<TPair<const ReportPackageData*, const FAssetData>> ExternalPackageDatas;
				TMap<const FStringView, int32> ExistingPackageNameToMigrationDataIndex;

				FPackageMigrationImpl::SetupPublicAssetPackagesMigrationData(PackageDataToMigrate
					, ExternalPackageDatas
					, ExistingPackageNameToMigrationDataIndex
					, PackageMigrationContext
					, MigrationImplContext
					, MigrateLog);

				PackageMigrationContext.CurrentStep =  UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::PostAssetMigrationPackageDataCreated;
				OnPackageMigration.Broadcast(PackageMigrationContext);


				FPackageMigrationImpl::SetupExternalAssetPackagesMigrationData(ExternalPackageDatas
					, ExistingPackageNameToMigrationDataIndex
					, PackageMigrationContext
					, MigrationImplContext);

				PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::PostExternalMigrationPackageDataCreated;
				OnPackageMigration.Broadcast(PackageMigrationContext);


				FPackageMigrationImpl::ProcessExcludedDependencies(ExcludedDependencies, PackageMigrationContext);
				
				PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::PostExcludedDependenciesCreated;
				OnPackageMigration.Broadcast(PackageMigrationContext);
			}

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InTheWayPackagesMoved;
			OnPackageMigration.Broadcast(PackageMigrationContext);


			// 3) Instanced load
			TArray<FName> PackagesToClean;
			PackagesToClean.Reserve(PackageMigrationContext.MigrationPackagesData.Num());
			FPackageMigrationImpl::CreatePackagesAndSetupLinkers(PackageMigrationContext, MigrationImplContext, PackagesToClean);

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InstancedPackagesCreated;
			OnPackageMigration.Broadcast(PackageMigrationContext);

			FPackageMigrationImpl::LoadInstancedPackages(PackageMigrationContext, MigrationImplContext);

			// Collect the garbage created by the load here and it also help detecting potential issues in migration (which can cause crashes).
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InstancedPackagesLoaded;
			OnPackageMigration.Broadcast(PackageMigrationContext);


			// 4) Saves
			FPackageMigrationImpl::SaveInstancedPackagesIntoDestination(PackageMigrationContext, MigrationImplContext);

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InstancedPackagesSaved;
			OnPackageMigration.Broadcast(PackageMigrationContext);


			// 5) Remove the packages created for the migration
			FPackageMigrationImpl::CleanInstancedPackages(PackagesToClean, PackageMigrationContext);

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::PostCleaningInstancedPackages;
			OnPackageMigration.Broadcast(PackageMigrationContext);

			// 6) Restore the existing package that where in the way
			FPackageMigrationImpl::RestoreInTheWayPackages(PackageMigrationContext);

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::InTheWayPackagesRestored;
			OnPackageMigration.Broadcast(PackageMigrationContext);

			FPackageMigrationImpl::BuildAndNotifyLogWindow(PackageMigrationContext, MigrateLog, MigrationImplContext);

			PackageMigrationContext.CurrentStep = UE::AssetTools::FPackageMigrationContext::EPackageMigrationStep::EndMigration;
			OnPackageMigration.Broadcast(PackageMigrationContext);
		}
	}
	else
	{
		bool bAbort = false;

		// Associate each Content folder in the target Plugin hierarchy to a content root string in UFS
		TMap<FName, FString> DestContentRootsToFolders;

		// Assets in /Game always map directly to the destination
		DestContentRootsToFolders.Add(FName(TEXT("/Game")), DestinationFolder);

		// Determine if the destination is a project content folder
		TArray<FString> ProjectFiles;
		IFileManager::Get().FindFiles(ProjectFiles, *(DestinationFolder + TEXT("../")), TEXT("uproject"));
		bool bIsDestinationAProject = !ProjectFiles.IsEmpty();

		// If our destination is a project, it could have plugins...
		if (bIsDestinationAProject)
		{
			// Find all "Content" folders under the destination ../Plugins directory
			TArray<FString> ContentFolders;
			IFileManager::Get().FindFilesRecursive(ContentFolders, *(DestinationFolder + TEXT("../Plugins/")), TEXT("Content"), false, true);

			for (const FString& Folder : ContentFolders)
			{
				// Parse the parent folder of .../Content from "Folder"
				FString Path, Content, Root;
				bool bSplitContent = Folder.Split(TEXT("/"), &Path, &Content, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
				bool bSplitParentPath = Path.Split(TEXT("/"), nullptr, &Root, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
				if (!bSplitContent || !bSplitParentPath)
				{
					MigrateLog.Error(FText::Format(LOCTEXT("MigratePackages_NoMountPointFolder", "Unable to determine mount point for folder {0}"), FText::FromString(Folder)));
					bAbort = true;
					continue;
				}

				// Determine this folder name to be a content root in the destination
				FString DestContentRoot = FString(TEXT("/")) + Root;
				FString DestContentFolder = FPaths::ConvertRelativePathToFull(Folder + TEXT("/"));
				DestContentRootsToFolders.Add(FName(DestContentRoot), DestContentFolder);
			}
		}

		if (bAbort)
		{
			MigrateLog.Notify();
			return;
		}

		// Check if the root of any of the packages to migrate have no destination
		TArray<FName> LostPackages;
		TSet<FName> LostPackageRoots;
		for (const FName& PackageName : AllPackageNamesToMove)
		{
			// Acquire the mount point for this package
			FString Folder = TEXT("/") + FPackageName::GetPackageMountPoint(PackageName.ToString()).ToString();

			// If this is /Game package, it doesn't need special handling and is simply bound for the destination, continue
			if (Folder == TEXT("/Game"))
			{
				continue;
			}

			// Resolve the disk folder of this package's mount point so we compare directory names directly
			//  We do this FileSystem to FileSystem compare instead of mount point (UFS) to content root (FileSystem)
			//  The mount point name likely comes from the FriendlyName in the uplugin, or the uplugin basename
			//  What's important here is that we succeed in finding _the same plugin_ between a copy/pasted project
			if (!FPackageName::TryConvertLongPackageNameToFilename(Folder, Folder))
			{
				MigrateLog.Error(FText::Format(LOCTEXT("MigratePackages_NoContentFolder", "Unable to determine content folder for asset {0}"), FText::FromString(PackageName.ToString())));
				bAbort = true;
				continue;
			}
			Folder.RemoveFromEnd(TEXT("/"));

			// Parse the parent folder of .../Content from "Folder"
			FString Path, Content, Root;
			bool bSplitContent = Folder.Split(TEXT("/"), &Path, &Content, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			bool bSplitParentPath = Path.Split(TEXT("/"), nullptr, &Root, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (!bSplitContent || !bSplitParentPath)
			{
				MigrateLog.Error(FText::Format(LOCTEXT("MigratePackages_NoMountPointPackage", "Unable to determine mount point for package {0}"), FText::FromString(PackageName.ToString())));
				bAbort = true;
				continue;
			}

			// Check to see if the content root exists in the destination, otherwise it's "Lost"
			FString SrcContentRoot = FString(TEXT("/")) + Root;
			FName SrcContentRootName(SrcContentRoot);
			if (!DestContentRootsToFolders.Find(SrcContentRootName))
			{
				LostPackages.Add(PackageName);
				LostPackageRoots.Add(SrcContentRootName);
			}
		}

		if (bAbort)
		{
			MigrateLog.Notify();
			return;
		}

		// If some packages don't have a matching content root in the destination, prompt for desired behavior
		if (!LostPackages.IsEmpty())
		{
			if (CanShowDialogs() && Options.bPrompt && Options.OrphanFolder.IsEmpty())
			{
				FString LostPackageRootsString;
				for (const FName& PackageRoot : LostPackageRoots)
				{
					LostPackageRootsString += FString(TEXT("\n\t")) + PackageRoot.ToString();
				}

				// Prompt to consolidate to a migration folder
				FText Prompt = FText::Format(LOCTEXT("MigratePackages_ConsolidateToTemp", "Some selected assets don't have a corresponding content root in the destination.{0}\n\nWould you like to save a copy of all selected assets into a folder with consolidated references? If you select No then assets in the above roots will not be migrated."), FText::FromString(LostPackageRootsString));
				switch (FMessageDialog::Open(EAppMsgType::YesNoCancel, Prompt))
				{
				case EAppReturnType::Yes:
					// No op
					break;
				case EAppReturnType::No:
					LostPackages.Reset();
					break;
				case EAppReturnType::Cancel:
					return;
				}
			}
			else if (Options.OrphanFolder.IsEmpty())
			{
				LostPackages.Reset();
			}
		}

		// This will be used to tidy up the temp folder after we copy the migrated assets
		FString SrcDiskFolderFilename;

		// Fixing up references requires resaving packages to a temporary location
		if (!LostPackages.IsEmpty())
		{
			// Query the user for a folder to migrate assets into. This folder will exist temporary in this project, and is the destination for migration in the target project
			FString FolderName = Options.OrphanFolder.IsEmpty() ? TEXT("Migrated") : Options.OrphanFolder;
			if (CanShowDialogs() && Options.bPrompt)
			{
				bool bIsOkButtonEnabled = true;
				TSharedRef<SEditableTextBox> EditableTextBox = SNew(SEditableTextBox)
					.Text(FText::FromString(FolderName))
					.OnVerifyTextChanged_Lambda([&bIsOkButtonEnabled](const FText& InNewText, FText& OutErrorMessage) -> bool
						{
							if (InNewText.ToString().Contains(TEXT("/")))
							{
								OutErrorMessage = LOCTEXT("Migrated_CannotContainSlashes", "Cannot use a slash in a folder name.");

								// Disable Ok if the string is invalid
								bIsOkButtonEnabled = false;
								return false;
							}

							// Enable Ok if the string is valid
							bIsOkButtonEnabled = true;
							return true;
						})
					.OnTextCommitted_Lambda([&FolderName](const FText& NewValue, ETextCommit::Type)
						{
							// Set the result if they modified the text
							FolderName = NewValue.ToString();
						});

				// Set the result if they just click Ok
				SGenericDialogWidget::FArguments FolderDialogArguments;
				FolderDialogArguments.OnOkPressed_Lambda([&EditableTextBox, &FolderName]()
					{
						FolderName = EditableTextBox->GetText().ToString();
					});

				// Present the Dialog
				SGenericDialogWidget::OpenDialog(LOCTEXT("MigratePackages_FolderName", "Folder for Migrated Assets"),
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MigratePackages_SpecifyConsolidateFolder", "Please specify a new folder name to consolidate the assets into."))
					]
					+ SVerticalBox::Slot()
					.Padding(5.0f)
					[
						SNew(SSpacer)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.0f)
					[
						EditableTextBox
					],
					FolderDialogArguments, true);

				// Sanity the user input
				if (FolderName.IsEmpty())
				{
					return;
				}
			}
			// Remove forbidden characters
			FolderName = FolderName.Replace(TEXT("/"), TEXT(""));

			// Verify that we don't have any assets that exist where we want to perform our consolidation
			//  We could do the same check at the destination, but the package copy code will happily overwrite,
			//  and it seems reasonable someone would want to Migrate several times if snapshotting changes from another project.
			//  So for now we don't necessitate that the destination content folder be missing as well.
			FString SrcUfsFolderName = FolderName;
			SrcUfsFolderName.InsertAt(0, TEXT("/Game/"));
			SrcDiskFolderFilename = FPackageName::LongPackageNameToFilename(SrcUfsFolderName, TEXT(""));
			if (IFileManager::Get().DirectoryExists(*SrcDiskFolderFilename))
			{
				const FText Message = FText::Format(LOCTEXT("MigratePackages_InvalidMigrateFolder", "{0} exists on disk in the source project, and cannot be used to consolidate assets."), FText::FromString(SrcDiskFolderFilename));
				if (CanShowDialogs() && Options.bPrompt)
				{
					EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::Ok, Message);
				}
				else
				{
					UE_LOGF(LogAssetTools, Error, "%ls", *Message.ToString());
				}
				return;
			}

			// Resolve the packages to migrate to assets
			TArray<UObject*> SrcObjects;
			{
				FScopedSlowTask SlowTask(static_cast<float>(AllPackageNamesToMove.Num()), LOCTEXT("MigratePackages_Loading", "Loading Packages..."));
				SlowTask.MakeDialog();
				for (const FName& SrcPackage : AllPackageNamesToMove)
				{
					SlowTask.EnterProgressFrame();
					UPackage* LoadedPackage = UPackageTools::LoadPackage(SrcPackage.ToString());
					if (!LoadedPackage)
					{
						MigrateLog.Error(FText::Format(LOCTEXT("MigratePackages_FailedToLoadPackage", "Failed to load package {0}"), FText::FromString(SrcPackage.ToString())));
						bAbort = true;
						continue;
					}

					UObject* Object = LoadedPackage->FindAssetInPackage();
					if (Object)
					{
						SrcObjects.Add(Object);
					}
					else
					{
						MigrateLog.Warning(FText::Format(LOCTEXT("MigratePackages_PackageHasNoAsset", "Package {0} has no asset in it"), FText::FromString(SrcPackage.ToString())));
					}

					UBlueprint* Blueprint = Cast<UBlueprint>(Object);
					if (Blueprint && (!Blueprint->SkeletonGeneratedClass || !Blueprint->GeneratedClass))
					{
						MigrateLog.Error(FText::Format(LOCTEXT("MigratePackages_InvalidBluepriint", "Blueprint is invalid in package {0}"), FText::FromString(SrcPackage.ToString())));
						bAbort = true;
						continue;
					}
				}
			}

			if (bAbort)
			{
				MigrateLog.Notify();
				return;
			}

			// To handle complex references and assets in different Plugins, we must first duplicate to temp packages
			TArray<UObject*> TempObjects;
			TMap<UObject*, UObject*> ReplacementMap;
			{
				FScopedSlowTask SlowTask(static_cast<float>(SrcObjects.Num()), LOCTEXT("MigratePackages_Duplicate", "Duplicating Assets..."));
				SlowTask.MakeDialog();

				TSet<UPackage*> PackagesUserRefusedToFullyLoad;
				for (int i=0; i<SrcObjects.Num(); ++i)
				{
					SlowTask.EnterProgressFrame();

					UObject* Object = SrcObjects[i];
					FString PackageName = Object->GetPackage()->GetName();

					FString Path, Root;
					PackageName.RemoveFromStart(TEXT("/"));
					bool bSplitRoot = PackageName.Split(TEXT("/"), &Root, &Path);
					if (!bSplitRoot)
					{
						MigrateLog.Error(FText::Format(LOCTEXT("MigratePackages_NoMountPointPackage", "Unable to determine mount point for package {0}"), FText::FromString(PackageName)));
						continue;
					}

					ObjectTools::FPackageGroupName PackageGroupName;
					PackageGroupName.ObjectName = Object->GetName();
					PackageGroupName.PackageName = SrcUfsFolderName + TEXT("/") + Path;
					FString GroupName = Object->GetFullGroupName(/*bStartWithOuter =*/true);
					if (GroupName != TEXT("None"))
					{
						PackageGroupName.GroupName = GroupName;
					}

					UObject* NewObject = ObjectTools::DuplicateSingleObject(Object, PackageGroupName, PackagesUserRefusedToFullyLoad);
					if (NewObject)
					{
						TempObjects.Add(NewObject);
						ReplacementMap.Add(SrcObjects[i], TempObjects[i]);
					}
				}
			}

			// Update references between TempObjects (to reference each other)
			{
				FScopedSlowTask SlowTask(static_cast<float>(TempObjects.Num()), LOCTEXT("MigratePackages_ReplaceReferences", "Replacing References..."));
				SlowTask.MakeDialog();

				for (int i=0; i<TempObjects.Num(); ++i)
				{
					SlowTask.EnterProgressFrame();
					UObject* TempObject = TempObjects[i];

					FArchiveReplaceObjectRef<UObject> ReplaceAr(TempObject, ReplacementMap, EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
				}
			}

			// Save fixed up packages to the migrated folder, and update the set of files to copy to be those migrated packages
			{
				FScopedSlowTask SlowTask(static_cast<float>(TempObjects.Num()), LOCTEXT("MigratePackages_ReplaceReferences", "Replacing References..."));
				SlowTask.MakeDialog();

				TSet<FName> NewPackageNamesToMove;
				for (int i=0; i<TempObjects.Num(); ++i)
				{
					SlowTask.EnterProgressFrame();
					UObject* TempObject = TempObjects[i];

					// Calculate the file path to the new, migrated package
					FString const TempPackageName = TempObject->GetPackage()->GetName();
					FString const TempPackageFilename = FPackageName::LongPackageNameToFilename(TempPackageName, FPackageName::GetAssetPackageExtension());

					// Save it
					FSavePackageArgs SaveArgs;
					GEditor->Save(TempObject->GetPackage(), /*InAsset=*/nullptr, *TempPackageFilename, SaveArgs);

					NewPackageNamesToMove.Add(FName(TempPackageName));
				}

				AllPackageNamesToMove = NewPackageNamesToMove;
			}
		}

		bool bUserCanceled = false;

		EAppReturnType::Type LastResponse = EAppReturnType::Yes;
		if (!Options.bPrompt)
		{
			switch (Options.AssetConflict)
			{
				case EAssetMigrationConflict::Overwrite:
					LastResponse = EAppReturnType::YesAll;
					break;
				case EAssetMigrationConflict::Skip:
					LastResponse = EAppReturnType::NoAll;
					break;
				case EAssetMigrationConflict::Cancel:
					LastResponse = EAppReturnType::Cancel;
					break;
			}
		}
		TArray<FString> SuccessfullyCopiedFiles;
		TArray<FString> SuccessfullyCopiedPackages;
		FString CopyErrors;

		{
			FScopedSlowTask SlowTask(static_cast<float>(AllPackageNamesToMove.Num()), LOCTEXT("MigratePackages_CopyingFiles", "Copying Files..."));
			SlowTask.MakeDialog();

			for ( const FName& PackageNameToMove : AllPackageNamesToMove )
			{
				SlowTask.EnterProgressFrame();

				const FString& PackageName = PackageNameToMove.ToString();
				FString SrcFilename;
			
				if (!FPackageName::DoesPackageExist(PackageName, &SrcFilename))
				{
					const FText ErrorMessage = FText::Format(LOCTEXT("MigratePackages_PackageMissing", "{0} does not exist on disk."), FText::FromString(PackageName));
					UE_LOGF(LogAssetTools, Warning, "%ls", *ErrorMessage.ToString());
					CopyErrors += ErrorMessage.ToString() + LINE_TERMINATOR;
				}
				else if (SrcFilename.Contains(FPaths::EngineContentDir()))
				{
					const FString LeafName = SrcFilename.Replace(*FPaths::EngineContentDir(), TEXT("Engine/"));
					CopyErrors += FText::Format(LOCTEXT("MigratePackages_EngineContent", "Unable to migrate Engine asset {0}. Engine assets cannot be migrated."), FText::FromString(LeafName)).ToString() + LINE_TERMINATOR;
				}
				else
				{
					bool bFileOKToCopy = true;

					FString Path = PackageNameToMove.ToString();
					FString PackageRoot;
					Path.RemoveFromStart(TEXT("/"));
					Path.Split("/", &PackageRoot, &Path, ESearchCase::IgnoreCase, ESearchDir::FromStart);
					PackageRoot = TEXT("/") + PackageRoot;

					FString DestFilename;
					TMap<FName, FString>::ValueType* DestRootFolder = DestContentRootsToFolders.Find(FName(PackageRoot));
					if (ensure(DestRootFolder))
					{
						DestFilename = *DestRootFolder;
					}
					else
					{
						continue;
					}

					FString SubFolder;
					if ( SrcFilename.Split( TEXT("/Content/"), nullptr, &SubFolder ) )
					{
						DestFilename += *SubFolder;

						if ( IFileManager::Get().FileSize(*DestFilename) > 0 )
						{
							// The destination file already exists! Ask the user what to do.
							EAppReturnType::Type Response;
							if (!CanShowDialogs() || !Options.bPrompt || LastResponse == EAppReturnType::YesAll || LastResponse == EAppReturnType::NoAll)
							{
								Response = LastResponse;
							}
							else
							{
								const FText Message = FText::Format( LOCTEXT("MigratePackages_AlreadyExists", "An asset already exists at location {0} would you like to overwrite it?"), FText::FromString(DestFilename) );
								Response = FMessageDialog::Open( EAppMsgType::YesNoYesAllNoAllCancel, Message );
								LastResponse = Response;
							}

							if (Response == EAppReturnType::Cancel)
							{
								// The user chose to cancel mid-operation. Break out.
								bUserCanceled = true;
								break;
							}

							const bool bWantOverwrite = Response == EAppReturnType::Yes || Response == EAppReturnType::YesAll;
							if( !bWantOverwrite )
							{
								// User chose not to replace the package
								bFileOKToCopy = false;
							}
						}
					}
					else
					{
						// Couldn't find Content folder in source path
						bFileOKToCopy = false;
					}

					if ( bFileOKToCopy )
					{
						if ( IFileManager::Get().Copy(*DestFilename, *SrcFilename) == COPY_OK )
						{
							SuccessfullyCopiedPackages.Add(PackageName);
							SuccessfullyCopiedFiles.Add(DestFilename);
						}
						else
						{
							UE_LOGF(LogAssetTools, Warning, "Failed to copy %ls to %ls while migrating assets", *SrcFilename, *DestFilename);
							CopyErrors += SrcFilename + LINE_TERMINATOR;
						}
					}
				}
			}
		}

		// If we are consolidating lost packages, we are copying temporary packages, so clean them up.
		if (!LostPackages.IsEmpty())
		{
			FScopedSlowTask SlowTask(static_cast<float>(AllPackageNamesToMove.Num()), LOCTEXT("MigratePackages_CleaningUp", "Cleaning Up..."));
			SlowTask.MakeDialog();

			TArray<UObject*> ObjectsToDelete;
			for (const FName& PackageNameToMove : AllPackageNamesToMove)
			{
				SlowTask.EnterProgressFrame();

				UPackage* Package = UPackageTools::LoadPackage(PackageNameToMove.ToString());
				if (Package)
				{
					ObjectsToDelete.Add(Package);
				}
			}

			ObjectTools::ForceDeleteObjects(ObjectsToDelete, /*bShowConfirmation=*/false);

			if (!IFileManager::Get().DeleteDirectory(*SrcDiskFolderFilename, /*RequireExists =*/ false, /*Tree =*/ true))
			{
				UE_LOGF(LogAssetTools, Warning, "Failed to delete temporary directory %ls while migrating assets", *SrcDiskFolderFilename);
				CopyErrors += SrcDiskFolderFilename + LINE_TERMINATOR;
			}
		}

		FString SourceControlErrors;

		if ( !bUserCanceled && SuccessfullyCopiedFiles.Num() > 0 )
		{
			// attempt to add files to source control (this can quite easily fail, but if it works it is very useful)
			if(GetDefault<UEditorLoadingSavingSettings>()->bSCCAutoAddNewFiles)
			{
				if(ISourceControlModule::Get().IsEnabled())
				{
					ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
					if(SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), SuccessfullyCopiedFiles) == ECommandResult::Failed)
					{
						FScopedSlowTask SlowTask(static_cast<float>(SuccessfullyCopiedFiles.Num()), LOCTEXT("MigratePackages_AddToSourceControl", "Adding To Revision Control..."));
						SlowTask.MakeDialog();

						for(auto FileIt(SuccessfullyCopiedFiles.CreateConstIterator()); FileIt; FileIt++)
						{
							SlowTask.EnterProgressFrame();
							if(!SourceControlProvider.GetState(*FileIt, EStateCacheUsage::Use)->IsAdded())
							{
								SourceControlErrors += FText::Format(LOCTEXT("MigratePackages_SourceControlError", "{0} could not be added to revision control"), FText::FromString(*FileIt)).ToString();
								SourceControlErrors += LINE_TERMINATOR;
							}
						}
					}
				}
			}
		}

		FText LogMessage = FText::FromString(TEXT("Content migration completed successfully!"));
		EMessageSeverity::Type Severity = EMessageSeverity::Info;
		if ( CopyErrors.Len() > 0 || SourceControlErrors.Len() > 0 )
		{
			FString ErrorMessage;
			Severity = EMessageSeverity::Error;
			if( CopyErrors.Len() > 0 )
			{
				MigrateLog.NewPage( LOCTEXT("MigratePackages_CopyErrorsPage", "Copy Errors") );
				MigrateLog.Error(FText::FromString(*CopyErrors));
				ErrorMessage += FText::Format(LOCTEXT( "MigratePackages_CopyErrors", "Copied {0} files. Some content could not be copied."), FText::AsNumber(SuccessfullyCopiedPackages.Num())).ToString();
			}
			if( SourceControlErrors.Len() > 0 )
			{
				MigrateLog.NewPage( LOCTEXT("MigratePackages_SourceControlErrorsListPage", "Revision Control Errors") );
				MigrateLog.Error(FText::FromString(*SourceControlErrors));
				ErrorMessage += LINE_TERMINATOR;
				ErrorMessage += LOCTEXT( "MigratePackages_SourceControlErrorsList", "Some files reported revision control errors.").ToString();
			}
			if ( SuccessfullyCopiedPackages.Num() > 0 )
			{
				MigrateLog.NewPage( LOCTEXT("MigratePackages_CopyErrorsSuccesslistPage", "Copied Successfully") );
				MigrateLog.Info(FText::FromString(*SourceControlErrors));
				ErrorMessage += LINE_TERMINATOR;
				ErrorMessage += LOCTEXT( "MigratePackages_CopyErrorsSuccesslist", "Some files were copied successfully.").ToString();
				for ( auto FileIt = SuccessfullyCopiedPackages.CreateConstIterator(); FileIt; ++FileIt )
				{
					if(FileIt->Len()>0)
					{
						MigrateLog.Info(FText::FromString(*FileIt));
					}
				}
			}
			LogMessage = FText::FromString(ErrorMessage);
		}
		else if (bUserCanceled)
		{
			LogMessage = LOCTEXT("MigratePackages_CanceledPage", "Content migration was canceled.");
		}
		else if ( !bUserCanceled )
		{
			MigrateLog.NewPage( LOCTEXT("MigratePackages_CompletePage", "Content migration completed successfully!") );
			for ( auto FileIt = SuccessfullyCopiedPackages.CreateConstIterator(); FileIt; ++FileIt )
			{
				if(FileIt->Len()>0)
				{
					MigrateLog.Info(FText::FromString(*FileIt));
				}
			}
		}
		MigrateLog.Notify(LogMessage, Severity, true);
	}
}

bool UAssetToolsImpl::CanMigratePackage(FName PackageName) const
{
	for (const TPair<FName, UE::AssetTools::FCanMigrateAsset>& Pair : CanMigrateAssetDelegates)
	{
		if (Pair.Value.IsBound())
		{
			if (!Pair.Value.Execute(PackageName))
			{
				return false;
			}
		}
	}

	return true;
}

void UAssetToolsImpl::RecursiveGetDependencies(const FName& PackageName, TSet<FName>& AllDependencies, TSet<FString>& OutExternalObjectsPaths, TSet<FName>& ExcludedDependencies, const TFunction<bool (FName)>& ShouldExcludeFromDependenciesSearch) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FName> Dependencies;
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.GetDependencies(PackageName, Dependencies);
	
	for (TArray<FName>::TConstIterator DependsIt = Dependencies.CreateConstIterator(); DependsIt; ++DependsIt)
	{
		FString DependencyName = (*DependsIt).ToString();

		const bool bIsScriptPackage = DependencyName.StartsWith(TEXT("/Script"));

		// The asset registry can give some reference to some deleted assets. We don't want to migrate these.
		const bool bAssetExist = AssetRegistry.GetAssetPackageDataCopy(*DependsIt).IsSet();

		if (!bIsScriptPackage && bAssetExist)
		{
			uint32 DependsHash = GetTypeHash(*DependsIt);
			if (!AllDependencies.ContainsByHash(DependsHash, *DependsIt) && !ExcludedDependencies.ContainsByHash(DependsHash, *DependsIt))
			{
				// Early stop the dependency search
				if (ShouldExcludeFromDependenciesSearch(*DependsIt))
				{
					ExcludedDependencies.AddByHash(DependsHash, *DependsIt);
					continue;
				}

				AllDependencies.AddByHash(DependsHash, *DependsIt);

				RecursiveGetDependencies(*DependsIt, AllDependencies, OutExternalObjectsPaths, ExcludedDependencies, ShouldExcludeFromDependenciesSearch);
			}
		}
	}

	// Handle Specific External Objects use case (only used for the Migrate path for now)
	// todo: revisit how to handle those in a more generic way. Should the FExternalActorAssetDependencyGatherer handle the external objects reference also?
	TArray<FAssetData> Assets;

	// The migration only work on the saved version of the assets so no need to scan the for the in memory only assets. This also greatly improve the performance of the migration when a lot assets are loaded in the editor.
	const bool bOnlyIncludeOnDiskAssets = true;
	if (AssetRegistryModule.Get().GetAssetsByPackageName(PackageName, Assets, bOnlyIncludeOnDiskAssets))
	{
		for (const FAssetData& AssetData : Assets)
		{
			if (AssetData.GetClass() && AssetData.GetClass()->IsChildOf<UWorld>())
			{
				TArray<FString> ExternalObjectsPaths = ULevel::GetExternalObjectsPaths(PackageName.ToString());
				for (const FString& ExternalObjectsPath : ExternalObjectsPaths)
				{
					if (!ExternalObjectsPath.IsEmpty() && !OutExternalObjectsPaths.Contains(ExternalObjectsPath))
					{
						OutExternalObjectsPaths.Add(ExternalObjectsPath);
						AssetRegistryModule.Get().ScanPathsSynchronous({ ExternalObjectsPath }, /*bForceRescan*/true, /*bIgnoreDenyListScanFilters*/true);

						TArray<FAssetData> ExternalObjectAssets;
						AssetRegistryModule.Get().GetAssetsByPath(FName(*ExternalObjectsPath), ExternalObjectAssets, /*bRecursive*/true, bOnlyIncludeOnDiskAssets);

						for (const FAssetData& ExternalObjectAsset : ExternalObjectAssets)
						{
							// We don't expose the early dependency search exit to the external objects/actors since to the users their are same the outer package that own these objects
							AllDependencies.Add(ExternalObjectAsset.PackageName);
							RecursiveGetDependencies(ExternalObjectAsset.PackageName, AllDependencies, OutExternalObjectsPaths, ExcludedDependencies, ShouldExcludeFromDependenciesSearch);
						}
					}
				}
			}
		}
	}
}

void UAssetToolsImpl::RecursiveGetDependenciesAdvanced(const FName& PackageName, FAdvancedCopyParams& CopyParams, TArray<FName>& AllDependencies, TMap<FName, FName>& DependencyMap, const FARCompiledFilter& CompiledExclusionFilter, TArray<FAssetData>& OptionalAssetData) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FName> Dependencies;
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	// We found an asset
	if (OptionalAssetData.Num() > 0)
	{
		AssetRegistryModule.Get().GetDependencies(PackageName, Dependencies);
		for (const FName& Dep : Dependencies)
		{
			if (!AllDependencies.Contains(Dep) && FPackageName::IsValidLongPackageName(Dep.ToString(), false))
			{
				TArray<FAssetData> DependencyAssetData;
				AssetRegistry.GetAssetsByPackageName(Dep, DependencyAssetData, true);
				AssetRegistry.UseFilterToExcludeAssets(DependencyAssetData, CompiledExclusionFilter);
				if (DependencyAssetData.Num() > 0)
				{
					AllDependencies.Add(Dep);
					DependencyMap.Add(Dep, PackageName);
					RecursiveGetDependenciesAdvanced(Dep, CopyParams, AllDependencies, DependencyMap, CompiledExclusionFilter, DependencyAssetData);
				}
			}
		}
	}
	else
	{
		TArray<FAssetData> PathAssetData;
		// We found a folder containing assets
		if (AssetRegistry.HasAssets(PackageName) && AssetRegistry.GetAssetsByPath(PackageName, PathAssetData))
		{

			AssetRegistry.UseFilterToExcludeAssets(PathAssetData, CompiledExclusionFilter);
			for(const FAssetData& Asset : PathAssetData)
			{
				AllDependencies.Add(*Asset.GetPackage()->GetName());
				// If we should check the assets we found for dependencies
				if (CopyParams.bShouldCheckForDependencies)
				{
					RecursiveGetDependenciesAdvanced(FName(*Asset.GetPackage()->GetName()), CopyParams, AllDependencies, DependencyMap, CompiledExclusionFilter, PathAssetData);
				}
			}
		}
		
		// Always get subpaths
		{
			TArray<FString> SubPaths;
			AssetRegistry.GetSubPaths(PackageName.ToString(), SubPaths, false);
			for (const FString& SubPath : SubPaths)
			{
				TArray<FAssetData> EmptyArray;
				RecursiveGetDependenciesAdvanced(FName(*SubPath), CopyParams, AllDependencies, DependencyMap, CompiledExclusionFilter, EmptyArray);
			}

		}
	}
}

void UAssetToolsImpl::FixupReferencers(const TArray<UObjectRedirector*>& Objects, bool bCheckoutDialogPrompt, ERedirectFixupMode FixupMode) const
{
	AssetFixUpRedirectors->FixupReferencers(Objects, bCheckoutDialogPrompt, FixupMode);
}

bool UAssetToolsImpl::IsFixupReferencersInProgress() const
{
	return AssetFixUpRedirectors->IsFixupReferencersInProgress();
}

void UAssetToolsImpl::OpenEditorForAssets(const TArray<UObject*>& Assets)
{
#if WITH_EDITOR
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
#endif
}

void UAssetToolsImpl::ConvertVirtualTextures(const TArray<UTexture2D *>& Textures, bool bConvertBackToNonVirtual, const TArray<UMaterial *>* RelatedMaterials /* = nullptr */) const
{
	IVirtualTexturingEditorModule* Module = FModuleManager::Get().GetModulePtr<IVirtualTexturingEditorModule>("VirtualTexturingEditor");
	Module->ConvertVirtualTextures(Textures, bConvertBackToNonVirtual, RelatedMaterials);
}

void UAssetToolsImpl::BeginAdvancedCopyPackages(const TArray<FName>& InputNamesToCopy, const FString& TargetPath) const
{
	BeginAdvancedCopyPackages(InputNamesToCopy, TargetPath, FAdvancedCopyCompletedEvent());
}

void UAssetToolsImpl::BeginAdvancedCopyPackages(const TArray<FName>& InputNamesToCopy, const FString& TargetPath, const FAdvancedCopyCompletedEvent& OnCopyComplete) const
{
	// Packages must be saved for the migration to work
	const bool bPromptUserToSave = true;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages))
	{
		IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
		if (AssetRegistry.IsLoadingAssets())
		{
			// Open a dialog asking the user to wait while assets are being discovered
			SDiscoveringAssetsDialog::OpenDiscoveringAssetsDialog(
				SDiscoveringAssetsDialog::FOnAssetsDiscovered::CreateUObject(this, &UAssetToolsImpl::PerformAdvancedCopyPackages, InputNamesToCopy, TargetPath, OnCopyComplete)
			);
		}
		else
		{
			// Assets are already discovered, perform the migration now
			PerformAdvancedCopyPackages(InputNamesToCopy, TargetPath, OnCopyComplete);
		}
	}
}

TArray<FName> UAssetToolsImpl::ExpandAssetsAndFoldersToJustAssets(TArray<FName> SelectedAssetAndFolderNames) const
{
	IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();

	TSet<FName> ExpandedAssets;
	for (auto NameIt = SelectedAssetAndFolderNames.CreateIterator(); NameIt; ++NameIt)
	{
		FName OriginalName = *NameIt;
		const FString& OriginalNameString = OriginalName.ToString();
		if (!FPackageName::DoesPackageExist(OriginalNameString))
		{
			TArray<FAssetData> AssetsInFolder;
			AssetRegistry.GetAssetsByPath(OriginalName, AssetsInFolder, true, false);
			for (const FAssetData& Asset : AssetsInFolder)
			{
				ExpandedAssets.Add(Asset.PackageName);
			}

			NameIt.RemoveCurrent();
		}
		else
		{
			ExpandedAssets.Add(OriginalName);
		}
	}

	TArray<FName> ExpandedAssetArray;
	for (FName ExpandedAsset : ExpandedAssets)
	{
		ExpandedAssetArray.Add(ExpandedAsset);
	}

	return ExpandedAssetArray;
}

void UAssetToolsImpl::PerformAdvancedCopyPackages(TArray<FName> SelectedAssetAndFolderNames, FString TargetPath, FAdvancedCopyCompletedEvent OnCopyComplete) const
{	
	TargetPath.RemoveFromEnd(TEXT("/"));

	//TArray<FName> ExpandedAssets = ExpandAssetsAndFoldersToJustAssets(SelectedAssetAndFolderNames);
	FAdvancedCopyParams CopyParams = FAdvancedCopyParams(SelectedAssetAndFolderNames, TargetPath);
	CopyParams.bShouldCheckForDependencies = true;
	CopyParams.OnCopyComplete = OnCopyComplete;

	// Suppress UI if we're running in unattended mode
	if (!CanShowDialogs())
	{
		CopyParams.bShouldSuppressUI = true;
	}

	for (auto NameIt = SelectedAssetAndFolderNames.CreateConstIterator(); NameIt; ++NameIt)
	{
		UAdvancedCopyCustomization* CopyCustomization = nullptr;

		const UAssetToolsSettings* Settings = GetDefault<UAssetToolsSettings>();
		FName OriginalName = *NameIt;
		const FString& OriginalNameString = OriginalName.ToString();
		FString SrcFilename;
		UObject* ExistingObject = nullptr;

		if (FPackageName::DoesPackageExist(OriginalNameString, &SrcFilename))
		{
			UPackage* Pkg = LoadPackage(nullptr, *OriginalNameString, LOAD_None);
			if (Pkg)
			{
				FString Name = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(SrcFilename));
				ExistingObject = StaticFindObject(UObject::StaticClass(), Pkg, *Name);
			}
		}

		if (ExistingObject)
		{
			// Try to find the customization in the settings
			for (const FAdvancedCopyMap& Customization : Settings->AdvancedCopyCustomizations)
			{
				if (Customization.ClassToCopy.GetAssetPathString() == ExistingObject->GetClass()->GetPathName())
				{
					UClass* CustomizationClass = Customization.AdvancedCopyCustomization.TryLoadClass<UAdvancedCopyCustomization>();
					if (CustomizationClass)
					{
						CopyCustomization = CustomizationClass->GetDefaultObject<UAdvancedCopyCustomization>();
					}
				}
			}
		}

		// If not able to find class in settings, fall back to default customization
		// by default, folders will use the default customization
		if (CopyCustomization == nullptr)
		{
			CopyCustomization = UAdvancedCopyCustomization::StaticClass()->GetDefaultObject<UAdvancedCopyCustomization>();
		}

		CopyParams.AddCustomization(CopyCustomization);
	}

	InitAdvancedCopyFromCopyParams(CopyParams);
}


void UAssetToolsImpl::InitAdvancedCopyFromCopyParams(FAdvancedCopyParams CopyParams) const
{
	TArray<TMap<FName, FName>> CompleteDependencyMap;
	TArray<TMap<FString, FString>> CompleteDestinationMap;

	TArray<FName> SelectedPackageNames = CopyParams.GetSelectedPackageOrFolderNames();

	FScopedSlowTask SlowTask(static_cast<float>(SelectedPackageNames.Num()), LOCTEXT("AdvancedCopyPrepareSlowTask", "Preparing Files for Advanced Copy"));
	SlowTask.MakeDialog();

	TArray<UAdvancedCopyCustomization*> CustomizationsToUse = CopyParams.GetCustomizationsToUse();

	for (int32 CustomizationIndex = 0; CustomizationIndex < CustomizationsToUse.Num(); CustomizationIndex++)
	{
		FName Package = SelectedPackageNames[CustomizationIndex];
		SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("AdvancedCopy_PreparingDependencies", "Preparing dependencies for {0}"), FText::FromString(Package.ToString())));

		UAdvancedCopyCustomization* CopyCustomization = CustomizationsToUse[CustomizationIndex];
		CopyCustomization->SetPackageThatInitiatedCopy(Package.ToString());
		// Give the customization a chance to edit the copy parameters
		CopyCustomization->EditCopyParams(CopyParams);
		TMap<FName, FName> DependencyMap = TMap<FName, FName>();
		TArray<FName> PackageNamesToCopy;

		// Get all packages to be copied
		GetAllAdvancedCopySources(Package, CopyParams, PackageNamesToCopy, DependencyMap, CopyCustomization);
		
		// Allow the customization to apply any additional filters
		CopyCustomization->ApplyAdditionalFiltering(PackageNamesToCopy);

		TMap<FString, FString> DestinationMap = TMap<FString, FString>();
		GenerateAdvancedCopyDestinations(CopyParams, PackageNamesToCopy, CopyCustomization, DestinationMap);
		CopyCustomization->TransformDestinationPaths(DestinationMap);
		CompleteDestinationMap.Add(DestinationMap);
		CompleteDependencyMap.Add(DependencyMap);
	}

	// Confirm that there is at least one package to move 
	if (CompleteDestinationMap.Num() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AdvancedCopyPackages_NoFilesToMove", "No files were found to move"));
		return;
	}

	// Prompt the user displaying all assets that are going to be migrated
	if (CopyParams.bShouldSuppressUI)
	{
		AdvancedCopyPackages_ReportConfirmed(CopyParams, CompleteDestinationMap);
	}
	else
	{
		SAdvancedCopyReportDialog::FOnReportConfirmed OnReportConfirmed = SAdvancedCopyReportDialog::FOnReportConfirmed::CreateUObject(this, &UAssetToolsImpl::AdvancedCopyPackages_ReportConfirmed);
		SAdvancedCopyReportDialog::OpenPackageReportDialog(CopyParams, CopyParams.GetDropLocationForAdvancedCopy(), CompleteDestinationMap, CompleteDependencyMap, OnReportConfirmed);
	}
}

void UAssetToolsImpl::AdvancedCopyPackages_ReportConfirmed(const FAdvancedCopyParams& CopyParams, const TArray<TMap<FString, FString>>& DestinationMap) const
{
	TArray<UAdvancedCopyCustomization*> CustomizationsToUse = CopyParams.GetCustomizationsToUse();
	for (int32 CustomizationIndex = 0; CustomizationIndex < CustomizationsToUse.Num(); CustomizationIndex++)
	{
		if (!CustomizationsToUse[CustomizationIndex]->CustomCopyValidate(DestinationMap[CustomizationIndex]))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AdvancedCopy_FailedCustomValidate", "Advanced Copy failed because the validation rules set in {0} failed."),
				FText::FromString(CustomizationsToUse[CustomizationIndex]->GetName())));

			return;
		}
	}
	AdvancedCopyPackages(CopyParams, DestinationMap);
}

bool UAssetToolsImpl::IsAssetClassSupported(const UClass* AssetClass) const
{
	TWeakPtr<IAssetTypeActions> AssetTypeActions = GetAssetTypeActionsForClass(AssetClass);
	if (AssetTypeActions.IsValid())
	{
		const TSharedRef<FPathPermissionList>& CreateAssetPermissionList = GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset);
		if (CreateAssetPermissionList->HasFiltering())
		{
			// Use the Real AssetClass passed from the Factory and not the one from AssetDefinition.
			// This way we avoid issue where the real AssetClass falls back to use the AssetDefinition of its parent.
			FNameBuilder AssetClassPath;
			AssetClass->GetClassPathName().AppendString(AssetClassPath);
			return CreateAssetPermissionList->PassesFilter(AssetClassPath.ToView());	
		}

		return true;
	}
	
	return false;
}

TArray<UFactory*> UAssetToolsImpl::GetNewAssetFactories() const
{
	TArray<UFactory*> Factories;

	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		if (!Class->IsChildOf(UFactory::StaticClass()) || Class->HasAnyClassFlags(CLASS_Abstract) || Class->HasAnyClassFlags(CLASS_Deprecated))
		{
			continue;
		}

		UFactory* Factory = Class->GetDefaultObject<UFactory>();
		if (!Factory->ShouldShowInNewMenu() || !ensure(!Factory->GetDisplayName().IsEmpty()) || !IsAssetClassSupported(Factory->GetSupportedClass()))
		{
			continue;
		}

		// For Blueprints, add sub-type filtering using the BP Editor's allow list
		// Otherwise, there's no way to distinguish between generic BP Actors vs BlueprintFunctionLibraries, etc.
		if (UBlueprintFactory* BPFactory = Cast<UBlueprintFactory>(Factory))
		{
			// Restrict BP factories based on their BlueprintType
			if (AllowedBlueprintTypes.Num() > 0 && !AllowedBlueprintTypes.Contains(BPFactory->BlueprintType))
			{
				continue;
			}

			// BPFactor->ParentClass is stored as a UObject instead of a class... most things should be UClass but we'll just skip it if not
			if (UClass* BPParentClass = Cast<UClass>(BPFactory->ParentClass))
			{
				FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
				if (const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter())
				{
					TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
					FClassViewerInitializationOptions ClassViewerOptions = {};
					if (!GlobalClassFilter->IsClassAllowed(ClassViewerOptions, BPParentClass, ClassFilterFuncs))
					{
						continue;
					}
				}
			}
		}

		Factories.Add(Factory);
	}

	return MoveTemp(Factories);
}

const TSharedRef<FPathPermissionList>& UAssetToolsImpl::GetAssetClassPathPermissionList(EAssetClassAction AssetClassAction) const
{
	if (AssetClassAction < EAssetClassAction::AllAssetActions)
	{
		return AssetClassPermissionList[(int32)AssetClassAction];
	}

	static TSharedRef<FPathPermissionList> Empty = MakeShared<FPathPermissionList>();
	return Empty;
}

const TSharedRef<FNamePermissionList>& UAssetToolsImpl::GetImportExtensionPermissionList() const
{
	return ImportExtensionPermissionList;
}

bool UAssetToolsImpl::IsImportExtensionAllowed(const FStringView& Extension) const
{
	//Always test the extension in lower case
	FName NameExtension(Extension);
	return ImportExtensionPermissionList->PassesFilter(NameExtension);
}

TSet<EBlueprintType>& UAssetToolsImpl::GetAllowedBlueprintTypes()
{
	return AllowedBlueprintTypes;
}

void UAssetToolsImpl::AddSubContentDenyList(const FString& InMount)
{
	for (const FString& SubContentPath : SubContentDenyListPaths)
	{
		FolderPermissionList->AddDenyListItem("AssetToolsConfigFile", InMount / SubContentPath);
	}
}

void UAssetToolsImpl::OnContentPathMounted(const FString& InAssetPath, const FString& FileSystemPath)
{
	AddSubContentDenyList(InAssetPath);
}

TArray<UObject*> UAssetToolsImpl::ImportAssetsWithDialogImplementation(const FString& DestinationPath, bool bAllowAsyncImport)
{
	if (!GetWritableFolderPermissionList()->PassesStartsWithFilter(DestinationPath))
	{
		NotifyBlockedByWritableFolderFilter();
		return TArray<UObject*>();
	}

	TArray<UObject*> ReturnObjects;
	FString FileTypes, AllExtensions;
	TArray<UFactory*> Factories;

	// Get the list of valid factories
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* CurrentClass = (*It);

		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->bEditorImport)
			{
				Factories.Add(Factory);
			}
		}
	}

	TMultiMap<uint32, UFactory*> FilterIndexToFactory;
	// Generate the file types and extensions represented by the selected factories
	ObjectTools::GenerateFactoryFileExtensions(Factories, FileTypes, AllExtensions, FilterIndexToFactory);

	if (UInterchangeManager::IsInterchangeImportEnabled())
	{
		TArray<FString> InterchangeFileExtensions = UInterchangeManager::GetInterchangeManager().GetSupportedFormats(EInterchangeTranslatorType::Assets);
		ObjectTools::AppendFormatsFileExtensions(InterchangeFileExtensions, FileTypes, AllExtensions, FilterIndexToFactory);
	}

	FileTypes = FString::Printf(TEXT("All Files (%s)|%s|%s"), *AllExtensions, *AllExtensions, *FileTypes);

	// Prompt the user for the filenames
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	int32 FilterIndex = -1;

	if (DesktopPlatform)
	{
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("ImportDialogTitle", "Import").ToString(),
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT),
			TEXT(""),
			FileTypes,
			EFileDialogFlags::Multiple,
			OpenFilenames,
			FilterIndex
		);
	}

	if (bOpened)
	{
		if (OpenFilenames.Num() > 0)
		{
			UFactory* ChosenFactory = nullptr;
			// A filter may have been chosen but most likely for convenience, let ImportAssets decipher that.
			if (!UInterchangeManager::IsInterchangeImportEnabled() && FilterIndex > 0)
			{
				ChosenFactory = *FilterIndexToFactory.Find(FilterIndex);
			}


			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, OpenFilenames[0]);
			const bool bSyncToBrowser = false;
			TArray<TPair<FString, FString>>* FilesAndDestination = nullptr;
			ReturnObjects = ImportAssets(OpenFilenames, DestinationPath, ChosenFactory, bSyncToBrowser, FilesAndDestination, bAllowAsyncImport);
		}
	}

	return ReturnObjects;
}

TSharedRef<FPathPermissionList>& UAssetToolsImpl::GetFolderPermissionList()
{
	return FolderPermissionList;
}

TSharedRef<FPathPermissionList>& UAssetToolsImpl::GetWritableFolderPermissionList()
{
	return WritableFolderPermissionList;
}

bool UAssetToolsImpl::IsAssetVisible(const FAssetData& AssetData, bool bCheckAliases) const
{
	// If the class is not visible, the asset is never visible regardless of the path
	if (!GetAssetClassPathPermissionList(EAssetClassAction::ViewAsset)->PassesFilter(AssetData.AssetClassPath.ToString()))
	{
		return false;
	}
	
	// If this asset's package is in a visible folder, then it is visible
	if (FolderPermissionList->PassesStartsWithFilter(AssetData.PackagePath))
	{
		return true;
	}

	// Otherwise, check if any of the asset's aliases are in a visible folder
	if (bCheckAliases)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		const TArray<FString> Aliases = ContentBrowserModule.Get().GetAliasesForPath(AssetData.ToSoftObjectPath());
		for (const FString& Alias : Aliases)
		{
			FStringView AliasPackagePath(Alias);
			int32 LastSlashIndex = INDEX_NONE;
			if (AliasPackagePath.FindLastChar(TEXT('/'), LastSlashIndex))
			{
				AliasPackagePath.LeftInline(LastSlashIndex);
				if (FolderPermissionList->PassesStartsWithFilter(AliasPackagePath))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool UAssetToolsImpl::IsAssetReadOnly(const FAssetData& AssetData) const
{
	// Cooked assets are always read-only
	if (AssetData.HasAnyPackageFlags(PKG_Cooked | PKG_FilterEditorOnly))
	{
		return true;
	}

	// If this asset's package isn't in a writable folder, then it's read-only
	if (!WritableFolderPermissionList->PassesStartsWithFilter(AssetData.PackagePath))
	{
		return true;
	}

	return false;
}

FAssetData UAssetToolsImpl::FindAssetByVersePath(const UE::Core::FVersePath& VersePath, bool bIncludeOnlyOnDiskAssets, bool bSkipARFilteredAssets) const
{
	if (!VersePath.IsValid())
	{
		return {};
	}

	const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	UE::AssetRegistry::EEnumerateAssetsFlags EnumerateAssetsFlags =
		(bIncludeOnlyOnDiskAssets ? UE::AssetRegistry::EEnumerateAssetsFlags::None : UE::AssetRegistry::EEnumerateAssetsFlags::OnlyOnDiskAssets) |
		(bSkipARFilteredAssets ? UE::AssetRegistry::EEnumerateAssetsFlags::None : UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets);

	FAssetData AssetData;

	IPluginManager::Get().EnumerateVersePathMountPoints(VersePath, [&VersePath, bIncludeOnlyOnDiskAssets, bSkipARFilteredAssets, &AssetRegistry, EnumerateAssetsFlags, &AssetData](FStringView MountedAssetPath, FStringView LeafPath)
	{
		auto GetAssetByObjectPath = [bIncludeOnlyOnDiskAssets, bSkipARFilteredAssets, &AssetRegistry](const FStringView& PackageName, const FStringView& AssetName) -> FAssetData
		{
			if (PackageName.Len() >= NAME_SIZE || AssetName.Len() >= NAME_SIZE)
			{
				return {};
			}

			// If we can't find the package name, it must not exist, no need to pollute with random clipboard strings.
			FName PackageFName(PackageName, FNAME_Find);
			if (PackageFName.IsNone())
			{
				return {};
			}

			// If we can't find the asset name, it must not exist, no need to pollute with random clipboard strings.
			FName AssetFName(AssetName, FNAME_Find);
			if (AssetFName.IsNone())
			{
				return {};
			}

			return AssetRegistry.GetAssetByObjectPath(
				FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(PackageFName, AssetFName)), bIncludeOnlyOnDiskAssets, bSkipARFilteredAssets);
		};

		// The last part of the leaf path should be the top level object name.
		FStringView RelativePath;
		FStringView AssetName;
		int32 LastSlashIndex = INDEX_NONE;
		if (LeafPath.FindLastChar(TEXT('/'), LastSlashIndex))
		{
			RelativePath = FStringView(LeafPath).Left(LastSlashIndex + 1);
			AssetName = FStringView(LeafPath).Mid(LastSlashIndex + 1);
		}
		else
		{
			AssetName = LeafPath;
		}

		if (AssetName.IsEmpty())
		{
			// Can't resolve an empty name, keep checking each plug-in.
			return true;
		}

		{
			// Typically, the short package name and top level object name match, so check that first.
			FNameBuilder PackageNameBuilder;
			PackageNameBuilder.Append(MountedAssetPath);
			PackageNameBuilder.Append(RelativePath);
			PackageNameBuilder.Append(AssetName);

			FNameBuilder AssetNameBuilder;
			AssetNameBuilder.Append(AssetName);

			AssetData = GetAssetByObjectPath(PackageNameBuilder, AssetNameBuilder);
			if (AssetData.IsValid())
			{
				// Found a match, stop checking subsequent plug-ins.
				return false;
			}

			// It could be that we have a cooked package that only contains compiled blueprints, so check that next.
			AssetNameBuilder.Append(TEXT("_C"));

			AssetData = GetAssetByObjectPath(PackageNameBuilder, AssetNameBuilder);
			if (AssetData.IsValid())
			{
				// If it is a compiled blueprint, GetVersePath will return a Verse path without the _C.
				if (VersePath == AssetData.GetVersePath())
				{
					// Found a match, stop checking subsequent plug-ins.
					return false;
				}

				AssetData = {};
			}
		}

		// Since we don't know the short package name, and the Verse path could point to a top level object whose name doesn't
		// match the short package name, we need to scan the package path for other possible assets.
		{
			FNameBuilder PackagePathBuilder;
			PackagePathBuilder.Append(MountedAssetPath);
			PackagePathBuilder.Append(RelativePath);

			// Remove trailing '/'.
			PackagePathBuilder.RemoveSuffix(1);

			if (PackagePathBuilder.Len() < NAME_SIZE)
			{
				// If we can't find the package path, it must not exist in the path tree, no need to pollute with random clipboard strings.
				FName PackagePath(PackagePathBuilder.ToView(), FNAME_Find);
				if (!PackagePath.IsNone())
				{
					FARFilter ARFilter;
					ARFilter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;
					ARFilter.PackagePaths.Add(PackagePath);

					AssetRegistry.EnumerateAssets(ARFilter, [VerseClassPath = UVerseClass::StaticClass()->GetClassPathName(), &VersePath, &AssetData](const FAssetData& PossibleAssetData)
					{
						// We will check Verse classes later on.
						if (VerseClassPath != PossibleAssetData.AssetClassPath && VersePath == PossibleAssetData.GetVersePath())
						{
							AssetData = PossibleAssetData;
						}
						return !AssetData.IsValid();
					}, EnumerateAssetsFlags);

					// Check if EnumerateAssets found a match.
					if (AssetData.IsValid())
					{
						// Found a match, stop checking subsequent plug-ins.
						return false;
					}
				}
			}
		}

		// Keep checking each plug-in.
		return true;
	});

	// Check if EnumerateVersePathMountPoints found a match.
	if (!AssetData.IsValid())
	{
		// Check if we have any Verse classes that match the Verse path.
		// These can have arbitrary verse paths since the package Verse path is defined by the UE module (ModuleRules.SetupVerse).
		FARFilter ARFilter;
		ARFilter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;
		ARFilter.ClassPaths.Add(UVerseClass::StaticClass()->GetClassPathName());

		AssetRegistry.EnumerateAssets(ARFilter, [&VersePath, &AssetData](const FAssetData& PossibleAssetData)
		{
			if (VersePath == PossibleAssetData.GetVersePath())
			{
				AssetData = PossibleAssetData;
			}
			return !AssetData.IsValid();
		}, EnumerateAssetsFlags);
	}

	return AssetData;
}

bool UAssetToolsImpl::ShowingContentVersePath() const
{
	if (ShowingContentVersePathDelegate.IsBound())
	{
		return ShowingContentVersePathDelegate.Execute();
	}
	return false;
}

UE::AssetTools::FShowingContentVersePath& UAssetToolsImpl::GetOnShowingContentVersePath()
{
	return ShowingContentVersePathDelegate;
}

const FString& UAssetToolsImpl::GetUserFacingPluginName(const IPlugin& Plugin) const
{
	if (!Plugin.GetVersePath().IsEmpty() && ShowingContentVersePath())
	{
		return Plugin.GetVersePath();
	}

	return Plugin.GetFriendlyName();
}

FString UAssetToolsImpl::GetUserFacingPluginName(FStringView PluginName) const
{
	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
	{
		return GetUserFacingPluginName(*Plugin);
	}

	return FString(PluginName);
}

FString UAssetToolsImpl::GetUserFacingLongPackagePath(FStringView LongPackagePath) const
{
	if (!LongPackagePath.IsEmpty() && ShowingContentVersePath())
	{
		UE::Core::FVersePath VersePath = FPackageName::LongPackagePathToVersePath(LongPackagePath);
		if (VersePath.IsValid())
		{
			return MoveTemp(VersePath).ToString();
		}
	}

	return FString(LongPackagePath);
}

FString UAssetToolsImpl::GetUserFacingLongPackageName(FName PackageName) const
{
	if (!PackageName.IsNone() && ShowingContentVersePath())
	{
		TMap<FName, FAssetData> Assets;
		UE::AssetRegistry::GetAssetForPackages(TConstArrayView<FName>(&PackageName, 1), Assets);

		if (const FAssetData* AssetData = Assets.Find(PackageName))
		{
			UE::Core::FVersePath VersePath = AssetData->GetVersePath();
			if (VersePath.IsValid())
			{
				return MoveTemp(VersePath).ToString();
			}
		}
	}

	return PackageName.ToString();
}

TArray<FString> UAssetToolsImpl::GetUserFacingLongPackageNames(TConstArrayView<FName> PackageNames) const
{
	if (PackageNames.IsEmpty())
	{
		return {};
	}

	TArray<FString> UserFacingPaths;
	UserFacingPaths.Reserve(PackageNames.Num());
	
	if (ShowingContentVersePath())
	{
		TMap<FName, FAssetData> Assets;
		UE::AssetRegistry::GetAssetForPackages(PackageNames, Assets);

		for (const FName& PackageName : PackageNames)
		{
			if (!PackageName.IsNone())
			{
				if (const FAssetData* AssetData = Assets.Find(PackageName))
				{
					UE::Core::FVersePath VersePath = AssetData->GetVersePath();
					if (VersePath.IsValid())
					{
						UserFacingPaths.Add(MoveTemp(VersePath).ToString());
						continue;
					}
				}
			}

			UserFacingPaths.Add(PackageName.ToString());
		}
	}
	else
	{
		for (const FName& PackageName : PackageNames)
		{
			UserFacingPaths.Add(PackageName.ToString());
		}
	}

	return UserFacingPaths;
}

FString UAssetToolsImpl::GetUserFacingObjectPath(const FSoftObjectPath& ObjectPath) const
{
	// If the object path is just a package name, get the path for the package.
	if (ObjectPath.IsAsset() && ObjectPath.GetAssetFName().IsNone())
	{
		return GetUserFacingLongPackageName(ObjectPath.GetLongPackageFName());
	}

	if (ShowingContentVersePath())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData AssetData;
		if (AssetRegistryModule.Get().TryGetAssetByObjectPath(ObjectPath, AssetData) == UE::AssetRegistry::EExists::Exists)
		{
			UE::Core::FVersePath VersePath = AssetData.GetVersePath();
			if (VersePath.IsValid())
			{
				return MoveTemp(VersePath).ToString();
			}
		}
	}

	return ObjectPath.ToString();
}

TArray<FString> UAssetToolsImpl::GetUserFacingObjectPaths(TConstArrayView<FSoftObjectPath> ObjectPaths) const
{
	if (ObjectPaths.IsEmpty())
	{
		return {};
	}

	TArray<FString> UserFacingPaths;
	UserFacingPaths.Reserve(ObjectPaths.Num());

	if (ShowingContentVersePath())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		// Get the primary asset for any object paths that are just package names.
		TMap<FName, FAssetData> Assets;
		{
			TArray<FName> PackageNames;

			for (const FSoftObjectPath& ObjectPath : ObjectPaths)
			{
				if (ObjectPath.IsAsset() && ObjectPath.GetAssetFName().IsNone())
				{
					PackageNames.Add(ObjectPath.GetLongPackageFName());
				}
			}

			if (!PackageNames.IsEmpty())
			{
				UE::AssetRegistry::GetAssetForPackages(PackageNames, Assets);
			}
		}

		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			UE::Core::FVersePath VersePath;

			// If the object path is just a package name, get the Verse path for the primary asset.
			if (ObjectPath.IsAsset() && ObjectPath.GetAssetFName().IsNone())
			{
				if (const FAssetData* AssetData = Assets.Find(ObjectPath.GetLongPackageFName()))
				{
					VersePath = AssetData->GetVersePath();
				}
			}
			else
			{
				FAssetData AssetData;
				if (AssetRegistryModule.Get().TryGetAssetByObjectPath(ObjectPath, AssetData) == UE::AssetRegistry::EExists::Exists)
				{
					VersePath = AssetData.GetVersePath();
				}
			}

			if (VersePath.IsValid())
			{
				UserFacingPaths.Add(MoveTemp(VersePath).ToString());
			}
			else
			{
				UserFacingPaths.Add(ObjectPath.ToString());
			}
		}
	}
	else
	{
		for (const FSoftObjectPath& ObjectPath : ObjectPaths)
		{
			UserFacingPaths.Add(ObjectPath.ToString());
		}
	}

	return UserFacingPaths;
}

FString UAssetToolsImpl::GetUserFacingLongPackageName(const UPackage& Package) const
{
	return GetUserFacingLongPackageName(Package.GetFName());
}

TArray<FString> UAssetToolsImpl::GetUserFacingLongPackageNames(TConstArrayView<const UPackage*> Packages) const
{
	if (Packages.IsEmpty())
	{
		return {};
	}

	TArray<FName, TInlineAllocator<8>> PackageNames;
	PackageNames.Reserve(Packages.Num());

	for (const UPackage* Package : Packages)
	{
		if (ensure(Package))
		{
			PackageNames.Add(Package->GetFName());
		}
		else
		{
			PackageNames.Emplace();
		}
	}

	return GetUserFacingLongPackageNames(PackageNames);
}

FString UAssetToolsImpl::GetUserFacingFullName(const UObject& Object) const
{
	TStringBuilder<256> StringBuilder;

	if (ShowingContentVersePath())
	{
		const UE::Core::FVersePath VersePath = Object.GetVersePath();
		if (VersePath.IsValid())
		{
			Object.GetClass()->GetFName().AppendString(StringBuilder);
			StringBuilder.AppendChar(TEXT(' '));
			StringBuilder.Append(VersePath.ToString());
			return StringBuilder.ToString();
		}
	}

	Object.GetFullName(StringBuilder);
	return StringBuilder.ToString();
}

FString UAssetToolsImpl::GetUserFacingPathName(const UObject& Object) const
{
	if (ShowingContentVersePath())
	{
		UE::Core::FVersePath VersePath = Object.GetVersePath();
		if (VersePath.IsValid())
		{
			return MoveTemp(VersePath).ToString();
		}
	}

	return Object.GetPathName();
}

FString UAssetToolsImpl::GetUserFacingFullName(const FAssetData& AssetData) const
{
	TStringBuilder<256> StringBuilder;

	if (ShowingContentVersePath())
	{
		const UE::Core::FVersePath VersePath = AssetData.GetVersePath();
		if (VersePath.IsValid())
		{
			AssetData.AssetClassPath.AppendString(StringBuilder);
			StringBuilder.AppendChar(TEXT(' '));
			StringBuilder.Append(VersePath.ToString());
			return StringBuilder.ToString();
		}
	}

	AssetData.GetFullName(StringBuilder);
	return StringBuilder.ToString();
}

FString UAssetToolsImpl::GetUserFacingPathName(const FAssetData& AssetData) const
{
	if (ShowingContentVersePath())
	{
		UE::Core::FVersePath VersePath = AssetData.GetVersePath();
		if (VersePath.IsValid())
		{
			return MoveTemp(VersePath).ToString();
		}
	}

	return AssetData.GetObjectPathString();
}

bool UAssetToolsImpl::AllPassWritableFolderFilter(const TArray<FString>& InPaths) const
{
	if (WritableFolderPermissionList->HasFiltering())
	{
		for (const FString& Path : InPaths)
		{
			if (!WritableFolderPermissionList->PassesStartsWithFilter(Path))
			{
				return false;
			}
		}
	}

	return true;
}

void UAssetToolsImpl::NotifyBlockedByWritableFolderFilter() const
{
	FSlateNotificationManager::Get().AddNotification(FNotificationInfo(LOCTEXT("NotifyBlockedByWritableFolderFilter", "Folder is locked")));
}

bool UAssetToolsImpl::IsNameAllowed(const FString& Name, FText* OutErrorMessage) const
{
	for (const TPair<FName, FIsNameAllowed>& DelegatePair : IsNameAllowedDelegates)
	{
		if (!DelegatePair.Value.Execute(Name, OutErrorMessage))
		{
			return false;
		}
	}
	return true;
}

void UAssetToolsImpl::RegisterIsNameAllowedDelegate(const FName OwnerName, FIsNameAllowed Delegate)
{
	IsNameAllowedDelegates.Add(OwnerName, Delegate);
}

void UAssetToolsImpl::UnregisterIsNameAllowedDelegate(const FName OwnerName)
{
	IsNameAllowedDelegates.Remove(OwnerName);
}

bool UAssetToolsImpl::IsObjectPathAllowed(const FString& FullPathWithName, FText* OutErrorMessage /* = nullptr */) const
{
	for (const TPair<FName, FIsObjectPathAllowed>& DelegatePair : IsObjectPathAllowedDelegates)
	{
		if (!DelegatePair.Value.Execute(FullPathWithName, OutErrorMessage))
		{
			return false;
		}
	}
	return true;
}

void UAssetToolsImpl::RegisterIsObjectPathAllowedDelegate(const FName OwnerName, FIsObjectPathAllowed Delegate)
{
	IsObjectPathAllowedDelegates.Add(OwnerName, Delegate);
}

void UAssetToolsImpl::UnregisterIsObjectPathAllowedDelegate(const FName OwnerName)
{
	IsObjectPathAllowedDelegates.Remove(OwnerName);
}

bool UAssetToolsImpl::SanitizeName(FString& NameToSanitize)
{
	//Call the delegates
	for (const TPair<FName, FSanitizeName>& DelegatePair : SanitizeNameDelegates)
	{
		DelegatePair.Value.Execute(NameToSanitize);
	}

	return ensure(IsNameAllowed(NameToSanitize, nullptr));
}

void UAssetToolsImpl::RegisterSanitizeNameDelegate(const FName OwnerName, FSanitizeName Delegate)
{
	SanitizeNameDelegates.Add(OwnerName, Delegate);
}

void UAssetToolsImpl::UnregisterSanitizeNameDelegate(const FName OwnerName)
{
	SanitizeNameDelegates.Remove(OwnerName);
}

TOptional<FString> UAssetToolsImpl::GetDefaultAssetNameForClass(const UClass* Class, const UFactory* Factory, const UObject* SourceObject)
{
	FString DefaultAssetName;
	if (FString* AssetName = DefaultNewAssetNames.Find(Class))
	{
		DefaultAssetName = *AssetName;
	}

	if (const FAssetNamingConventionEntry* NamingConvention = GetDefault<UAssetToolsEditorSettings>()->AssetNamingConventionOverrides.Find(Class))
	{
		DefaultAssetName = FString::Format(TEXT("{0}{1}{2}"),
			{
				NamingConvention->Prefix.Get(TEXT("")),
				DefaultAssetName.IsEmpty() ? NamingConvention->AssetName.Get(Factory ? Factory->GetDefaultNewAssetName() : TEXT("")) : DefaultAssetName,
				NamingConvention->Suffix.Get(TEXT(""))
			});
	}

	if (DefaultAssetName.IsEmpty())
	{
		return TOptional<FString>();
	}

	DefaultAssetName = DefaultAssetName.Replace(TEXT("{source}"), SourceObject ? *SourceObject->GetName() : TEXT(""), ESearchCase::IgnoreCase);

	SanitizeName(DefaultAssetName);

	FString SanitizedDefaultName = ObjectTools::SanitizeInvalidChars(DefaultAssetName, INVALID_LONGPACKAGE_CHARACTERS INVALID_OBJECTNAME_CHARACTERS);

	FText ErrorText;
	if (!FFileHelper::IsFilenameValidForSaving(SanitizedDefaultName, ErrorText))
	{
		UE_LOGF(LogAssetTools, Warning, "The default name '%ls' for class '%ls' is invalid: %ls", *SanitizedDefaultName, *Class->GetName(), *ErrorText.ToString());
		return TOptional<FString>();
	}

	return SanitizedDefaultName;
}

void UAssetToolsImpl::RegisterDefaultAssetNameForClass(const UClass* Class, const FString& DefaultName)
{
	DefaultNewAssetNames.Add(Class, DefaultName);
}

void UAssetToolsImpl::UnregisterDefaultAssetNameForClass(const UClass* Class)
{
	DefaultNewAssetNames.Remove(Class);
}

void UAssetToolsImpl::RegisterCanMigrateAsset(const FName OwnerName, UE::AssetTools::FCanMigrateAsset Delegate)
{
	CanMigrateAssetDelegates.Add(OwnerName, MoveTemp(Delegate));
}

void UAssetToolsImpl::UnregisterCanMigrateAsset(const FName OwnerName)
{
	CanMigrateAssetDelegates.Remove(OwnerName);
}

bool UAssetToolsImpl::CanAssetBePublic(FStringView AssetPath, FText* DisallowReason) const
{
	for (const TPair<FName, UE::AssetTools::FCanAssetBePublic>& Pair : CanAssetBePublicDelegates)
	{
		if (Pair.Value.IsBound())
		{
			if (!Pair.Value.Execute(AssetPath, DisallowReason))
			{
				return false;
			}
		}
	}
	return true;
}

void UAssetToolsImpl::RegisterCanAssetBePublic(const FName OwnerName, UE::AssetTools::FCanAssetBePublic Delegate)
{
	CanAssetBePublicDelegates.Add(OwnerName, MoveTemp(Delegate));
}

void UAssetToolsImpl::UnregisterCanAssetBePublic(const FName OwnerName)
{
	CanAssetBePublicDelegates.Remove(OwnerName);
}

#undef LOCTEXT_NAMESPACE

