// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRenameManager.h"
#include "Misc/NamePermissionList.h"
#include "UObject/StrongObjectPtr.h"
#include "Factories/Factory.h"
#include "AssetTools.generated.h"

class FAssetFixUpRedirectors;
class UToolMenu;
class IClassTypeActions;
class UAutomatedAssetImportData;
class UAssetImportTask;
struct ReportPackageData;

/** Parameters for importing specific set of files */
struct FAssetImportParams
{
	FAssetImportParams()
		: SpecifiedFactory(nullptr)
		, ImportData(nullptr)
		, AssetImportTask(nullptr)
		, bSyncToBrowser(true)
		, bForceOverrideExisting(false)
		, bAutomated(false)
		, bAllowAsyncImport(false)
		, bSceneImport(false)
	{}

	/** Factory to use for importing files */
	TStrongObjectPtr<UFactory> SpecifiedFactory;
	/** Data used to determine rules for importing assets through the automated command line interface */
	const UAutomatedAssetImportData* ImportData;
	/** Script exposed rules and state for importing assets */
	UAssetImportTask* AssetImportTask;
	/** Whether or not to sync the content browser to the assets after import */
	bool bSyncToBrowser : 1;
	/** Whether or not we are forcing existing assets to be overriden without asking */
	bool bForceOverrideExisting : 1;
	/** Whether or not this is an automated import */
	bool bAutomated : 1;
	/** Temporary setting to enable the async import. (We will figure a better solution in a later release) */
	bool bAllowAsyncImport : 1;
	/** Whether or not this is a scene import */
	bool bSceneImport : 1;
};


/** For backwards compatibility */
typedef class UAssetToolsImpl FAssetTools;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UCLASS(transient)
class UAssetToolsImpl : public UObject, public IAssetTools
{
	GENERATED_BODY()

public:
	UAssetToolsImpl(const FObjectInitializer& ObjectInitializer);

	// IAssetTools implementation
	virtual void RegisterAssetTypeActions(const TSharedRef<IAssetTypeActions>& NewActions) override;
	virtual void UnregisterAssetTypeActions(const TSharedRef<IAssetTypeActions>& ActionsToRemove) override;
	virtual void GetAssetTypeActionsList( TArray<TWeakPtr<IAssetTypeActions>>& OutAssetTypeActionsList ) const override;
	virtual TWeakPtr<IAssetTypeActions> GetAssetTypeActionsForClass(const UClass* Class) const override;

	virtual TSharedPtr<ILocalizedAssetTools> GetLocalizedAssetTools() const override;
	virtual bool CanLocalize(const UClass* Class) const override;

	virtual TOptional<FLinearColor> GetTypeColor(const UClass* Class) const override;
	
	virtual TArray<TWeakPtr<IAssetTypeActions>> GetAssetTypeActionsListForClass(const UClass* Class) const override;
	virtual EAssetTypeCategories::Type RegisterAdvancedAssetCategory(FName CategoryKey, FText CategoryDisplayName) override;
	virtual EAssetTypeCategories::Type FindAdvancedAssetCategory(FName CategoryKey) const override;
	virtual void GetAllAdvancedAssetCategories(TArray<FAdvancedAssetCategory>& OutCategoryList) const override;
	virtual void RegisterClassTypeActions(const TSharedRef<IClassTypeActions>& NewActions) override;
	virtual void UnregisterClassTypeActions(const TSharedRef<IClassTypeActions>& ActionsToRemove) override;
	virtual void GetClassTypeActionsList( TArray<TWeakPtr<IClassTypeActions>>& OutClassTypeActionsList ) const override;
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActionsForClass( UClass* Class ) const override;
	virtual UObject* CreateAsset(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FName CallingContext = NAME_None, bool bOverwriteExisting = false) override;
	virtual bool CreateAssetAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete = FAssetCreateComplete(), FAssetCreateCancelled OnCancelled = FAssetCreateCancelled(), FName CallingContext = NAME_None, bool bOverwriteExisting = false) override;
	virtual void CreateAssetsFrom(TConstArrayView<UObject*> SourceObjects, UClass* CreateAssetType, const FString& DefaultSuffix, TFunctionRef<UFactory*(UObject*)> FactoryConstructor, FName CallingContext) override;
	virtual UObject* CreateAssetWithDialog(UClass* AssetClass, UFactory* Factory, FName CallingContext = NAME_None) override;
	virtual UObject* CreateAssetWithDialog(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FName CallingContext, const bool bCallConfigureProperties = true) override;
	virtual bool CreateAssetWithDialogAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled, FName CallingContext) override;
	virtual bool CreateAssetWithDialogAsync(UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete = FAssetCreateComplete(), FAssetCreateCancelled OnCancelled = FAssetCreateCancelled(), FName CallingContext = NAME_None) override;
	virtual UObject* DuplicateAsset(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject) override;
	virtual UObject* DuplicateAssetWithDialog(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject) override;
	virtual UObject* DuplicateAssetWithDialogAndTitle(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject, FText DialogTitle) override;
	virtual void SetCreateAssetsAsExternallyReferenceable(bool bValue) override;
	virtual bool GetCreateAssetsAsExternallyReferenceable() override;
	virtual bool ShouldCreateAssetsAsExternallyReferenceableForPath(const FStringView AssetPath) const override;
	virtual UE::AssetTools::FShouldCreateAssetsAsExternallyReferenceableForPath& GetOnShouldCreateAssetsAsExternallyReferenceableForPath() override;
	virtual bool IsDiscoveringAssetsInProgress() const override;
	virtual void OpenDiscoveringAssetsDialog(const FOnAssetsDiscovered& InOnAssetsDiscovered) override;
	virtual bool RenameAssets(const TArray<FAssetRenameData>& AssetsAndNames) override;
	virtual EAssetRenameResult RenameAssetsWithDialog(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout = false) override;
	virtual void FindSoftReferencesToObject(FSoftObjectPath TargetObject, TArray<UObject*>& ReferencingObjects) override;
	virtual void FindSoftReferencesToObjects(const TArray<FSoftObjectPath>& TargetObjects, TMap<FSoftObjectPath, TArray<UObject*>>& ReferencingObjects) override;
	virtual void RenameReferencingSoftObjectPaths(const TArray<UPackage *> PackagesToCheck, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap) override;
	virtual TArray<UObject*> ImportAssetsWithDialog(const FString& DestinationPath) override;
	virtual void ImportAssetsWithDialogAsync(const FString& DestinationPath) override;
	virtual TArray<UObject*> ImportAssets(const TArray<FString>& Files, const FString& DestinationPath, UFactory* ChosenFactory, bool bSyncToBrowser = true, TArray<TPair<FString, FString>>* FilesAndDestinations = nullptr, bool bAllowAsyncImport = false, bool bSceneImport = false) const override;
	virtual TArray<UObject*> ImportAssetsAutomated(const UAutomatedAssetImportData* ImportData) override;
	virtual void ImportAssetTasks(const TArray<UAssetImportTask*>& ImportTasks) override;
	virtual void ExportAssets(const TArray<FString>& AssetsToExport, const FString& ExportPath) override;
	virtual void ExportAssets(const TArray<UObject*>& AssetsToExport, const FString& ExportPath) const override;
	virtual void ExportAssetsWithCleanFilename(const TArray<UObject*>& AssetsToExport, const FString& ExportPath) const override;
	virtual void ExportAssetsWithDialog(const TArray<UObject*>& AssetsToExport, bool bPromptForIndividualFilenames) override;
	virtual void ExportAssetsWithDialog(const TArray<FString>& AssetsToExport, bool bPromptForIndividualFilenames) override;
	virtual bool CanExportAssets(const TArray<FAssetData>& AssetsToExport) const override;
	virtual void CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName) override;
	virtual bool AssetUsesGenericThumbnail( const FAssetData& AssetData ) const override;
	virtual void DiffAgainstDepot(UObject* InObject, const FString& InPackagePath, const FString& InPackageName) const override;
	virtual void DiffAssets(UObject* OldAsset1, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override;
	virtual FString DumpAssetToTempFile(UObject* Asset) const override;
	virtual bool CreateDiffProcess(const FString& DiffCommand, const FString& OldTextFilename, const FString& NewTextFilename, const FString& DiffArgs = FString("")) const override;
	virtual void MigratePackages(const TArray<FName>& PackageNamesToMigrate) const override;
	virtual void MigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& TargetPath, const struct FMigrationOptions& Options = FMigrationOptions()) const override;
	virtual void MigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& TargetPath, TSet<FName> AllDependenciesToMigrate, TSet<FName> ExcludedDependencies, const struct FMigrationOptions& Options = FMigrationOptions()) const override;
	virtual void BuildMigrateDependencies(const TArray<FName>& PackageNamesToMigrate, TSet<FName>& OutAllDependenciesToMigrate, TSet<FName>& OutExcludedDependencies) const override;
	virtual UE::AssetTools::FOnPackageMigration& GetOnPackageMigration() override;
	virtual void BeginAdvancedCopyPackages(const TArray<FName>& InputNamesToCopy, const FString& TargetPath) const override;
	virtual void BeginAdvancedCopyPackages(const TArray<FName>& InputNamesToCopy, const FString& TargetPath, const FAdvancedCopyCompletedEvent& OnCopyComplete) const override;
	virtual void FixupReferencers(const TArray<UObjectRedirector*>& Objects, bool bCheckoutDialogPrompt = true, ERedirectFixupMode FixupMode = ERedirectFixupMode::DeleteFixedUpRedirectors) const override;
	virtual bool IsFixupReferencersInProgress() const override;
	virtual FAssetPostRenameEvent& OnAssetPostRename() override { return AssetRenameManager->OnAssetPostRenameEvent(); }
	virtual void ExpandDirectories(const TArray<FString>& Files, const FString& DestinationPath, TArray<TPair<FString, FString>>& FilesAndDestinations) const override;
	virtual bool AdvancedCopyPackages(const FAdvancedCopyParams& CopyParams, const TArray<TMap<FString, FString>>& PackagesAndDestinations) const override;
	virtual bool AdvancedCopyPackages(const TMap<FString, FString>& SourceAndDestPackages, const bool bForceAutosave, const bool bCopyOverAllDestinationOverlaps, FDuplicatedObjects* OutDuplicatedObjects, EMessageSeverity::Type NotificationSeverityFilter) const override;
	virtual void GenerateAdvancedCopyDestinations(FAdvancedCopyParams& InParams, const TArray<FName>& InPackageNamesToCopy, const class UAdvancedCopyCustomization* CopyCustomization, TMap<FString, FString>& OutPackagesAndDestinations) const override;
	virtual bool FlattenAdvancedCopyDestinations(const TArray<TMap<FString, FString>>& PackagesAndDestinations, TMap<FString, FString>& FlattenedPackagesAndDestinations) const override;
	virtual bool ValidateFlattenedAdvancedCopyDestinations(const TMap<FString, FString>& FlattenedPackagesAndDestinations) const override;
	virtual void GetAllAdvancedCopySources(FName SelectedPackage, FAdvancedCopyParams& CopyParams, TArray<FName>& OutPackageNamesToCopy, TMap<FName, FName>& DependencyMap, const class UAdvancedCopyCustomization* CopyCustomization) const override;
	virtual void InitAdvancedCopyFromCopyParams(FAdvancedCopyParams CopyParams) const override;
	virtual void OpenEditorForAssets(const TArray<UObject*>& Assets) override;
	virtual void ConvertVirtualTextures(const TArray<UTexture2D*>& Textures, bool bConvertBackToNonVirtual, const TArray<UMaterial*>* RelatedMaterials = nullptr) const override;
	virtual bool IsAssetClassSupported(const UClass* AssetClass) const override;
	virtual TArray<UFactory*> GetNewAssetFactories() const override;
	virtual const TSharedRef<FPathPermissionList>& GetAssetClassPathPermissionList(EAssetClassAction AssetClassAction) const override;
	virtual const TSharedRef<FNamePermissionList>& GetImportExtensionPermissionList() const override;
	virtual bool IsImportExtensionAllowed(const FStringView& Extension) const override;
	virtual TSet<EBlueprintType>& GetAllowedBlueprintTypes() override;
	virtual TSharedRef<FPathPermissionList>& GetFolderPermissionList() override;
	virtual TSharedRef<FPathPermissionList>& GetWritableFolderPermissionList() override;
	virtual bool IsAssetVisible(const FAssetData& AssetData, bool bCheckAliases = true) const override;
	virtual bool IsAssetReadOnly(const FAssetData& AssetData) const override;
	virtual FAssetData FindAssetByVersePath(const UE::Core::FVersePath& VersePath, bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets = true) const override;
	virtual bool ShowingContentVersePath() const override;
	virtual UE::AssetTools::FShowingContentVersePath& GetOnShowingContentVersePath() override;
	virtual const FString& GetUserFacingPluginName(const IPlugin& Plugin) const override;
	virtual FString GetUserFacingPluginName(FStringView PluginName) const override;
	virtual FString GetUserFacingLongPackagePath(FStringView LongPackagePath) const override;
	virtual FString GetUserFacingLongPackageName(FName PackageName) const override;
	virtual TArray<FString> GetUserFacingLongPackageNames(TConstArrayView<FName> PackageNames) const override;
	virtual FString GetUserFacingObjectPath(const FSoftObjectPath& ObjectPath) const override;
	virtual TArray<FString> GetUserFacingObjectPaths(TConstArrayView<FSoftObjectPath> ObjectPaths) const override;
	virtual FString GetUserFacingLongPackageName(const UPackage& Package) const override;
	virtual TArray<FString> GetUserFacingLongPackageNames(TConstArrayView<const UPackage*> Packages) const override;
	virtual FString GetUserFacingFullName(const UObject& Object) const override;
	virtual FString GetUserFacingPathName(const UObject& Object) const override;
	virtual FString GetUserFacingFullName(const FAssetData& AssetData) const override;
	virtual FString GetUserFacingPathName(const FAssetData& AssetData) const override;
	virtual bool AllPassWritableFolderFilter(const TArray<FString>& InPaths) const override;
	virtual void NotifyBlockedByWritableFolderFilter() const;
	virtual bool IsNameAllowed(const FString& Name, FText* OutErrorMessage = nullptr) const override;
	virtual void RegisterIsNameAllowedDelegate(const FName OwnerName, FIsNameAllowed Delegate) override;
	virtual void UnregisterIsNameAllowedDelegate(const FName OwnerName) override;
	virtual bool IsObjectPathAllowed(const FString& FullPathWithName, FText* OutErrorMessage = nullptr) const override;
	virtual void RegisterIsObjectPathAllowedDelegate(const FName OwnerName, FIsObjectPathAllowed Delegate) override;
	virtual void UnregisterIsObjectPathAllowedDelegate(const FName OwnerName) override;
	virtual bool SanitizeName(FString& NameToSanitize) override;
	virtual void RegisterSanitizeNameDelegate(const FName OwnerName, FSanitizeName Delegate) override;
	virtual void UnregisterSanitizeNameDelegate(const FName OwnerName) override;
	virtual TOptional<FString> GetDefaultAssetNameForClass(const UClass* Class, const UFactory* Factory, const UObject* SourceObject) override;
	virtual void RegisterDefaultAssetNameForClass(const UClass* Class, const FString& DefaultName) override;
	virtual void UnregisterDefaultAssetNameForClass(const UClass* Class) override;
	virtual void RegisterCanMigrateAsset(const FName OwnerName, UE::AssetTools::FCanMigrateAsset Delegate) override;
	virtual void UnregisterCanMigrateAsset(const FName OwnerName) override;
	virtual bool CanAssetBePublic(FStringView AssetPath, FText* DisallowReason = nullptr) const override;
	virtual void RegisterCanAssetBePublic(const FName OwnerName, UE::AssetTools::FCanAssetBePublic Delegate) override;
	virtual void UnregisterCanAssetBePublic(const FName OwnerName) override;

	virtual void SyncBrowserToAssets(const TArray<UObject*>& AssetsToSync) override;
	virtual void SyncBrowserToAssets(const TArray<FAssetData>& AssetsToSync) override;

public:
	/** Gets the asset tools singleton as a FAssetTools for asset tools module use */
	static UAssetToolsImpl& Get();

	/** The manager to handle renaming assets */
	TSharedPtr<FAssetRenameManager> AssetRenameManager;

	/** The tools to manage localized variants */
	TSharedPtr<ILocalizedAssetTools> LocalizedAssetTools;

	/** The manager to handle fixing up redirectors */
	TSharedPtr<FAssetFixUpRedirectors> AssetFixUpRedirectors;

	/** Check if the Application can currently show dialogs, meaning the App is attended, Slate is initialized and we are not running an unattended or automation scripts */
	static bool CanShowDialogs();
private:
	static FString GenerateAdvancedCopyDestinationPackageName(const FString& SourcePackage, const FString& SourcePath, const FString& DestinationFolder);

	/** Checks to see if a package is marked for delete, then asks the user if they would like to check in the deleted file before they can continue. Returns true when it is safe to proceed. */
	bool CheckForDeletedPackage(const UPackage* Package) const;

	/** Returns true if the supplied Asset name and package are currently valid for creation. */
	bool CanCreateAsset(const FString& AssetName, const FString& PackageName, const FText& OperationText, bool bOverwriteExisting) const;

	/** Begins the package migration, after assets have been discovered */
	void PerformMigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& DestinationPath, const FMigrationOptions& Options, TSet<FName> AllDependenciesToMigrate, TSet<FName> ExcludedDependencies, bool bSanitizeInputs) const;

	/** Check if an asset can migrated */
	bool CanMigratePackage(FName PackageName) const;

	TArray<FName> ExpandAssetsAndFoldersToJustAssets(TArray<FName> SelectedAssetAndFolderNames) const;

	/** Begins the package advanced copy, after assets have been discovered */
	void PerformAdvancedCopyPackages(TArray<FName> SelectedPackageNames, FString TargetPath, FAdvancedCopyCompletedEvent OnCopyComplete) const;

	/** Copies files after the final list was confirmed */
	void MigratePackages_ReportConfirmed(TSharedPtr<TArray<ReportPackageData>> PackageDataToMigrate, const FString DestinationPath, TSet<FName> ExcludedDependencies, TMap<FName, TSet<FName>> PackageToExternalObjectPackages, const FMigrationOptions Options) const;

	/** Copies files after the final list was confirmed */
	void AdvancedCopyPackages_ReportConfirmed(const FAdvancedCopyParams& CopyParam, const TArray<TMap<FString, FString>>& DestinationMap) const;

	/** Gets the dependencies of the specified package recursively */
	void RecursiveGetDependencies(const FName& PackageName, TSet<FName>& AllDependencies, TSet<FString>& ExternalObjectsPaths, TSet<FName>& ExcludedDependencies, const TFunction<bool(FName)>& ShouldExcludeFromDependenciesSearch) const;

	/** Gets the dependencies of the specified package recursively while omitting things that don't pass the FARFilter passed in from FAdvancedCopyParams */
	void RecursiveGetDependenciesAdvanced(const FName& PackageName, FAdvancedCopyParams& CopyParams, TArray<FName>& AllDependencies, TMap<FName, FName>& DependencyMap, const FARCompiledFilter& CompiledExclusionFilter, TArray<FAssetData>& OptionalAssetData) const;

	/** Records the time taken for an import and reports it to engine analytics, if available */
	static void OnNewImportRecord(UClass* AssetType, const FString& FileExtension, bool bSucceeded, bool bWasCancelled, const FDateTime& StartTime);

	/** Records what assets users are creating */
	static void OnNewCreateRecord(UClass* AssetType, bool bDuplicated);

	/** Internal method that performs the actual asset importing */
	TArray<UObject*> ImportAssetsInternal(const TArray<FString>& Files, const FString& RootDestinationPath, TArray<TPair<FString, FString>> *FilesAndDestinationsPtr, const FAssetImportParams& ImportParams) const;

	/** Internal method to export assets.  If no export path is created a user will be prompted for one.  if bPromptIndividualFilenames is true a user will be asked per file. if bSaveAsCleanFilename is true the file will be saved as the clean filename */
	void ExportAssetsInternal(const TArray<UObject*>& ObjectsToExport, bool bPromptIndividualFilenames, bool bSaveAsCleanFilename, const FString& ExportPath) const;

	UObject* PerformDuplicateAsset(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject, bool bWithDialog);

	/**
	 * Add sub content deny list filter for a new mount point
	 * @param InMount The mount point
	 */
	void AddSubContentDenyList(const FString& InMount);

	/** Called when a new mount is added to add the proper sub content deny list to it. */
	void OnContentPathMounted(const FString& InAssetPath, const FString& FileSystemPath);

	/** Implementation for the import with dialog functions */
	TArray<UObject*> ImportAssetsWithDialogImplementation(const FString& DestinationPath, bool bAllowAsyncImport);

	/** Make sure we're not syncing */
	void SyncAssetTypesToAssetDefinitions();

	/** Helper to remove an entry AssetTypeActionsList given a class returned by GetSupportedClass */
	void RemoveAssetTypeActionBySupportedClass(const UClass* SupportedClass);

	/** Called when async asset creation is completed successfully. */
	void CreateAssetAsync_OnComplete(UFactory* Factory, TStrongObjectPtr<UPackage> Package, UClass* AssetClass, FString AssetName, EObjectFlags Flags,
		FName CallingContext, FString PackageName, FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled);

	/** Called when async asset creation with a file location picker dialog is completed successfully. */
	void CreateAssetWithDialogAsync_OnComplete(UFactory* Factory, UClass* AssetClass, FName CallingContext, FString SaveObjectPath,
		FAssetCreateComplete OnComplete, FAssetCreateCancelled OnCancelled);

private:
	uint64 CachedVersion = 0;
	
	/** The list of all registered AssetTypeActions */
	TArray<TSharedRef<IAssetTypeActions>> AssetTypeActionsList;

	/** Lookup of the index into AssetTypeActionsList given the SupportedClass of a registered IAssetTypeActions */
	TMap<const UClass*, int> AssetTypeActionsLookup;

	/** The list of all registered ClassTypeActions */
	TArray<TSharedRef<IClassTypeActions>> ClassTypeActionsList;

	/** The categories that have been allocated already */
	TMap<FName, FAdvancedAssetCategory> AllocatedCategoryBits;
	
	/** The next user category bit to allocate (set to 0 when there are no more bits left) */
	uint32 NextUserCategoryBit;

	/** This should be removed with the removal of GetAssetClassPermissionList functions */
	TSharedRef<FNamePermissionList> AssetClassPermissionList_DEPRECATED;

	/** Permission lists of assets by class path name, one for each EAssetClassAction */
	TArray<TSharedRef<FPathPermissionList>> AssetClassPermissionList;

	TSharedRef<FNamePermissionList> ImportExtensionPermissionList;
	
	/** Which types of BlueprintFactories are allowed to be returned through GetNewAssetFactories. Empty set allows all types. */
	TSet<EBlueprintType> AllowedBlueprintTypes;

	/** Permission list of folder paths */
	TSharedRef<FPathPermissionList> FolderPermissionList;

	/** Permission list of folder paths to write to */
	TSharedRef<FPathPermissionList> WritableFolderPermissionList;

	/** List of sub content paths denied for every mount. */
	TArray<FString> SubContentDenyListPaths;

	/** Default names to use when creating new assets */
	TMap<const UClass*, FString> DefaultNewAssetNames;

	bool CreateAssetsAsExternallyReferenceable;
	UE::AssetTools::FShouldCreateAssetsAsExternallyReferenceableForPath CreateAssetsAsExternallyReferenceableForPathDelegate;

	UE::AssetTools::FShowingContentVersePath ShowingContentVersePathDelegate;

	TMap<FName, FIsNameAllowed> IsNameAllowedDelegates;
	TMap<FName, FIsObjectPathAllowed> IsObjectPathAllowedDelegates;
	TMap<FName, FSanitizeName> SanitizeNameDelegates;

	UE::AssetTools::FOnPackageMigration OnPackageMigration;

	TMap<FName, UE::AssetTools::FCanMigrateAsset> CanMigrateAssetDelegates;
	
	TMap<FName, UE::AssetTools::FCanAssetBePublic> CanAssetBePublicDelegates;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS
