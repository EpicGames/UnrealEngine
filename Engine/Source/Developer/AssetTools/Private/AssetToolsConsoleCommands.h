// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define LOCTEXT_NAMESPACE "AssetTools"

class FAssetToolsModule;

class FAssetToolsConsoleCommands
{
public:
	const FAssetToolsModule& Module;
	
	//FAutoConsoleCommand CreateCommand;
	
	FAssetToolsConsoleCommands(const FAssetToolsModule& InModule)
		: Module(InModule)
	//,	CreateCommand(
	//	TEXT( "CollectionManager.Create" ),
	//	*LOCTEXT("CommandText_Create", "Creates a collection of the specified name and type").ToString(),
	//	FConsoleCommandWithArgsDelegate::CreateRaw( this, &FCollectionManagerConsoleCommands::Create ) )
	{}

	//void Create(const TArray<FString>& Args)
	//{
	//}
};


#undef LOCTEXT_NAMESPACE
