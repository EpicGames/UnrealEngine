// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetToolsModule.h"
#include "AssetToolsLog.h"
#include "AssetTools.h"
#include "AssetToolsConsoleCommands.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"

IMPLEMENT_MODULE( FAssetToolsModule, AssetTools );
DEFINE_LOG_CATEGORY(LogAssetTools);

void FAssetToolsModule::StartupModule()
{
	ConsoleCommands = new FAssetToolsConsoleCommands(*this);

	AssetToolsPtr = MakeWeakObjectPtr(const_cast<UAssetToolsImpl*>(GetDefault<UAssetToolsImpl>()));

	// create a message log for the asset tools to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowPages = true;
	MessageLogModule.RegisterLogListing("AssetTools", NSLOCTEXT("AssetTools", "AssetToolsLogLabel", "Asset Tools"), InitOptions);
}

void FAssetToolsModule::ShutdownModule()
{
	AssetToolsPtr = nullptr;

	if (ConsoleCommands != NULL)
	{
		delete ConsoleCommands;
		ConsoleCommands = NULL;
	}

	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		// unregister message log
		FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing("AssetTools");
	}
}

IAssetTools& FAssetToolsModule::Get() const
{
	return *AssetToolsPtr;
}
