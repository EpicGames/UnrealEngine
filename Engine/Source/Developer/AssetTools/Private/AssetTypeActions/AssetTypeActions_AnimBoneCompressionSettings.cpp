// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_AnimBoneCompressionSettings.h"
#include "Animation/AnimSequence.h"
#include "Dialogs/Dialogs.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectIterator.h"


#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FAssetTypeActions_AnimBoneCompressionSettings::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	TSharedRef<FSimpleAssetEditor> AssetEditor = FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, InObjects);

	auto SettingAssets = GetTypedWeakObjectPtrs<UAnimBoneCompressionSettings>(InObjects);
	if (SettingAssets.Num() == 1)
	{
		TSharedPtr<class FUICommandList> PluginCommands = MakeShareable(new FUICommandList);
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
		ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, PluginCommands, FToolBarExtensionDelegate::CreateRaw(this, &FAssetTypeActions_AnimBoneCompressionSettings::AddToolbarExtension, SettingAssets[0]));
		AssetEditor->AddToolbarExtender(ToolbarExtender);

		AssetEditor->RegenerateMenusAndToolbars();
	}
}

void FAssetTypeActions_AnimBoneCompressionSettings::AddToolbarExtension(FToolBarBuilder& Builder, TWeakObjectPtr<UAnimBoneCompressionSettings> BoneSettings)
{
	Builder.BeginSection("Compress");
	Builder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_AnimBoneCompressionSettings::ExecuteCompression, BoneSettings)
		),
		NAME_None,
		LOCTEXT("AnimBoneCompressionSettings_Compress", "Compress"),
		LOCTEXT("AnimBoneCompressionSettings_CompressTooltip", "All animation sequences that use these settings will be compressed."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ApplyCompression")
	);
	Builder.EndSection();
}

void FAssetTypeActions_AnimBoneCompressionSettings::GetActions(const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder)
{
	auto SettingAssets = GetTypedWeakObjectPtrs<UAnimBoneCompressionSettings>(InObjects);

	if (SettingAssets.Num() != 1)
	{
		return;
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AnimBoneCompressionSettings_Compress", "Compress"),
		LOCTEXT("AnimBoneCompressionSettings_CompressTooltip", "All animation sequences that use these settings will be compressed."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ApplyCompression.Small"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_AnimBoneCompressionSettings::ExecuteCompression, SettingAssets[0])
		)
	);
}

void FAssetTypeActions_AnimBoneCompressionSettings::ExecuteCompression(TWeakObjectPtr<UAnimBoneCompressionSettings> BoneSettings)
{
	if (!BoneSettings.IsValid())
	{
		return;
	}

	UAnimBoneCompressionSettings* Settings = BoneSettings.Get();

	TArray<UAnimSequence*> AnimSeqsToRecompress;
	for (TObjectIterator<UAnimSequence> It; It; ++It)
	{
		UAnimSequence* AnimSeq = *It;
		if (AnimSeq->GetOutermost() == GetTransientPackage())
		{
			continue;
		}

		if (AnimSeq->BoneCompressionSettings == Settings)
		{
			AnimSeqsToRecompress.Add(AnimSeq);
		}
	}

	if (AnimSeqsToRecompress.Num() == 0)
	{
		return;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("NumAnimSequences"), FText::AsNumber(AnimSeqsToRecompress.Num()));
	FText DialogText = FText::Format(LOCTEXT("AnimBoneCompressionSettings_CompressWarningText", "{NumAnimSequences} animation sequences are about to compress."), Arguments);
	FText DialogTitle = LOCTEXT("AnimBoneCompressionSettings_CompressWarning", "Warning");
	const EAppReturnType::Type DlgResult = FMessageDialog::Open(EAppMsgType::OkCancel, DialogText, DialogTitle);
	if (DlgResult != EAppReturnType::Ok)
	{
		return;
	}

	const FText StatusText = FText::Format(LOCTEXT("AnimBoneCompressionSettings_Compressing", "Compressing '{0}' animations"), FText::AsNumber(AnimSeqsToRecompress.Num()));
	FScopedSlowTask LoadingAnimSlowTask(static_cast<float>(AnimSeqsToRecompress.Num()), StatusText);
	LoadingAnimSlowTask.MakeDialog();

	for (UAnimSequence* AnimSeq : AnimSeqsToRecompress)
	{
		LoadingAnimSlowTask.EnterProgressFrame();
		AnimSeq->CacheDerivedDataForCurrentPlatform();
	}
}

const TArray<FText>& FAssetTypeActions_AnimBoneCompressionSettings::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("AnimAdvancedSubMenu", "Advanced")
	};
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE