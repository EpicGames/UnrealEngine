// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_AnimCurveCompressionSettings.h"
#include "Animation/AnimSequence.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FAssetTypeActions_AnimCurveCompressionSettings::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	TSharedRef<FSimpleAssetEditor> AssetEditor = FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, InObjects);

	auto SettingAssets = GetTypedWeakObjectPtrs<UAnimCurveCompressionSettings>(InObjects);
	if (SettingAssets.Num() == 1)
	{
		TSharedPtr<class FUICommandList> PluginCommands = MakeShareable(new FUICommandList);
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
		ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, PluginCommands, FToolBarExtensionDelegate::CreateRaw(this, &FAssetTypeActions_AnimCurveCompressionSettings::AddToolbarExtension, SettingAssets[0]));
		AssetEditor->AddToolbarExtender(ToolbarExtender);

		AssetEditor->RegenerateMenusAndToolbars();
	}
}

void FAssetTypeActions_AnimCurveCompressionSettings::AddToolbarExtension(FToolBarBuilder& Builder, TWeakObjectPtr<UAnimCurveCompressionSettings> CurveSettings)
{
	Builder.BeginSection("Compress");
	Builder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_AnimCurveCompressionSettings::ExecuteCompression, CurveSettings)
		),
		NAME_None,
		LOCTEXT("AnimCurveCompressionSettings_Compress", "Compress"),
		LOCTEXT("AnimCurveCompressionSettings_CompressTooltip", "All animation sequences that use these settings will be compressed."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ApplyCompression")
		);
	Builder.EndSection();
}

void FAssetTypeActions_AnimCurveCompressionSettings::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	auto SettingAssets = GetTypedWeakObjectPtrs<UAnimCurveCompressionSettings>(InObjects);

	if (SettingAssets.Num() != 1)
	{
		return;
	}

	Section.AddMenuEntry(
		"AnimCurveCompressionSettings_Compress",
		LOCTEXT("AnimCurveCompressionSettings_Compress", "Compress"),
		LOCTEXT("AnimCurveCompressionSettings_CompressTooltip", "All animation sequences that use these settings will be compressed."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ApplyCompression.Small"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_AnimCurveCompressionSettings::ExecuteCompression, SettingAssets[0])
		)
	);
}

void FAssetTypeActions_AnimCurveCompressionSettings::ExecuteCompression(TWeakObjectPtr<UAnimCurveCompressionSettings> CurveSettings)
{
	if (!CurveSettings.IsValid())
	{
		return;
	}

	UAnimCurveCompressionSettings* Settings = CurveSettings.Get();

	TArray<UAnimSequence*> AnimSeqsToRecompress;
	for (TObjectIterator<UAnimSequence> It; It; ++It)
	{
		UAnimSequence* AnimSeq = *It;
		if (AnimSeq->GetOutermost() == GetTransientPackage())
		{
			continue;
		}

		if (AnimSeq->CurveCompressionSettings == Settings)
		{
			AnimSeqsToRecompress.Add(AnimSeq);
		}
	}

	if (AnimSeqsToRecompress.Num() == 0)
	{
		return;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("NumAnimSequences"), FText::AsNumber(AnimSeqsToRecompress.Num()));
	FText DialogText = FText::Format(LOCTEXT("AnimCurveCompressionSettings_CompressWarningText", "{NumAnimSequences} animation sequences are about to compress."), Arguments);
	FText DialogTitle = LOCTEXT("AnimCurveCompressionSettings_CompressWarning", "Warning");
	const EAppReturnType::Type DlgResult = FMessageDialog::Open(EAppMsgType::OkCancel, DialogText, DialogTitle);
	if (DlgResult != EAppReturnType::Ok)
	{
		return;
	}

	const FText StatusText = FText::Format(LOCTEXT("AnimCurveCompressionSettings_Compressing", "Compressing '{0}' animations"), FText::AsNumber(AnimSeqsToRecompress.Num()));
	FScopedSlowTask LoadingAnimSlowTask(static_cast<float>(AnimSeqsToRecompress.Num()), StatusText);
	LoadingAnimSlowTask.MakeDialog();

	for (UAnimSequence* AnimSeq : AnimSeqsToRecompress)
	{
		LoadingAnimSlowTask.EnterProgressFrame();
		AnimSeq->CacheDerivedDataForCurrentPlatform();
	}
}

const TArray<FText>& FAssetTypeActions_AnimCurveCompressionSettings::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("AnimAdvancedSubMenu", "Advanced")
	};
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
