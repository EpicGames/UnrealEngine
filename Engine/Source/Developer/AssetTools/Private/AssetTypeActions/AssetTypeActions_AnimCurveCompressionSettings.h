// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"
#include "Animation/AnimCurveCompressionSettings.h"

class FAssetTypeActions_AnimCurveCompressionSettings : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimCurveCompressionSettings", "Curve Compression Settings"); }
	virtual FColor GetTypeColor() const override { return FColor(255, 255, 0); }
	virtual UClass* GetSupportedClass() const override { return UAnimCurveCompressionSettings::StaticClass(); }
	virtual bool CanFilter() override { return true; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual const TArray<FText>& GetSubMenus() const override;

	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;

	virtual void GetActions(const TArray<UObject*>& InObjects, struct FToolMenuSection& Section) override;

private:
	void AddToolbarExtension(FToolBarBuilder& Builder, TWeakObjectPtr<UAnimCurveCompressionSettings> CurveSettings);
	void ExecuteCompression(TWeakObjectPtr<UAnimCurveCompressionSettings> CurveSettings);
};
