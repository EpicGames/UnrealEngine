// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_AnimationAsset.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "AssetTools.h"
#include "PersonaModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SSkeletonWidget.h"
#include "IAnimationEditorModule.h"
#include "Preferences/PersonaOptions.h"
#include "ToolMenus.h"
#include "Framework/Application/SlateApplication.h"
#include "Algo/Transform.h"
#include "IPersonaToolkit.h"
#if WITH_EDITOR
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "AssetTypeActions"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UThumbnailInfo* FAssetTypeActions_AnimationAsset::GetThumbnailInfo(UObject* Asset) const
{
	UAnimationAsset* Anim = CastChecked<UAnimationAsset>(Asset);
	UThumbnailInfo* ThumbnailInfo = Anim->ThumbnailInfo;
	if (ThumbnailInfo == NULL)
	{
		ThumbnailInfo = NewObject<USceneThumbnailInfo>(Anim, NAME_None, RF_Transactional);
		Anim->ThumbnailInfo = ThumbnailInfo;
	}

	return ThumbnailInfo;
}

void FAssetTypeActions_AnimationAsset::OpenAnimAssetEditor(const TArray<UObject*>& InObjects, bool bForceNewEditor, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

#if WITH_EDITOR
	// Force opening a new tab when the Animation Editor options window has the 'Always Open In New Tab' option enabled.
	const UPersonaOptions* PersonaOptions = GetDefault<UPersonaOptions>();
	check(PersonaOptions);
	bForceNewEditor |= PersonaOptions->bAlwaysOpenAnimationAssetsInNewTab;
#endif

	// Force new tab when shift is held down.
	bForceNewEditor |= FSlateApplication::Get().GetModifierKeys().IsShiftDown();

	// Find all the anim assets
	TArray<UAnimationAsset*> AnimAssets;
	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UAnimationAsset* AnimAsset = Cast<UAnimationAsset>(*ObjIt))
		{
			AnimAssets.Add(AnimAsset);
		}
	}

	// For each one..
	for (UAnimationAsset* AnimAsset : AnimAssets)
	{
		USkeleton* AnimSkeleton = AnimAsset->GetSkeleton();
		if (!AnimSkeleton)
		{
			FText ShouldRetargetMessage = LOCTEXT("ShouldRetargetAnimAsset_Message", "Could not find the skeleton for Anim '{AnimName}' Would you like to choose a new one?");

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("AnimName"), FText::FromString(AnimAsset->GetName()));

			if (FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(ShouldRetargetMessage, Arguments)) == EAppReturnType::Yes)
			{
				TArray<TObjectPtr<UObject>> AssetsToRetarget;
				AssetsToRetarget.Add(AnimAsset);
				const bool bSkeletonReplaced = ReplaceMissingSkeleton(AssetsToRetarget);
				if (!bSkeletonReplaced)
				{
					return; // Persona will crash if trying to load asset without a skeleton
				}
			}
			else
			{
				return;
			}
		}
		
		// First see if we already have it open
		const bool bBringToFrontIfOpen = true;
#if WITH_EDITOR
		if (IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimAsset, bBringToFrontIfOpen))
		{
			EditorInstance->FocusWindow();
		}
		else
#endif
		{
			// See if we are trying to open a single asset. If we are, we re-use a compatible anim editor.
			bool bSingleAsset = AnimAssets.Num() == 1;
			bool bFoundEditor = false;
			if (bSingleAsset && !bForceNewEditor)
			{
				// See if there is an animation asset with the same skeleton already being edited
				TArray<UObject*> AllEditedAssets;
#if WITH_EDITOR
				AllEditedAssets = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetAllEditedAssets();
#endif
				UAnimationAsset* CompatibleEditedAsset = nullptr;
				for (UObject* EditedAsset : AllEditedAssets)
				{
					UAnimationAsset* EditedAnimAsset = Cast<UAnimationAsset>(EditedAsset);
					if (EditedAnimAsset && EditedAnimAsset->GetSkeleton() == AnimAsset->GetSkeleton())
					{
						CompatibleEditedAsset = EditedAnimAsset;
						break;
					}
				}

				// If there is..
				if(CompatibleEditedAsset)
				{
					// Find the anim editors that are doing it
					TArray<IAssetEditorInstance*> AssetEditors;
#if WITH_EDITOR
					AssetEditors = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorsForAsset(CompatibleEditedAsset);
#endif
					for (IAssetEditorInstance* ExistingEditor : AssetEditors)
					{
						if (ExistingEditor->GetEditorName() == FName("AnimationEditor"))
						{
							// Change the current anim to this one
							IAnimationEditor* AnimEditor = static_cast<IAnimationEditor*>(ExistingEditor);
							if(AnimEditor->GetPersonaToolkit()->GetSkeleton() == AnimAsset->GetSkeleton())
							{
								AnimEditor->SetAnimationAsset(AnimAsset);
								AnimEditor->FocusWindow();
								bFoundEditor = true;
								break;
							}
						}
					}
				}
			}

			// We didn't find an editor, make a new one
			if (!bFoundEditor)
			{
				IAnimationEditorModule& AnimationEditorModule = FModuleManager::LoadModuleChecked<IAnimationEditorModule>("AnimationEditor");
				AnimationEditorModule.CreateAnimationEditor(Mode, EditWithinLevelEditor, AnimAsset);
			}
		}
	}
}

void FAssetTypeActions_AnimationAsset::ExecuteOpenInNewWindow(TArray<TWeakObjectPtr<UAnimationAsset>> Objects)
{
	TArray<UObject*> ObjectsToSync;
	Algo::Transform(Objects, ObjectsToSync, [](TWeakObjectPtr<UAnimationAsset> Obj) { return Obj.Get(); });

	OpenAnimAssetEditor(ObjectsToSync, true, nullptr);
}


void FAssetTypeActions_AnimationAsset::OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor )
{
	OpenAnimAssetEditor(InObjects, false, EditWithinLevelEditor);
}

void FAssetTypeActions_AnimationAsset::ExecuteFindSkeleton(TArray<TWeakObjectPtr<UAnimationAsset>> Objects)
{
	TArray<UObject*> ObjectsToSync;
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		auto Object = (*ObjIt).Get();
		if ( Object )
		{
			USkeleton* Skeleton = Object->GetSkeleton();
			if (Skeleton)
			{
				ObjectsToSync.AddUnique(Skeleton);
			}
		}
	}

	if ( ObjectsToSync.Num() > 0 )
	{
		FAssetTools::Get().SyncBrowserToAssets(ObjectsToSync);
	}
}

void FAssetTypeActions_AnimationAsset::ExecuteReplaceSkeleton(TArray<TWeakObjectPtr<UAnimationAsset>> Objects)
{
	TArray<TObjectPtr<UObject>> LoadedAnimAssets;
	for (const TWeakObjectPtr<UAnimationAsset>& AnimAsset : Objects)
	{
		LoadedAnimAssets.Add(AnimAsset.Get());
	}

	ReplaceMissingSkeleton(LoadedAnimAssets);
}

bool FAssetTypeActions_AnimationAsset::ReplaceMissingSkeleton(TArray<TObjectPtr<UObject>> InAnimationAssets) const
{
	// record anim assets that need skeleton replaced
	const TArray<TWeakObjectPtr<UObject>> AnimsToFix = GetTypedWeakObjectPtrs<UObject>(InAnimationAssets);
	// get a skeleton from the user and replace it
	const TSharedPtr<SReplaceMissingSkeletonDialog> PickSkeletonWindow = SNew(SReplaceMissingSkeletonDialog).AnimAssets(AnimsToFix);
	const bool bWasSkeletonReplaced = PickSkeletonWindow.Get()->ShowModal();
	return bWasSkeletonReplaced;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
