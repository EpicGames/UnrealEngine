// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_Blueprint.h"
#include "ToolMenus.h"
#include "GameFramework/Actor.h"
#include "Misc/PackageName.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "Factories/BlueprintFactory.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditor.h"
#include "SBlueprintDiff.h"
#include "Logging/MessageLog.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#if WITH_EDITOR
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FAssetTypeActions_Blueprint::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	TArray<TWeakObjectPtr<UBlueprint>> Blueprints = GetTypedWeakObjectPtrs<UBlueprint>(InObjects);
	
	if (Blueprints.Num() > 1)
	{
		// Ensure that all the selected blueprints are actors
		bool bCanEditSharedDefaults = true;
		for (const TWeakObjectPtr<UBlueprint>& Blueprint : Blueprints)
		{
			if (!Blueprint.Get()->ParentClass->IsChildOf(AActor::StaticClass()))
			{
				bCanEditSharedDefaults = false;
				break;
			}
		}

		if (bCanEditSharedDefaults)
		{
			Section.AddMenuEntry(
				"Blueprint_EditDefaults",
				LOCTEXT("Blueprint_EditDefaults", "Edit Shared Defaults"),
				LOCTEXT("Blueprint_EditDefaultsTooltip", "Edit the shared default properties of the selected blueprints."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.BlueprintDefaults"),
				FUIAction(
					FExecuteAction::CreateSP( this, &FAssetTypeActions_Blueprint::ExecuteEditDefaults, Blueprints ),
					FCanExecuteAction()
					)
				);
		}
	}

	if ( Blueprints.Num() == 1 && CanCreateNewDerivedBlueprint() )
	{
		TAttribute<FText>::FGetter DynamicTooltipGetter;
		DynamicTooltipGetter.BindSP(this, &FAssetTypeActions_Blueprint::GetNewDerivedBlueprintTooltip, Blueprints[0]);
		TAttribute<FText> DynamicTooltipAttribute = TAttribute<FText>::Create(DynamicTooltipGetter);

		Section.AddMenuEntry(
			"Blueprint_NewDerivedBlueprint",
			LOCTEXT("Blueprint_NewDerivedBlueprint", "Create Child Blueprint Class"),
			DynamicTooltipAttribute,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Blueprint"),
			FUIAction(
				FExecuteAction::CreateSP( this, &FAssetTypeActions_Blueprint::ExecuteNewDerivedBlueprint, Blueprints[0] ),
				FCanExecuteAction::CreateSP( this, &FAssetTypeActions_Blueprint::CanExecuteNewDerivedBlueprint, Blueprints[0] )
				)
			);
	}
}

void FAssetTypeActions_Blueprint::OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor )
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UObject* Object : InObjects)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(Object))
		{
			bool bLetOpen = true;
			if (!Blueprint->SkeletonGeneratedClass || !Blueprint->GeneratedClass)
			{
				bLetOpen = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("FailedToLoadBlueprintWithContinue", "Blueprint could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed! Do you want to continue (it can crash the editor)?"));
			}
			if (bLetOpen)
			{
				FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
				TSharedRef< IBlueprintEditor > NewKismetEditor = BlueprintEditorModule.CreateBlueprintEditor(Mode, EditWithinLevelEditor, Blueprint, ShouldUseDataOnlyEditor(Blueprint));
			}
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("FailedToLoadBlueprint", "Blueprint could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed!"));
		}
	}
}

bool FAssetTypeActions_Blueprint::CanMerge() const
{
	return true;
}

void FAssetTypeActions_Blueprint::Merge(UObject* InObject)
{
	UBlueprint* AsBlueprint = CastChecked<UBlueprint>(InObject);
	// Kludge to get the merge panel in the blueprint editor to show up:
#if WITH_EDITOR
	bool Success = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InObject);
	if( Success )
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );

		FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AsBlueprint, false));
		BlueprintEditor->CreateMergeToolTab();
	}
#endif
}

void FAssetTypeActions_Blueprint::Merge(UObject* BaseAsset, UObject* RemoteAsset, UObject* LocalAsset, const FOnMergeResolved& ResolutionCallback)
{
	UBlueprint* AsBlueprint = CastChecked<UBlueprint>(LocalAsset);
	check(LocalAsset->GetClass() == BaseAsset->GetClass());
	check(LocalAsset->GetClass() == RemoteAsset->GetClass());
#if WITH_EDITOR
	if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AsBlueprint))
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");

		FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AsBlueprint, /*bFocusIfOpen =*/false));
		BlueprintEditor->CreateMergeToolTab(Cast<UBlueprint>(BaseAsset), Cast<UBlueprint>(RemoteAsset), ResolutionCallback);
	}
#endif
}

bool FAssetTypeActions_Blueprint::CanCreateNewDerivedBlueprint() const
{
	return true;
}

UFactory* FAssetTypeActions_Blueprint::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
	BlueprintFactory->ParentClass = InBlueprint->GeneratedClass;
	return BlueprintFactory;
}

void FAssetTypeActions_Blueprint::ExecuteEditDefaults(TArray<TWeakObjectPtr<UBlueprint>> Objects)
{
	TArray< UBlueprint* > Blueprints;

	FMessageLog EditorErrors("EditorErrors");
	EditorErrors.NewPage(LOCTEXT("ExecuteEditDefaultsNewLogPage", "Loading Blueprints"));

	for (const TWeakObjectPtr<UBlueprint>& WeakBP : Objects)
	{
		if (UBlueprint* Object = WeakBP.Get())
		{
			// If the blueprint is valid, allow it to be added to the list, otherwise log the error.
			if ( Object->SkeletonGeneratedClass && Object->GeneratedClass )
			{
				Blueprints.Add(Object);
			}
			else
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ObjectName"), FText::FromString(Object->GetName()));
				EditorErrors.Error(FText::Format(LOCTEXT("LoadBlueprint_FailedLog", "{ObjectName} could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed!"), Arguments ) );
			}
		}
	}

	if ( Blueprints.Num() > 0 )
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );
		TSharedRef< IBlueprintEditor > NewBlueprintEditor = BlueprintEditorModule.CreateBlueprintEditor(  EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), Blueprints );
	}

	// Report errors
	EditorErrors.Notify(LOCTEXT("OpenDefaults_Failed", "Opening Class Defaults Failed!"));
}

void FAssetTypeActions_Blueprint::ExecuteNewDerivedBlueprint(TWeakObjectPtr<UBlueprint> InObject)
{
	if (UBlueprint* Object = InObject.Get())
	{
		// The menu option should ONLY be available if there is only one blueprint selected, validated by the menu creation code
		UBlueprint* TargetParentBP = Object;
		UClass* TargetParentClass = TargetParentBP->GeneratedClass;

		if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(TargetParentClass))
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidClassToMakeBlueprintFrom", "Invalid class with which to make a Blueprint."));
			return;
		}

		FString Name;
		FString PackageName;
		CreateUniqueAssetName(Object->GetOutermost()->GetName(), TEXT("_Child"), PackageName, Name);
		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		UFactory* Factory = GetFactoryForBlueprintType(TargetParentBP);

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().CreateNewAsset(Name, PackagePath, TargetParentBP->GetClass(), Factory);
	}
}

FText FAssetTypeActions_Blueprint::GetNewDerivedBlueprintTooltip(TWeakObjectPtr<UBlueprint> InObject)
{
	if (!CanExecuteNewDerivedBlueprint(InObject))
	{
		return LOCTEXT("Blueprint_NewDerivedBlueprintIsDeprecatedTooltip", "Blueprint class is deprecated, cannot derive a child Blueprint!");
	}
	else
	{
		return LOCTEXT("Blueprint_NewDerivedBlueprintTooltip", "Creates a Child Blueprint Class based on the current Blueprint, allowing you to create variants easily.");
	}
}

bool FAssetTypeActions_Blueprint::CanExecuteNewDerivedBlueprint(TWeakObjectPtr<UBlueprint> InObject)
{
	return FBlueprintEditorUtils::CanCreateChildBlueprint(InObject.Get());
}

bool FAssetTypeActions_Blueprint::ShouldUseDataOnlyEditor( const UBlueprint* Blueprint ) const
{
	return FBlueprintEditorUtils::IsDataOnlyBlueprint(Blueprint) 
		&& !FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint) 
		&& !FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint)
		&& !Blueprint->bForceFullEditor
		&& !Blueprint->bIsNewlyCreated;
}

void FAssetTypeActions_Blueprint::PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision) const
{
	UBlueprint* OldBlueprint = Cast<UBlueprint>(OldAsset);
	UBlueprint* NewBlueprint = Cast<UBlueprint>(NewAsset);

	SBlueprintDiff::CreateDiffWindow(OldBlueprint, NewBlueprint, OldRevision, NewRevision, GetSupportedClass());
}

UThumbnailInfo* FAssetTypeActions_Blueprint::GetThumbnailInfo(UObject* Asset) const
{
	// Blueprint thumbnail scenes are disabled for now
	UBlueprint* Blueprint = CastChecked<UBlueprint>(Asset);
	UThumbnailInfo* ThumbnailInfo = Blueprint->ThumbnailInfo;
	if ( ThumbnailInfo == NULL )
	{
		ThumbnailInfo = NewObject<USceneThumbnailInfo>(Blueprint, NAME_None, RF_Transactional);
		Blueprint->ThumbnailInfo = ThumbnailInfo;
	}

	return ThumbnailInfo;
}

FText FAssetTypeActions_Blueprint::GetAssetDescription(const FAssetData& AssetData) const
{
	FString Description = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintDescription));
	if (!Description.IsEmpty())
	{
		Description.ReplaceInline(TEXT("\\n"), TEXT("\n"));
		return FText::FromString(MoveTemp(Description));
	}

	return FText::GetEmpty();
}

TWeakPtr<IClassTypeActions> FAssetTypeActions_Blueprint::GetClassTypeActions(const FAssetData& AssetData) const
{
	// Blueprints get the class type actions for their parent native class - this avoids us having to load the blueprint
	UClass* ParentClass = nullptr;
	FString ParentClassName;
	if(!AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
	{
		AssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
	}
	if(!ParentClassName.IsEmpty())
	{
		UObject* Outer = nullptr;
		ResolveName(Outer, ParentClassName, false, false);
		ParentClass = FindObject<UClass>(Outer, *ParentClassName);
	}

	if(ParentClass)
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		return AssetToolsModule.Get().GetClassTypeActionsForClass(ParentClass);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
