// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_BlueprintGeneratedClass.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "ToolMenus.h"
#include "Misc/PackageName.h"
#include "Misc/MessageDialog.h"
#include "Factories/BlueprintFactory.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_BlueprintGeneratedClass::GetSupportedClass() const
{
	return UBlueprintGeneratedClass::StaticClass();
}

TWeakPtr<IClassTypeActions> FAssetTypeActions_BlueprintGeneratedClass::GetClassTypeActions(const FAssetData& AssetData) const
{
	// Blueprints get the class type actions for their parent native class.
	// Using asset tags avoids us having to load the blueprint
	UClass* ParentClass = nullptr;
	FString ParentClassName;
	if(!AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
	{
		AssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
	}
	if (!ParentClassName.IsEmpty())
	{
		ParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(ParentClassName));
	}

	if (ParentClass)
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		return AssetToolsModule.Get().GetClassTypeActionsForClass(ParentClass);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
