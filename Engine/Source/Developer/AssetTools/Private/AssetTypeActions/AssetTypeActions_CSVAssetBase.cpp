// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_CSVAssetBase.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FAssetTypeActions_CSVAssetBase::ExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions)
{
	for (TArray<FString>::TConstIterator FilenameIter(Filenames); FilenameIter; ++FilenameIter)
	{
		const FString CSVFilename = FPaths::ConvertRelativePathToFull(*FilenameIter);
		const FString RootPath = FPaths::GetPath(CSVFilename);
		const FString BaseFilename = FPaths::GetBaseFilename(CSVFilename, true);
		
		for (TArray<FString>::TConstIterator ExtensionItr(OverrideExtensions); ExtensionItr; ++ExtensionItr)
		{
			const FString FilenameWithExtension(FString::Printf(TEXT("%s/%s%s"), *RootPath, *BaseFilename, **ExtensionItr));
			if (VerifyFileExists(FilenameWithExtension))
			{
				FPlatformProcess::LaunchFileInDefaultExternalApplication(*FilenameWithExtension, NULL, ELaunchVerb::Edit);
				break;
			}
		}
	}
}

bool FAssetTypeActions_CSVAssetBase::CanExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions) const
{
	// Verify that extensions were provided
	if (OverrideExtensions.Num() == 0)
	{
		return false;
	}

	// Verify that the file exists with any of the given extensions
	for (TArray<FString>::TConstIterator FilenameIter(Filenames); FilenameIter; ++FilenameIter)
	{
		const FString CSVFilename = FPaths::ConvertRelativePathToFull(*FilenameIter);
		const FString RootPath = FPaths::GetPath(CSVFilename);
		const FString BaseFilename = FPaths::GetBaseFilename(CSVFilename, true);

		for (TArray<FString>::TConstIterator ExtensionItr(OverrideExtensions); ExtensionItr; ++ExtensionItr)
		{
			const FString FilenameWithExtension(FString::Printf(TEXT("%s/%s%s"), *RootPath, *BaseFilename, **ExtensionItr));
			if (VerifyFileExists(FilenameWithExtension))
			{
				return true;
			}
		}
	}

	return false;
}

bool FAssetTypeActions_CSVAssetBase::VerifyFileExists(const FString& InFileName) const
{
	return (!InFileName.IsEmpty() && IFileManager::Get().FileSize(*InFileName) != INDEX_NONE);
}

#undef LOCTEXT_NAMESPACE
