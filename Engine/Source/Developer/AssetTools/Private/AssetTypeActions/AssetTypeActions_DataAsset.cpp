// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_DataAsset.h"
#include "Engine/Engine.h"
#include "ToolMenuSection.h"
#include "ClassViewerModule.h"
#include "Kismet2/SClassPickerDialog.h"
#include "ClassViewerFilter.h"
#include "ObjectTools.h"
#include "SDetailsDiff.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText FAssetTypeActions_DataAsset::GetDisplayNameFromAssetData(const FAssetData& AssetData) const
{
	if (AssetData.IsValid())
	{
		const FAssetDataTagMapSharedView::FFindTagResult NativeClassTag = AssetData.TagsAndValues.FindTag(UDataAsset::NativeClassTag);
		if (NativeClassTag.IsSet())
		{
			if (UClass* FoundClass = FindObjectSafe<UClass>(nullptr, *NativeClassTag.GetValue()))
			{
				return FText::Format(LOCTEXT("DataAssetWithType", "Data Asset ({0})"), FoundClass->GetDisplayNameText());
			}
		}
	}

	return FText::GetEmpty();
}

void FAssetTypeActions_DataAsset::PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision) const
{
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(OldAsset, NewAsset, OldRevision, NewRevision, GetSupportedClass());
	// allow users to edit NewAsset if it's a local asset
	if (NewAsset && !FPackageName::IsTempPackage(NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(NewAsset);
	}
}

#undef LOCTEXT_NAMESPACE
