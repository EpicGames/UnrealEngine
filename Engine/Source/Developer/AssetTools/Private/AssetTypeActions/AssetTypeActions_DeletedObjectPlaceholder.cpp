// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_DeletedObjectPlaceholder.h"

#if WITH_EDITOR

FString FAssetTypeActions_DeletedObjectPlaceholder::GetObjectDisplayName(UObject* InObject) const
{
	if (UDeletedObjectPlaceholder* Object = Cast<UDeletedObjectPlaceholder>(InObject))
	{
		return Object->GetDisplayName();
	}
	return FAssetTypeActions_Base::GetObjectDisplayName(InObject);
}

#endif