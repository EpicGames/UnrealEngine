// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "AssetTypeActions_Base.h"
#include "AssetTypeCategories.h"
#include "DeletedObjectPlaceholder.h"

class FAssetTypeActions_DeletedObjectPlaceholder : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions interface
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DeletedObjectPlaceholder", "Deleted Object Placeholder"); }
	virtual FColor GetTypeColor() const override { return FColor::Red; }
	virtual UClass* GetSupportedClass() const override { return UDeletedObjectPlaceholder::StaticClass(); }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
	virtual bool CanLocalize() const override { return false; }
	virtual FString GetObjectDisplayName(UObject* InObject) const override;
	// End of IAssetTypeActions interface
};
#endif