// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_VariableFrameStrippingSettings.h"
#include "Animation/AnimSequence.h"
#include "Dialogs/Dialogs.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectIterator.h"


#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FAssetTypeActions_VariableFrameStrippingSettings::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	FAssetTypeActions_Base::OpenAssetEditor(InObjects, EditWithinLevelEditor);
}


#undef LOCTEXT_NAMESPACE