// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"
#include "Animation/VariableFrameStrippingSettings.h"

class FAssetTypeActions_VariableFrameStrippingSettings : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_VariableFrameStrippingSettings", "Variable Frame Stripping Settings"); }
	virtual FColor GetTypeColor() const override { return FColor(255, 255, 0); }
	virtual UClass* GetSupportedClass() const override { return UVariableFrameStrippingSettings::StaticClass(); }
	virtual bool CanFilter() override { return true; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }

	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;




};
