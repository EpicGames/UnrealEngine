// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetViewUtils.h"
#include "Algo/Transform.h"
#include "Bookmarks/BookmarkScoped.h"
#include "HAL/IConsoleManager.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/Paths.h"
#include "Misc/NamePermissionList.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "UnrealClient.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Settings/ContentBrowserSettings.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "Templates/GuardValueAccessors.h"
#include "SourceControlHelpers.h"
#include "FileHelpers.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Settings/EditorExperimentalSettings.h"

#include "PackagesDialog.h"
#include "PackageTools.h"
#include "ObjectTools.h"
#include "ImageUtils.h"
#include "Logging/MessageLog.h"
#include "Misc/EngineBuildSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "UObject/Linker.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Interfaces/IPluginManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "UObject/LinkerInstancingContext.h"

#define LOCTEXT_NAMESPACE "AssetViewUtils"

DEFINE_LOG_CATEGORY_STATIC(LogAssetViewTools, Warning, Warning);

#define MAX_CLASS_NAME_LENGTH 32 // Enforce a reasonable class name length so the path is not too long for FPlatformMisc::GetMaxPathLength()


namespace AssetViewUtils
{
	/** Callback used when a folder should be forced to be visible in the Content Browser */
	static FOnAlwaysShowPath OnAlwaysShowPathDelegate;

	/** Callback used when a folder is moved or renamed */
	static FOnFolderPathChanged OnFolderPathChangedDelegate;

	/** Callback used when a sync starts */
	static FOnSyncStart FOnSyncStartDelegate;

	/** Callback used when a sync finishes */
	static FOnSyncFinish FOnSyncFinishDelegate;

	/** Keep a map of all the paths that have custom colors, so updating the color in one location updates them all */
	static TMap< FString, FLinearColor > PathColors;

	/** Internal function to delete a folder from disk, but only if it is empty. InPathToDelete is in FPackageName format. */
	bool DeleteEmptyFolderFromDisk(const FString& InPathToDelete);

	/** Get all the objects in a list of asset data with optional load of all external packages */
	void GetObjectsInAssetData(const TArray<FAssetData>& AssetList, TArray<UObject*>& OutDroppedObjects, bool bLoadAllExternalObjects);
}

bool AssetViewUtils::OpenEditorForAsset(const FString& ObjectPath)
{
	// Load the asset if unloaded
	TArray<UObject*> LoadedObjects;
	TArray<FSoftObjectPath> ObjectPaths;
	ObjectPaths.Emplace(ObjectPath);

	// Here we want to load the asset as it will be passed to OpenEditorForAsset
	FLoadAssetsSettings Settings{
		.bFollowRedirectors = false,
		.bLoadWorldPartitionMaps = true,
		.bLoadAllExternalObjects = false,
	};
	LoadAssetsIfNeeded(ObjectPaths, LoadedObjects, Settings);

	// Open the editor for the specified asset
	UObject* FoundObject = FindObject<UObject>(nullptr, *ObjectPath);

	return OpenEditorForAsset(FoundObject);
}

bool AssetViewUtils::OpenEditorForAsset(UObject* Asset)
{
	if( Asset != NULL )
	{
		// @todo toolkit minor: Needs world-centric support?
		return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
	}

	return false;
}

bool AssetViewUtils::OpenEditorForAsset(const TArray<UObject*>& Assets)
{
	if ( Assets.Num() == 1 )
	{
		return OpenEditorForAsset(Assets[0]);
	}
	else if ( Assets.Num() > 1 )
	{
		return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
	}
	
	return false;
}

bool AssetViewUtils::LoadAssetsIfNeeded(const TArray<FString>& ObjectPaths, TArray<UObject*>& LoadedObjects, bool bAllowedToPromptToLoadAssets, bool bLoadRedirects)
{
	FLoadAssetsSettings Settings{
		.bFollowRedirectors = bLoadRedirects,
		.bLoadWorldPartitionMaps = false,
		.bLoadAllExternalObjects = false,
	};

	switch (LoadAssetsIfNeeded(ObjectPaths, LoadedObjects, Settings))
	{
		case ELoadAssetsResult::Success:
			return true;
		case ELoadAssetsResult::Cancelled:  // fallthrough
		case ELoadAssetsResult::SomeFailed: // fallthrough
			return false;
		default:
			check("Unhandled return value from LoadAssetsIfNeeded");
			return false;
	}
}

AssetViewUtils::ELoadAssetsResult AssetViewUtils::LoadAssetsIfNeeded(TConstArrayView<FString> ObjectPathStrings, TArray<UObject*>& OutLoadedObjects, const FLoadAssetsSettings& Settings)
{
	TArray<FSoftObjectPath> Paths;
	Paths.Reserve(ObjectPathStrings.Num());
	Algo::Transform(ObjectPathStrings, Paths, UE_PROJECTION(FSoftObjectPath));
	return LoadAssetsIfNeeded(Paths, OutLoadedObjects, Settings);
}

AssetViewUtils::ELoadAssetsResult AssetViewUtils::LoadAssetsIfNeeded(TConstArrayView<FSoftObjectPath> ObjectPaths, TArray<UObject*>& OutLoadedObjects, const FLoadAssetsSettings& Settings)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	TArray<FAssetData> Assets;
	Assets.Reserve(ObjectPaths.Num());
	TArray<FSoftObjectPath> ToScan;
	for (const FSoftObjectPath& Path : ObjectPaths)
	{
		FAssetData Asset = AssetRegistry.GetAssetByObjectPath(Path, true);
		if (Asset.IsValid())
		{
			Assets.Emplace(MoveTemp(Asset));
		}
		else
		{
			ToScan.Emplace(Path);
		}
	}
	if (ToScan.Num() > 0)
	{
		TArray<FString> ScanPaths;
		Algo::Transform(ToScan, ScanPaths, UE_PROJECTION_MEMBER(FSoftObjectPath, GetLongPackageName));
		AssetRegistry.ScanFilesSynchronous(ScanPaths);
		for (const FSoftObjectPath& Path : ToScan)
		{
			FAssetData Asset = AssetRegistry.GetAssetByObjectPath(Path, true);
			if (Asset.IsValid())
			{
				Assets.Emplace(MoveTemp(Asset));
			}
		}
	}
	return LoadAssetsIfNeeded(Assets, OutLoadedObjects, Settings);
}

AssetViewUtils::ELoadAssetsResult AssetViewUtils::LoadAssetsIfNeeded(TConstArrayView<FAssetData> Assets, TArray<UObject*>& OutLoadedObjects, const FLoadAssetsSettings& Settings)
{
	bool bAnySucceeded = false;

	// Build a list of unloaded assets
	TSet<FName> UnloadedObjectPaths;
	for (const FAssetData& Asset : Assets)
	{
		if (UObject* FoundObject = Asset.FastGetAsset(false))
		{
			OutLoadedObjects.Add(FoundObject);
			bAnySucceeded = true;
			continue;
		}

		if (Settings.bLoadWorldPartitionMaps || !ULevel::GetIsLevelPartitionedFromAsset(Asset))
		{
			UnloadedObjectPaths.Add(Asset.GetOptionalOuterPathName());
			UnloadedObjectPaths.Add(Asset.PackageName);
		}
	}

	UnloadedObjectPaths.Remove(NAME_None);

	if (UnloadedObjectPaths.Num() == 0)
	{
		return ELoadAssetsResult::Success;
	}

	if (Settings.bAlwaysPromptBeforeLoading || UnloadedObjectPaths.Num() > GetDefault<UContentBrowserSettings>()->NumObjectsToLoadBeforeWarning)
	{
		EAppReturnType::Type Decision = FMessageDialog::Open(EAppMsgType::YesNo,
			FText::Format(LOCTEXT("LoadingManyAssets", "About to load {0} assets. Are you sure you want to continue?"), UnloadedObjectPaths.Num()),
			LOCTEXT("LoadingManyAssetsTitle", "Loading Many Assets"));

		switch (Decision)
		{
			case EAppReturnType::Cancel: // fallthrough
			case EAppReturnType::No:     // fallthrough
			case EAppReturnType::NoAll:
				return ELoadAssetsResult::Cancelled;
			case EAppReturnType::Yes:      // fallthrough
			case EAppReturnType::YesAll:   // fallthrough
			case EAppReturnType::Ok:       // fallthrough
			case EAppReturnType::Retry:    // fallthrough
			case EAppReturnType::Continue: // fallthrough
				break;
		}
	}

	FScopedSlowTask SlowTask(static_cast<float>(UnloadedObjectPaths.Num()), LOCTEXT("LoadingAssets", "Loading Assets..."));
	// Always make dialog, even a single asset can be slow to load
	SlowTask.MakeDialog(Settings.bAllowCancel);

	bool bSomeObjectsFailedToLoad = false;
	bool bCancelled = false;
	{
		TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);
		const ELoadFlags LoadFlags = Settings.bFollowRedirectors ? LOAD_None : LOAD_NoRedirects;

		for (const FName& ObjectPath : UnloadedObjectPaths)
		{
			SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("LoadingAsset", "Loading {0}..."), FText::FromName(ObjectPath)));

			// Load up the object
			FLinkerInstancingContext InstancingContext(Settings.bLoadAllExternalObjects ? TSet<FName>{ ULevel::LoadAllExternalObjectsTag } : TSet<FName>());
			UObject* LoadedObject = LoadObject<UObject>(nullptr, *ObjectPath.ToString(), nullptr, LoadFlags, nullptr, &InstancingContext);
			if (LoadedObject)
			{
				OutLoadedObjects.Add(LoadedObject);
				bAnySucceeded = true;
			}
			else
			{
				bSomeObjectsFailedToLoad = true;
			}

			if (SlowTask.ShouldCancel())
			{
				bCancelled = true;
				break;
			}
		}
	}

	if (bSomeObjectsFailedToLoad)
	{
		FNotificationInfo Info(LOCTEXT("SomeLoadsFailed", "Failed to load some assets"));
		Info.ExpireDuration = 5.0f;
		Info.Hyperlink = FSimpleDelegate::CreateStatic([]() { FMessageLog("LoadErrors").Open(EMessageSeverity::Info, true); });
		Info.HyperlinkText = LOCTEXT("LoadObjectHyperlink", "Show Message Log");

		FSlateNotificationManager::Get().AddNotification(Info);
	}

	if (bCancelled)
	{
		return ELoadAssetsResult::Cancelled;
	}
	else if (bSomeObjectsFailedToLoad)
	{
		return ELoadAssetsResult::SomeFailed;
	}
	else
	{
		return ELoadAssetsResult::Success;
	}
}

void AssetViewUtils::GetUnloadedAssets(const TArray<FString>& ObjectPaths, TArray<FString>& OutUnloadedObjects)
{
	OutUnloadedObjects.Empty();

	// Build a list of unloaded assets and check if there are any parent folders
	for (int32 PathIdx = 0; PathIdx < ObjectPaths.Num(); ++PathIdx)
	{
		const FString& ObjectPath = ObjectPaths[PathIdx];

		UObject* FoundObject = FindObject<UObject>(NULL, *ObjectPath);
		if ( !FoundObject )
		{
			// Unloaded asset, we will load it later
			OutUnloadedObjects.Add(ObjectPath);
		}
	}
}

bool AssetViewUtils::PromptToLoadAssets(const TArray<FString>& UnloadedObjects)
{
	bool bShouldLoadAssets = false;

	// Prompt the user to load assets
	const FText Question = FText::Format( LOCTEXT("ConfirmLoadAssets", "You are about to load {0} assets. Would you like to proceed?"), FText::AsNumber( UnloadedObjects.Num() ) );
	if ( EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, Question) )
	{
		bShouldLoadAssets = true;
	}

	return bShouldLoadAssets;
}

void AssetViewUtils::CopyAssets(const TArray<UObject*>& Assets, const FString& DestPath)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(DestPath))
	{
		AssetToolsModule.Get().NotifyBlockedByWritableFolderFilter();
		return;
	}

	TArray<UObject*> NewObjects;
	ObjectTools::DuplicateObjects(Assets, TEXT(""), DestPath, /*bOpenDialog=*/false, &NewObjects);

	// If any objects were duplicated, report the success
	if ( NewObjects.Num() )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("Number"), NewObjects.Num() );
		const FText Message = FText::Format( LOCTEXT("AssetsDroppedCopy", "{Number} asset(s) copied"), Args );
		FSlateNotificationManager::Get().AddNotification(FNotificationInfo(Message));

		// Now branch the files in source control if possible
		check(Assets.Num() == NewObjects.Num());
		for(int32 ObjectIndex = 0; ObjectIndex < Assets.Num(); ObjectIndex++)
		{
			UObject* SourceAsset = Assets[ObjectIndex];
			UObject* DestAsset = NewObjects[ObjectIndex];
			SourceControlHelpers::CopyPackage(DestAsset->GetOutermost(), SourceAsset->GetOutermost());
		}
	}
}

void AssetViewUtils::MoveAssets(const TArray<UObject*>& Assets, const FString& DestPath, const FString& SourcePath)
{
	check(DestPath.Len() > 0);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(DestPath))
	{
		AssetToolsModule.Get().NotifyBlockedByWritableFolderFilter();
		return;
	}

	TArray<FAssetRenameData> AssetsAndNames;
	for ( auto AssetIt = Assets.CreateConstIterator(); AssetIt; ++AssetIt )
	{
		UObject* Asset = *AssetIt;

		if ( !ensure(Asset) )
		{
			continue;
		}

		FString PackagePath;
		FString ObjectName = Asset->GetName();

		if ( SourcePath.Len() )
		{
			const FString CurrentPackageName = Asset->GetOutermost()->GetName();

			// This is a relative operation
			if ( !ensure(CurrentPackageName.StartsWith(SourcePath)) )
			{
				continue;
			}
				
			// Collect the relative path then use it to determine the new location
			// For example, if SourcePath = /Game/MyPath and CurrentPackageName = /Game/MyPath/MySubPath/MyAsset
			//     /Game/MyPath/MySubPath/MyAsset -> /MySubPath

			const int32 ShortPackageNameLen = FPackageName::GetLongPackageAssetName(CurrentPackageName).Len();
			const int32 RelativePathLen = CurrentPackageName.Len() - ShortPackageNameLen - SourcePath.Len() - 1; // -1 to exclude the trailing "/"
			const FString RelativeDestPath = CurrentPackageName.Mid(SourcePath.Len(), RelativePathLen);

			PackagePath = DestPath + RelativeDestPath;
		}
		else
		{
			// Only a DestPath was supplied, use it
			PackagePath = DestPath;
		}

		const bool bSoftReferenceOnly = false;
		const bool bAlsoRenameLocalizedVariants = true;
		new(AssetsAndNames) FAssetRenameData(Asset, PackagePath, ObjectName, bSoftReferenceOnly, bAlsoRenameLocalizedVariants);
	}

	if ( AssetsAndNames.Num() > 0 )
	{
		const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
		AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames, Settings->GetAutomaticallyCheckoutOnAssetModification());
	}
}

int32 AssetViewUtils::DeleteAssets(const TArray<UObject*>& AssetsToDelete)
{
	return ObjectTools::DeleteObjects(AssetsToDelete);
}

bool AssetViewUtils::DeleteFolders(const TArray<FString>& PathsToDelete)
{
	FDeleteFolderParameters Options;
	Options.bShowConfirmationBeforeLoadingAssets = true;
	Options.PathsToDelete = PathsToDelete;
	return DeleteFolders(Options);
}

bool AssetViewUtils::DeleteFolders(const FDeleteFolderParameters& InOptions)
{
	// Get a list of assets in the paths to delete
	TArray<FAssetData> AssetDataList;
	GetAssetsInPaths(InOptions.PathsToDelete, AssetDataList);

	// Add localized variants associated with the current assets to delete if the user wants to include them
	if (!ObjectTools::AddExtraLocalizedVariantsToDelete(AssetDataList, AssetDataList))
	{
		// Either the user requested to cancel the current operation or an error occurred that requires interrupting the current operation...
		return false;
	}

	const int32 NumAssetsInPaths = AssetDataList.Num();
	bool bSuccess = false;
	if ( NumAssetsInPaths == 0 )
	{
		// There were no assets found, allow the empty folders deletion without any confirmation
		ObjectTools::FDeleteItemsParameters Options;
		Options.FoldersToDelete = InOptions.PathsToDelete;
		Options.bShowConfirmation = false;
		bSuccess = ObjectTools::DeleteItems(Options).FoldersDeleted == InOptions.PathsToDelete.Num();
	}
	else
	{
		if (InOptions.bShowConfirmationBeforeLoadingAssets)
		{
			FText Prompt;
			if (InOptions.PathsToDelete.Num() == 1)
			{
				Prompt = FText::Format(LOCTEXT("FolderDeleteConfirm_Single", "Delete folder '{0}'?"), FText::FromString(InOptions.PathsToDelete[0]));
			}
			else
			{
				Prompt = FText::Format(LOCTEXT("FolderDeleteConfirm_Multiple", "Delete {0} folders?"), FText::AsNumber(InOptions.PathsToDelete.Num()));
			}

			// If we had any folders selected, ask the user whether they want to delete them 
			// as it can be slow to build the deletion dialog on an accidental click (due to assets being loaded)
			if (FMessageDialog::Open(EAppMsgType::YesNo, Prompt, LOCTEXT("FolderDeleteConfirmTitle", "Delete")) != EAppReturnType::Type::Yes)
			{
				return false;
			}
		}

		// Load all the assets in the folder and attempt to delete them. If it was successful, allow the folder delete.
		TArray<UObject*> LoadedAssets;
		FLoadAssetsSettings Settings{
			.bFollowRedirectors = false,
			.bLoadWorldPartitionMaps = true,
			.bLoadAllExternalObjects = true,
		};
		if (LoadAssetsIfNeeded(AssetDataList, LoadedAssets, Settings) == ELoadAssetsResult::Success)
		{
			// Make sure loaded assets match with pending deletion assets
			bool bProcessDelete = LoadedAssets.Num() == NumAssetsInPaths;

			if (!bProcessDelete)
			{
				// Add any missing object contained within the same package from AssetDataList like blueprint generated class
				TSet<UObject*> LoadedAssetsSet(LoadedAssets);
				for (const FAssetData& Asset : AssetDataList)
				{
					if (UObject* Object = Asset.FastGetAsset(/** Load */false))
					{
						LoadedAssetsSet.Add(Object);
					}
				}

				LoadedAssets = LoadedAssetsSet.Array();
				bProcessDelete = LoadedAssets.Num() == NumAssetsInPaths;
			}

			if (bProcessDelete)
			{
				ObjectTools::FDeleteItemsParameters Options;
				Options.ObjectsToDelete = LoadedAssets;
				Options.FoldersToDelete = InOptions.PathsToDelete;
				ObjectTools::AddExtraObjectsToDelete(Options.ObjectsToDelete);
				ObjectTools::FDeleteItemsResult Result = ObjectTools::DeleteItems(Options);
				bSuccess = Result.FoldersDeleted == Options.FoldersToDelete.Num()
					&& Result.ObjectsDeleted == Options.ObjectsToDelete.Num();
			}
			else
			{
				// Not all the assets in the selected paths were loaded
				UE_LOGF(LogAssetViewTools, Warning, "Cannot load some assets for deletion operation (%i vs %i) cannot proceed...", LoadedAssets.Num(), NumAssetsInPaths);
			}
		}
		else
		{
			// The user declined to load some assets or some assets failed to load
		}
	}

	return bSuccess;
}

bool AssetViewUtils::DeleteEmptyFolderFromDisk(const FString& InPathToDelete)
{
	struct FEmptyFolderVisitor : public IPlatformFile::FDirectoryVisitor
	{
		bool bIsEmpty;

		FEmptyFolderVisitor()
			: bIsEmpty(true)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			if (!bIsDirectory)
			{
				bIsEmpty = false;
				return false; // abort searching
			}

			return true; // continue searching
		}
	};

	FString PathToDeleteOnDisk;
	if (FPackageName::TryConvertLongPackageNameToFilename(InPathToDelete, PathToDeleteOnDisk))
	{
		// Look for files on disk in case the folder contains things not tracked by the asset registry
		FEmptyFolderVisitor EmptyFolderVisitor;
		IFileManager::Get().IterateDirectoryRecursively(*PathToDeleteOnDisk, EmptyFolderVisitor);

		if (EmptyFolderVisitor.bIsEmpty)
		{
			return IFileManager::Get().DeleteDirectory(*PathToDeleteOnDisk, false, true);
		}
	}

	return false;
}

void AssetViewUtils::GetAssetsInPaths(const TArray<FString>& InPaths, TArray<FAssetData>& OutAssetDataList)
{
	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Form a filter from the paths
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	for (int32 PathIdx = 0; PathIdx < InPaths.Num(); ++PathIdx)
	{
		new (Filter.PackagePaths) FName(*InPaths[PathIdx]);
	}

	// Query for a list of assets in the selected paths
	AssetRegistryModule.Get().GetAssets(Filter, OutAssetDataList);
}

bool AssetViewUtils::SavePackages(const TArray<UPackage*>& Packages)
{
	const bool bCheckDirty = false;
	const bool bPromptToSave = false;
	const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(Packages, bCheckDirty, bPromptToSave);

	return Return == FEditorFileUtils::EPromptReturnCode::PR_Success;
}

bool AssetViewUtils::SaveDirtyPackages()
{
	const bool bPromptUserToSave = true;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	const bool bFastSave = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined = false;
	return FEditorFileUtils::SaveDirtyPackages( bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined );
}

TArray<UPackage*> AssetViewUtils::LoadPackages(const TArray<FString>& PackageNames)
{
	TArray<UPackage*> LoadedPackages;

	GWarn->BeginSlowTask( LOCTEXT("LoadingPackages", "Loading Packages..."), true );

	for (int32 PackageIdx = 0; PackageIdx < PackageNames.Num(); ++PackageIdx)
	{
		const FString& PackageName = PackageNames[PackageIdx];

		if ( !ensure(PackageName.Len() > 0) )
		{
			// Empty package name. Skip it.
			continue;
		}

		UPackage* Package = FindPackage(NULL, *PackageName);

		if ( Package != NULL )
		{
			// The package is at least partially loaded. Fully load it.
			Package->FullyLoad();
		}
		else
		{
			// The package is unloaded. Try to load the package from disk.
			Package = UPackageTools::LoadPackage(PackageName);
		}

		// If the package was loaded, add it to the loaded packages list.
		if ( Package != NULL )
		{
			LoadedPackages.Add(Package);
		}
	}

	GWarn->EndSlowTask();

	return LoadedPackages;
}

bool AssetViewUtils::RenameFolder(const FString& DestPath, const FString& SourcePath)
{
	if (DestPath == SourcePath)
	{
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	// move any assets in our folder
	TArray<FAssetData> AssetsInFolder;
	AssetRegistryModule.Get().GetAssetsByPath(*SourcePath, AssetsInFolder, true);
	TArray<UObject*> ObjectsInFolder;
	const bool bLoadAllExternalObjects = true;
	GetObjectsInAssetData(AssetsInFolder, ObjectsInFolder, bLoadAllExternalObjects);

	FOnPreDestructiveAssetActionArgs PreDestructiveArgs(ObjectsInFolder, EDestructiveAssetActions::AssetRename);
	FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);
	if (!PreDestructiveArgs.Result.bSuccess)
	{
		UE_LOGF(LogAssetViewTools, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
		return false;
	}

	MoveAssets(ObjectsInFolder, DestPath, SourcePath);

	// Now check to see if the original folder is empty, if so we can delete it
	TArray<FAssetData> AssetsInOriginalFolder;
	AssetRegistryModule.Get().GetAssetsByPath(*SourcePath, AssetsInOriginalFolder, true);
	if (AssetsInOriginalFolder.Num() == 0)
	{
		TArray<FString> FoldersToDelete;
		FoldersToDelete.Add(SourcePath);
		DeleteFolders(FoldersToDelete);
	}

	// set color of folder to new path
	const TOptional<FLinearColor> FolderColor = GetPathColor(SourcePath);
	if (FolderColor.IsSet())
	{
		SetPathColor(SourcePath, TOptional<FLinearColor>());
		SetPathColor(DestPath, FolderColor);
	}

	{
		const FMovedContentFolder ChangedPath = MakeTuple(SourcePath, DestPath);
		OnFolderPathChangedDelegate.Broadcast(MakeArrayView(&ChangedPath, 1));
	}

	return true;
}

bool AssetViewUtils::CopyFolders(const TArray<FString>& InSourcePathNames, const FString& DestPath)
{
	TMap<FString, TArray<UObject*> > SourcePathToLoadedAssets;

	// Make sure the destination path is not in the source path list
	TArray<FString> SourcePathNames = InSourcePathNames;
	SourcePathNames.Remove(DestPath);

	// Load all assets in the source paths
	if (!PrepareFoldersForDragDrop(SourcePathNames, SourcePathToLoadedAssets))
	{
		return false;
	}

	// Load the Asset Registry to update paths during the copy
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	
	// For every path which contained valid assets...
	for ( auto PathIt = SourcePathToLoadedAssets.CreateConstIterator(); PathIt; ++PathIt )
	{
		// Put dragged folders in a sub-folder under the destination path
		const FString SourcePath = PathIt.Key();
		FString SubFolderName = FPackageName::GetLongPackageAssetName(SourcePath);
		FString Destination = DestPath + TEXT("/") + SubFolderName;

		// Add the new path to notify sources views
		AssetRegistryModule.Get().AddPath(Destination);
		OnAlwaysShowPathDelegate.Broadcast(Destination);

		// If any assets were in this path...
		if ( PathIt.Value().Num() > 0 )
		{
			// Copy assets and supply a source path to indicate it is relative
			ObjectTools::DuplicateObjects( PathIt.Value(), SourcePath, Destination, /*bOpenDialog=*/false );
		}

		const TOptional<FLinearColor> FolderColor = GetPathColor(SourcePath);
		if (FolderColor.IsSet())
		{
			SetPathColor(Destination, FolderColor);
		}
	}

	return true;
}

bool AssetViewUtils::MoveFolders(const TArray<FString>& InSourcePathNames, const FString& DestPath)
{
	TMap<FString, TArray<UObject*> > SourcePathToLoadedAssets;
	FString DestPathWithTrailingSlash = DestPath / "";

	// Do not allow parent directories to be moved to themselves or children.
	TArray<FString> SourcePathNames = InSourcePathNames;
	TArray<FString> SourcePathNamesToRemove;
	for (auto SourcePathIt = SourcePathNames.CreateConstIterator(); SourcePathIt; ++SourcePathIt)
	{
		if(DestPathWithTrailingSlash.StartsWith(*SourcePathIt / ""))
		{
			SourcePathNamesToRemove.Add(*SourcePathIt);
		}
	}
	for (auto SourcePathToRemoveIt = SourcePathNamesToRemove.CreateConstIterator(); SourcePathToRemoveIt; ++SourcePathToRemoveIt)
	{
		SourcePathNames.Remove(*SourcePathToRemoveIt);
	}

	// Load all assets in the source paths
	if (!PrepareFoldersForDragDrop(SourcePathNames, SourcePathToLoadedAssets))
	{
		return false;
	}

	TArray<UObject*> AssetsToMove;
	TArray<FTopLevelAssetPath> Destinations;
	for (auto PathIt = SourcePathToLoadedAssets.CreateConstIterator(); PathIt; ++PathIt)
	{
		AssetsToMove.Append(PathIt.Value());

		const FString& SourcePath = PathIt.Key();
		const FStringView SourcePathPrefix = FPathViews::GetPath(SourcePath);

		for (UObject* Obj : PathIt.Value())
		{
			FSoftObjectPath SourceObjectPath(Obj);
			FNameBuilder SourcePackagePath(SourceObjectPath.GetLongPackageFName());
			FStringView RelativePath = SourcePackagePath.ToView().RightChop(SourcePathPrefix.Len() + 1);

			FNameBuilder DestPackagePath;
			DestPackagePath.Append(DestPath);
			FPathViews::AppendPath(DestPackagePath, RelativePath);

			FTopLevelAssetPath Dest(FName(DestPackagePath.ToView()), SourceObjectPath.GetAssetFName());
			Destinations.Add(Dest);
		}
	}

	FOnPreDestructiveAssetActionArgs PreDestructiveArgs(AssetsToMove, EDestructiveAssetActions::AssetMove, Destinations);
	FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);
	if (!PreDestructiveArgs.Result.bSuccess)
	{
		UE_LOGF(LogAssetViewTools, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
		return false;
	}
	
	// Load the Asset Registry to update paths during the move
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<TTuple<FString, FString>> ChangedPaths;

	// For every path which contained valid assets...
	for ( auto PathIt = SourcePathToLoadedAssets.CreateConstIterator(); PathIt; ++PathIt )
	{
		// Put dragged folders in a sub-folder under the destination path
		const FString SourcePath = PathIt.Key();
		const FString SubFolderName = FPackageName::GetLongPackageAssetName(SourcePath);
		const FString Destination = DestPathWithTrailingSlash + SubFolderName;

		// Add the new path to notify sources views
		AssetRegistryModule.Get().AddPath(Destination);
		OnAlwaysShowPathDelegate.Broadcast(Destination);

		// If any assets were in this path...
		if ( PathIt.Value().Num() > 0 )
		{
			// Move assets and supply a source path to indicate it is relative
			MoveAssets( PathIt.Value(), Destination, PathIt.Key() );
		}

		TArray<FString> PathsToRescan;
		PathsToRescan.Add(Destination);

		// Attempt to remove the old path
		if (DeleteEmptyFolderFromDisk(SourcePath))
		{
			AssetRegistryModule.Get().RemovePath(SourcePath);
		}
		else if (PathIt.Value().Num() > 0)
		{
			PathsToRescan.Add(SourcePath);
		}

		const bool bForceRescan = true;
		AssetRegistryModule.Get().ScanPathsSynchronous(PathsToRescan, bForceRescan);

		const TOptional<FLinearColor> FolderColor = GetPathColor(SourcePath);
		if (FolderColor.IsSet())
		{
			SetPathColor(SourcePath, TOptional<FLinearColor>());
			SetPathColor(Destination, FolderColor);
		}

		ChangedPaths.Add(MakeTuple(SourcePath, Destination));
	}

	OnFolderPathChangedDelegate.Broadcast(ChangedPaths);

	return true;
}

bool AssetViewUtils::PrepareFoldersForDragDrop(const TArray<FString>& SourcePathNames, TMap< FString, TArray<UObject*> >& OutSourcePathToLoadedAssets)
{
	TSet<UObject*> AllFoundObjects;

	// Load the Asset Registry to update paths during the move
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	// Check up-front how many assets we might load in this operation & warn the user
	TArray<FString> ObjectPathsToWarnAbout;
	for ( auto PathIt = SourcePathNames.CreateConstIterator(); PathIt; ++PathIt )
	{
		// Get all assets in this path
		TArray<FAssetData> AssetDataList;
		AssetRegistryModule.Get().GetAssetsByPath(FName(**PathIt), AssetDataList, true);

		for ( auto AssetIt = AssetDataList.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			ObjectPathsToWarnAbout.Add((*AssetIt).GetObjectPathString());
		}
	}

	GWarn->BeginSlowTask(LOCTEXT("FolderDragDrop_Loading", "Loading folders"), true);

	// For every source path, load every package in the path (if necessary) and keep track of the assets that were loaded
	for ( auto PathIt = SourcePathNames.CreateConstIterator(); PathIt; ++PathIt )
	{
		// Get all assets in this path
		TArray<FAssetData> AssetDataList;
		AssetRegistryModule.Get().GetAssetsByPath(FName(**PathIt), AssetDataList, true);

		// Form a list of all object paths for these assets
		TArray<FString> ObjectPaths;
		for ( auto AssetIt = AssetDataList.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			ObjectPaths.Add((*AssetIt).GetObjectPathString());
		}

		// Load all assets in this path if needed
		TArray<UObject*> AllLoadedAssets;
		FLoadAssetsSettings Settings{
			.bFollowRedirectors = false,
			.bLoadWorldPartitionMaps = true,
			.bLoadAllExternalObjects = true
		};
		LoadAssetsIfNeeded(ObjectPaths, AllLoadedAssets, Settings);

		// Add a slash to the end of the path so StartsWith doesn't get a false positive on similarly named folders
		const FString SourcePathWithSlash = *PathIt + TEXT("/");

		// Find all files in this path and subpaths
		TArray<FString> Filenames;
		FString RootFolder = FPackageName::LongPackageNameToFilename(SourcePathWithSlash);
		FPackageName::FindPackagesInDirectory(Filenames, RootFolder);

		// Now find all assets in memory that were loaded from this path that are valid for drag-droppping
		TArray<UObject*> ValidLoadedAssets;
		for ( auto AssetIt = AllLoadedAssets.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			UObject* Asset = *AssetIt;
			if ( (Asset->GetClass() != UObjectRedirector::StaticClass() &&				// Skip object redirectors
				 !AllFoundObjects.Contains(Asset)										// Skip assets we have already found to avoid processing them twice
				) )
			{
				ValidLoadedAssets.Add(Asset);
				AllFoundObjects.Add(Asset);
			}
		}

		// Add an entry of the map of source paths to assets found, whether any assets were found or not
		OutSourcePathToLoadedAssets.Add(*PathIt, ValidLoadedAssets);
	}

	GWarn->EndSlowTask();

	ensure(SourcePathNames.Num() == OutSourcePathToLoadedAssets.Num());
	return true;
}

void AssetViewUtils::CaptureThumbnailFromViewport(FViewport* InViewport, const TArray<FAssetData>& InAssetsToAssign)
{
	//capture the thumbnail
	uint32 SrcWidth = InViewport->GetSizeXY().X;
	uint32 SrcHeight = InViewport->GetSizeXY().Y;
	// Read the contents of the viewport into an array.
	TArray<FColor> OrigBitmap;
	if (InViewport->ReadPixels(OrigBitmap))
	{
		check(OrigBitmap.Num() == SrcWidth * SrcHeight);

		//pin to smallest value
		int32 CropSize = FMath::Min<uint32>(SrcWidth, SrcHeight);
		//pin to max size
		int32 ScaledSize  = FMath::Min<uint32>(ThumbnailTools::DefaultThumbnailSize, CropSize);

		//calculations for cropping
		TArray<FColor> CroppedBitmap;
		CroppedBitmap.AddUninitialized(CropSize*CropSize);
		//Crop the image
		int32 CroppedSrcTop  = (SrcHeight - CropSize)/2;
		int32 CroppedSrcLeft = (SrcWidth - CropSize)/2;
		for (int32 Row = 0; Row < CropSize; ++Row)
		{
			//Row*Side of a row*byte per color
			int32 SrcPixelIndex = (CroppedSrcTop+Row)*SrcWidth + CroppedSrcLeft;
			const void* SrcPtr = &(OrigBitmap[SrcPixelIndex]);
			void* DstPtr = &(CroppedBitmap[Row*CropSize]);
			FMemory::Memcpy(DstPtr, SrcPtr, CropSize*4);
		}

		//Scale image down if needed
		TArray<FColor> ScaledBitmap;
		if (ScaledSize < CropSize)
		{
			FImageUtils::ImageResize( CropSize, CropSize, CroppedBitmap, ScaledSize, ScaledSize, ScaledBitmap, true );
		}
		else
		{
			//just copy the data over. sizes are the same
			ScaledBitmap = CroppedBitmap;
		}

		//setup actual thumbnail
		FObjectThumbnail TempThumbnail;
		TempThumbnail.SetImageSize( ScaledSize, ScaledSize );
		TArray<uint8>& ThumbnailByteArray = TempThumbnail.AccessImageData();

		// Copy scaled image into destination thumb
		int32 MemorySize = ScaledSize*ScaledSize*sizeof(FColor);
		ThumbnailByteArray.AddUninitialized(MemorySize);
		FMemory::Memcpy(&(ThumbnailByteArray[0]), &(ScaledBitmap[0]), MemorySize);

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

		//check if each asset should receive the new thumb nail
		for ( auto AssetIt = InAssetsToAssign.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			const FAssetData& CurrentAsset = *AssetIt;

			//assign the thumbnail and dirty
			const FString ObjectFullName = CurrentAsset.GetFullName();
			const FString PackageName    = CurrentAsset.PackageName.ToString();

			UPackage* AssetPackage = CurrentAsset.GetPackage();
			if ( ensure(AssetPackage) )
			{
				FObjectThumbnail* NewThumbnail = ThumbnailTools::CacheThumbnail(ObjectFullName, &TempThumbnail, AssetPackage);
				if ( ensure(NewThumbnail) )
				{
					//we need to indicate that the package needs to be resaved
					AssetPackage->MarkPackageDirty();

					// Let the content browser know that we've changed the thumbnail
					NewThumbnail->MarkAsDirty();
						
					// Signal that the asset was changed if it is loaded so thumbnail pools will update
					if ( CurrentAsset.IsAssetLoaded() )
					{
						CurrentAsset.GetAsset()->PostEditChange();
					}

					//Set that thumbnail as a valid custom thumbnail so it'll be saved out
					NewThumbnail->SetCreatedAfterCustomThumbsEnabled();
				}
			}
		}
	}
}

void AssetViewUtils::ClearCustomThumbnails(const TArray<FAssetData>& InAssetsToAssign)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	//check if each asset should receive the new thumb nail
	for ( auto AssetIt = InAssetsToAssign.CreateConstIterator(); AssetIt; ++AssetIt )
	{
		const FAssetData& CurrentAsset = *AssetIt;

		// check whether this is a type that uses one of the shared static thumbnails
		if ( AssetToolsModule.Get().AssetUsesGenericThumbnail( CurrentAsset ) )
		{
			//assign the thumbnail and dirty
			const FString ObjectFullName = CurrentAsset.GetFullName();

			// Load the asset so that we can find and clear the thumbnail
			UPackage* AssetPackage = CurrentAsset.GetPackage();
			if ( ensure(AssetPackage) )
			{
				ThumbnailTools::CacheEmptyThumbnail( ObjectFullName, AssetPackage);

				//we need to indicate that the package needs to be resaved
				AssetPackage->MarkPackageDirty();

				// Signal that the asset was changed if it is loaded so thumbnail pools will update
				if ( CurrentAsset.IsAssetLoaded() )
				{
					CurrentAsset.GetAsset()->PostEditChange();
				}
			}
		}
	}
}

bool AssetViewUtils::AssetHasCustomThumbnail( const FAssetData& AssetData )
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if ( AssetToolsModule.Get().AssetUsesGenericThumbnail(AssetData) )
	{
		return ThumbnailTools::AssetHasCustomThumbnail(AssetData.GetFullName());
	}

	return false;
}

bool AssetViewUtils::IsProjectFolder(const FStringView InPath, const bool bIncludePlugins)
{
	static const FString ProjectPathWithSlash = TEXT("/Game");
	static const FString ProjectPathWithoutSlash = TEXT("Game");

	if (InPath.StartsWith(ProjectPathWithSlash) || InPath == ProjectPathWithoutSlash)
	{
		return true;
	}

	if (bIncludePlugins)
	{
		EPluginLoadedFrom PluginSource = EPluginLoadedFrom::Engine;
		if (IsPluginFolder(InPath, &PluginSource))
		{
			return PluginSource == EPluginLoadedFrom::Project;
		}
	}

	return false;
}

bool AssetViewUtils::IsEngineFolder(const FStringView InPath, const bool bIncludePlugins)
{
	static const FString EnginePathWithSlash = TEXT("/Engine");
	static const FString EnginePathWithoutSlash = TEXT("Engine");

	if (InPath.StartsWith(EnginePathWithSlash) || InPath == EnginePathWithoutSlash)
	{
		return true;
	}

	if (bIncludePlugins)
	{
		EPluginLoadedFrom PluginSource = EPluginLoadedFrom::Engine;
		if (IsPluginFolder(InPath, &PluginSource))
		{
			return PluginSource == EPluginLoadedFrom::Engine;
		}
	}

	return false;
}

bool AssetViewUtils::IsDevelopersFolder( const FStringView InPath )
{
	static const FString DeveloperPathWithoutSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir()).LeftChop(1);
	return InPath.StartsWith(DeveloperPathWithoutSlash) && (InPath.Len() == DeveloperPathWithoutSlash.Len() || InPath[DeveloperPathWithoutSlash.Len()] == TEXT('/'));
}

bool AssetViewUtils::IsPluginFolder(const FStringView InPath, EPluginLoadedFrom* OutPluginSource)
{
	if (TSharedPtr<IPlugin> Plugin = GetPluginForFolder(InPath))
	{
		if (OutPluginSource)
		{
			*OutPluginSource = Plugin->GetLoadedFrom();
		}

		return true;
	}

	return false;
}

TSharedPtr<IPlugin> AssetViewUtils::GetPluginForFolder(const FStringView InPath)
{
	FStringView PluginName(InPath);
	if (PluginName.StartsWith(TEXT('/')))
	{
		PluginName.RightChopInline(1);
	}

	int32 FoundIndex = INDEX_NONE;
	if (PluginName.FindChar(TEXT('/'), FoundIndex))
	{
		PluginName.LeftInline(FoundIndex);
	}

	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
	{
		if (Plugin->IsEnabled() && Plugin->CanContainContent())
		{
			return Plugin;
		}
	}

	return nullptr;
}

void AssetViewUtils::GetObjectsInAssetData(const TArray<FAssetData>& AssetList, TArray<UObject*>& OutDroppedObjects, bool bLoadAllExternalObjects)
{
	for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
	{
		const FAssetData& AssetData = AssetList[AssetIdx];
		UObject* Obj = AssetData.GetAsset(bLoadAllExternalObjects ? TSet<FName> { ULevel::LoadAllExternalObjectsTag } : TSet<FName>());
		if (Obj)
		{
			OutDroppedObjects.Add(Obj);
		}
	}
}

void AssetViewUtils::GetObjectsInAssetData(const TArray<FAssetData>& AssetList, TArray<UObject*>& OutDroppedObjects)
{	
	const bool bLoadAllExternalObjects = false;
	GetObjectsInAssetData(AssetList, OutDroppedObjects, bLoadAllExternalObjects);
}

bool AssetViewUtils::IsValidFolderName(const FString& FolderName, FText& Reason)
{
	// Check length of the folder name
	if ( FolderName.Len() == 0 )
	{
		Reason = LOCTEXT( "InvalidFolderName_IsTooShort", "Please provide a name for this folder." );
		return false;
	}

	if ( FolderName.Len() > GetMaxCookPathLen())
	{
		Reason = FText::Format( LOCTEXT("InvalidFolderName_TooLongForCooking", "Filename '{0}' is too long; this may interfere with cooking for consoles. Unreal filenames should be no longer than {1} characters." ),
			FText::FromString(FolderName), FText::AsNumber(GetMaxCookPathLen()) );
		return false;
	}

	const FString InvalidChars = INVALID_LONGPACKAGE_CHARACTERS TEXT("/[]"); // Slash and Square brackets are invalid characters for a folder name

	// See if the name contains invalid characters.
	FString Char;
	for( int32 CharIdx = 0; CharIdx < FolderName.Len(); ++CharIdx )
	{
		Char = FolderName.Mid(CharIdx, 1);

		if ( InvalidChars.Contains(*Char) )
		{
			FString ReadableInvalidChars = InvalidChars;
			ReadableInvalidChars.ReplaceInline(TEXT("\r"), TEXT(""));
			ReadableInvalidChars.ReplaceInline(TEXT("\n"), TEXT(""));
			ReadableInvalidChars.ReplaceInline(TEXT("\t"), TEXT(""));

			Reason = FText::Format(LOCTEXT("InvalidFolderName_InvalidCharacters", "A folder name may not contain any of the following characters: {0}"), FText::FromString(ReadableInvalidChars));
			return false;
		}
	}

	// Check custom filter set by external module
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().IsNameAllowed(FolderName, &Reason))
	{
		return false;
	}
	
	return FFileHelper::IsFilenameValidForSaving( FolderName, Reason );
}

bool AssetViewUtils::DoesFolderExist(const FString& FolderPath)
{
	TArray<FString> SubPaths;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetSubPaths(FPaths::GetPath(FolderPath), SubPaths, false);

	for(auto SubPathIt(SubPaths.CreateConstIterator()); SubPathIt; SubPathIt++)	
	{
		if ( *SubPathIt == FolderPath )
		{
			return true;
		}
	}

	return false;
}

const TSharedPtr<FLinearColor> AssetViewUtils::LoadColor(const FString& FolderPath)
{
	TOptional<FLinearColor> FoundColor = GetPathColor(FolderPath);
	if (FoundColor.IsSet())
	{
		return MakeShared<FLinearColor>(FoundColor.GetValue());
	}
	return TSharedPtr<FLinearColor>();
}

TOptional<FLinearColor> AssetViewUtils::GetPathColor(const FString& FolderPath)
{
	auto GetPathColorInternal = [](const FString& InPath) -> TOptional<FLinearColor>
	{
		// See if we have a value cached first
		FLinearColor* CachedColor = PathColors.Find(InPath);
		if(CachedColor)
		{
			return *CachedColor;
		}
		
		// Loads the color of folder at the given path from the config
		if(FPaths::FileExists(GEditorPerProjectIni))
		{
			// Create a new entry from the config, skip if it's default
			FString ColorStr;
			if(GConfig->GetString(TEXT("PathColor"), *InPath, ColorStr, GEditorPerProjectIni))
			{
				FLinearColor Color;
				if(Color.InitFromString(ColorStr) && !Color.Equals(GetDefaultColor()))
				{
					return PathColors.Add(InPath, FLinearColor(Color));
				}
			}
			else
			{
				return PathColors.Add(InPath, FLinearColor(GetDefaultColor()));
			}
		}

		return TOptional<FLinearColor>();
	};

	// First try and find the color using the given path, as this works correctly for both assets and classes
	TOptional<FLinearColor> FoundColor = GetPathColorInternal(FolderPath);
	if(FoundColor.IsSet())
	{
		return FoundColor;
	}

	// If that failed, try and use the filename (assets used to use this as their color key, but it doesn't work with classes)
	{
		FString RelativePath;
		if (FPackageName::TryConvertLongPackageNameToFilename(FolderPath / FString(), RelativePath))
		{
			return GetPathColorInternal(RelativePath);
		}
	}

	return TOptional<FLinearColor>();
}

void AssetViewUtils::SaveColor(const FString& FolderPath, const TSharedPtr<FLinearColor>& FolderColor, bool bForceAdd)
{
	TOptional<FLinearColor> OptionalFolderColor;
	if (FolderColor)
	{
		OptionalFolderColor = *FolderColor.Get();
	}
	else if (bForceAdd)
	{
		OptionalFolderColor = GetDefaultColor();
	}
	SetPathColor(FolderPath, OptionalFolderColor);
}

void AssetViewUtils::SetPathColor(const FString& FolderPath, TOptional<FLinearColor> FolderColor)
{
	auto SetPathColorInternal = [](const FString& InPath, FLinearColor InFolderColor)
	{
		// Saves the color of the folder to the config
		if(FPaths::FileExists(GEditorPerProjectIni))
		{
			GConfig->SetString(TEXT("PathColor"), *InPath, *InFolderColor.ToString(), GEditorPerProjectIni);
		}

		// Update the map too
		PathColors.Add(InPath, InFolderColor);

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		ContentBrowserModule.GetOnSetFolderColor().Broadcast(InPath);
	};

	auto RemoveColorInternal = [](const FString& InPath)
	{
		// Remove the color of the folder from the config
		if(FPaths::FileExists(GEditorPerProjectIni))
		{
			GConfig->RemoveKey(TEXT("PathColor"), *InPath, GEditorPerProjectIni);
		}

		// Update the map too
		PathColors.Remove(InPath);
	};

	// Remove the color if it's invalid or default
	const bool bRemove = !FolderColor.IsSet() || FolderColor->Equals(GetDefaultColor());
	if(bRemove)
	{
		RemoveColorInternal(FolderPath);
	}
	else
	{
		SetPathColorInternal(FolderPath, FolderColor.GetValue());
	}

	// Make sure and remove any colors using the legacy path format
	{
		FString RelativePath;
		if (FPackageName::TryConvertLongPackageNameToFilename(FolderPath / FString(), RelativePath))
		{
			return RemoveColorInternal(RelativePath);
		}
	}
}

bool AssetViewUtils::HasCustomColors( TArray< FLinearColor >* OutColors )
{
	// Check to see how many paths are currently using this color
	// Note: we have to use the config, as paths which haven't been rendered yet aren't registered in the map
	bool bHasCustom = false;
	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		// Read individual entries from a config file.
		TArray< FString > Section; 
		GConfig->GetSection( TEXT("PathColor"), Section, GEditorPerProjectIni );

		for( int32 SectionIndex = 0; SectionIndex < Section.Num(); SectionIndex++ )
		{
			FString EntryStr = Section[ SectionIndex ];
			EntryStr.TrimStartInline();

			FString PathStr;
			FString ColorStr;
			if ( EntryStr.Split( TEXT( "=" ), &PathStr, &ColorStr ) )
			{
				// Ignore any that have invalid or default colors
				FLinearColor CurrentColor;
				if( CurrentColor.InitFromString( ColorStr ) && !CurrentColor.Equals( GetDefaultColor() ) )
				{
					bHasCustom = true;
					if ( OutColors )
					{
						// Only add if not already present (ignores near matches too)
						bool bAdded = false;
						for( int32 ColorIndex = 0; ColorIndex < OutColors->Num(); ColorIndex++ )
						{
							const FLinearColor& Color = (*OutColors)[ ColorIndex ];
							if( CurrentColor.Equals( Color ) )
							{
								bAdded = true;
								break;
							}
						}
						if ( !bAdded )
						{
							OutColors->Add( CurrentColor );
						}
					}
					else
					{
						break;
					}
				}
			}
		}
	}
	return bHasCustom;
}

FLinearColor AssetViewUtils::GetDefaultColor()
{
	// The default tint the folder should appear as
	static const FName FolderColorName("ContentBrowser.DefaultFolderColor");
	return FAppStyle::Get().GetSlateColor(FolderColorName).GetSpecifiedColor();
}

namespace AssetViewUtils
{
static const TConsoleVariableData<int32>* CVarMaxFullPathLength = IConsoleManager::Get().RegisterConsoleVariable(TEXT("MaxAssetFullPath"), 0, TEXT("Maximum full path name of an asset, or <=0 to use GetMaxCookPathLen()."))->AsVariableInt();
static const TConsoleVariableData<FString>* CVarExternalPluginCookedRootPath = IConsoleManager::Get().RegisterConsoleVariable(TEXT("ExternalPluginCookedAssetRootPath"), TEXT(""), TEXT("Root path to use when estimating the cooked path external plugin assets, or empty to use the standard engine/project root."))->AsVariableString();

int32 GetMaxFullPathLength()
{
	int32 MaxFullPathLength = CVarMaxFullPathLength->GetValueOnGameThread();
	if (MaxFullPathLength <= 0)
	{
		MaxFullPathLength = GetMaxCookPathLen();
	}
	return MaxFullPathLength;
}

}

bool AssetViewUtils::IsValidObjectPathForCreate(const FString& ObjectPath, FText& OutErrorMessage, bool bAllowExistingAsset)
{
	return IsValidObjectPathForCreate(ObjectPath, nullptr, OutErrorMessage, bAllowExistingAsset);
}

bool AssetViewUtils::IsValidObjectPathForCreate(const FString& ObjectPath, const UClass* ObjectClass, FText& OutErrorMessage, bool bAllowExistingAsset)
{
	const FString ObjectName = FPackageName::ObjectPathToPathWithinPackage(ObjectPath);

	// Make sure the new name only contains valid characters
	if (!FName::IsValidXName( ObjectName, INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &OutErrorMessage))
	{
		// Return false to indicate that the user should enter a new name
		return false;
	}

	const FString PackageName = FPackageName::ObjectPathToPackageName(ObjectPath);

	if (!IsValidPackageForCooking(PackageName, OutErrorMessage))
	{
		return false;
	}

	// Make sure we are not creating an FName that is too large
	if ( ObjectPath.Len() >= NAME_SIZE )
	{
		OutErrorMessage = FText::Format(LOCTEXT("AssetNameTooLong", "The name for the asset is too long, the maximum is '{0}'. \nPlease choose a shorter name for the asset or create it in a shallower folder structure."),
			FText::AsNumber(NAME_SIZE - 1));
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Make sure we are not creating an path that is too long for the OS
	FString RelativePathFilename;
	if (!FPackageName::TryConvertLongPackageNameToFilename(PackageName, RelativePathFilename, FPackageName::GetAssetPackageExtension()))	// full relative path with name + extension
	{
		OutErrorMessage = LOCTEXT("ConvertToFilename", "Package name could not be converted to file name.");
		return false;
	}

	const FString FullPath = FPaths::ConvertRelativePathToFull(RelativePathFilename);	// path to file on disk
	if (int32 MaxFullPathLength = GetMaxFullPathLength();
		FullPath.Len() > MaxFullPathLength)
	{
		// The full path for the asset is too long
		OutErrorMessage = FText::Format(LOCTEXT("AssetPathTooLong", "The absolute file path for the asset is too long, the maximum is '{0}'. \nPlease choose a shorter name for the asset or create it in a shallower folder structure."),
			FText::AsNumber(MaxFullPathLength));
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Check for an existing asset
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData ExistingAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
	if (ExistingAsset.IsValid())
	{
		// An asset of a different type already exists at this location, inform the user and continue
		if (ObjectClass && !ExistingAsset.IsInstanceOf(ObjectClass))
		{
			OutErrorMessage = FText::Format(LOCTEXT("RenameAssetOtherTypeAlreadyExists", "An asset of type '{0}' already exists at this location with the name '{1}'."), FText::FromString(ExistingAsset.AssetClassPath.ToString()), FText::FromString(ObjectName));
			
			// Return false to indicate that the user should enter a new name
			return false;
		}
        // This asset already exists at this location, warn user if asked to.
		else if (!bAllowExistingAsset)
		{
			OutErrorMessage = FText::Format( LOCTEXT("RenameAssetAlreadyExists", "An asset already exists at this location with the name '{0}'."), FText::FromString( ObjectName ) );

			// Return false to indicate that the user should enter a new name
			return false;
		}
	}

	// Check custom filter set by external module
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().IsNameAllowed(ObjectName, &OutErrorMessage))
	{
		return false;
	}

	/** 
	 * Make sure the name is not already a class or otherwise invalid for saving
	 * Some other function above also test for some issues that this function can also find,
	 * but they have an better error message more suited for the asset view.
	 * Because of this, this test should be the last one to be run.
	 */
	if ( !FFileHelper::IsFilenameValidForSaving(ObjectName, OutErrorMessage) )
	{
		// Return false to indicate that the user should enter a new name
		return false;
	}


	return true;
}

bool AssetViewUtils::IsValidFolderPathForCreate(const FString& InFolderPath, const FString& NewFolderName, FText& OutErrorMessage, bool bAllowExistingFolder)
{
	if (!IsValidFolderName(NewFolderName, OutErrorMessage))
	{
		return false;
	}

	const FString NewFolderPath = InFolderPath / NewFolderName;

	if (!bAllowExistingFolder && DoesFolderExist(NewFolderPath))
	{
		OutErrorMessage = LOCTEXT("RenameFolderAlreadyExists", "A folder already exists at this location with this name.");
		return false;
	}

	// Make sure we are not creating an FName that is too large
	if (NewFolderPath.Len() >= NAME_SIZE)
	{
		OutErrorMessage = FText::Format(LOCTEXT("PackagePathTooLong", "The path for the folder is too deep, the maximum is '{0}'. \nPlease choose a shorter name for the folder or create it in a shallower folder structure."),
			FText::AsNumber(NAME_SIZE - 1));
		// Return false to indicate that the user should enter a new name
		return false;
	}

	// Make sure we are not creating a folder path that is too long
	if (NewFolderPath.Len() > GetMaxCookPathLen() - MAX_CLASS_NAME_LENGTH)
	{
		// The full path for the folder is too long
		OutErrorMessage = FText::Format(LOCTEXT("RenameFolderPathTooLong", "The full path for the folder is too deep, the maximum is '{0}'. Please choose a shorter name for the folder or create it in a shallower folder structure."),
			FText::AsNumber(GetMaxCookPathLen()));
		// Return false to indicate that the user should enter a new name for the folder
		return false;
	}

	const bool bDisplayL10N = GetDefault<UContentBrowserSettings>()->GetDisplayL10NFolder();
	if (!bDisplayL10N && FPackageName::IsLocalizedPackage(NewFolderPath))
	{
		OutErrorMessage = LOCTEXT("LocalizationFolderReserved", "The L10N folder is reserved for localized content and is currently hidden.");
		return false;
	}

	return true;
}

FString AssetViewUtils::GetPackagePathWithinRoot(const FString& PackageName)
{
	const FString AbsoluteRootPath = FPaths::ConvertRelativePathToFull(FPaths::RootDir());

	FString RelativePathToAsset;
	if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, RelativePathToAsset))
	{
		const FString AbsolutePathToAsset = FPaths::ConvertRelativePathToFull(RelativePathToAsset);

		RelativePathToAsset = AbsolutePathToAsset;
		FPaths::RemoveDuplicateSlashes(RelativePathToAsset);
		RelativePathToAsset.RemoveFromStart(AbsoluteRootPath, ESearchCase::CaseSensitive);
	}

	return RelativePathToAsset;
}

int32 AssetViewUtils::GetPackageLengthForCooking(const FString& PackageName, bool bIsInternalBuild, FString CookSubPath)
{
	FString RelativePathToAsset;
	if (!FPackageName::TryConvertLongPackageNameToFilename(PackageName, RelativePathToAsset, FPackageName::GetAssetPackageExtension()))
	{
		return 0;
	}

	const TCHAR* GameName = FApp::GetProjectName();

	const TSharedPtr<IPlugin> PluginContainingAsset = GetPluginForFolder(PackageName);
	const bool bIsEngineAsset = IsEngineFolder(PackageName) || (PluginContainingAsset && PluginContainingAsset->GetLoadedFrom() == EPluginLoadedFrom::Engine);
	const bool bIsProjectAsset = !bIsEngineAsset;
	
	// If the user didn't supply a custom cook sub path, we use "Android_ASTCClient" below as it's probably the longest platform name, so will also prove that any shorter platform names will validate correctly
	if (CookSubPath.IsEmpty())
	{
		CookSubPath = FPaths::Combine(TEXT("Saved"), TEXT("Cooked"), TEXT("Android_ASTCClient"), bIsEngineAsset ? TEXT("Engine") : GameName, TEXT("")); // Trailing empty entry ensures a trailing slash
	}	
	else
	{
		CookSubPath = FPaths::Combine(CookSubPath, bIsEngineAsset ? TEXT("Engine") : GameName, TEXT("")); // Trailing empty entry ensures a trailing slash
	}

	const FString AbsolutePathToAsset = FPaths::ConvertRelativePathToFull(RelativePathToAsset);

	FString AbsoluteTargetPath = FPaths::ConvertRelativePathToFull(bIsEngineAsset ? FPaths::EngineDir() : FPaths::ProjectDir());

	int32 AssetPathRelativeToCookRootLen = AbsolutePathToAsset.Len();
	if (AbsolutePathToAsset.StartsWith(AbsoluteTargetPath, ESearchCase::IgnoreCase))
	{
		AssetPathRelativeToCookRootLen -= AbsoluteTargetPath.Len();
	}
	else if (ensureMsgf(PluginContainingAsset, TEXT("Only plugins can exist outside of the expected target path of '%s'. '%s' will not calculate an accurate result!"), *AbsoluteTargetPath, *AbsolutePathToAsset))
	{
		const FString AbsolutePluginRootPath = FPaths::ConvertRelativePathToFull(PluginContainingAsset->GetBaseDir());
		if (ensureMsgf(AbsolutePathToAsset.StartsWith(AbsolutePluginRootPath, ESearchCase::IgnoreCase), TEXT("%s should start with %s"), *AbsolutePathToAsset, *AbsolutePluginRootPath))
		{
			AssetPathRelativeToCookRootLen -= AbsolutePluginRootPath.Len();
			AssetPathRelativeToCookRootLen += FCString::Strlen(TEXT("Plugins/GameFeatures/XXXXXXXX-XXXX-XXXX-XXXX-XXXXXXXXXXXX")); // Use a GUID instead of the plugin name, as some external plugins cook as a GUID
		}

		if (FString ExternalPluginCookedRootPath = CVarExternalPluginCookedRootPath->GetValueOnGameThread(); ExternalPluginCookedRootPath.Len() > 0)
		{
			bIsInternalBuild = false;
			AbsoluteTargetPath = MoveTemp(ExternalPluginCookedRootPath);
			AbsoluteTargetPath /= FString();
		}
	}

	if (bIsInternalBuild)
	{
		// We assume a constant size for the build machine base path for things that reside within the UE source tree
		const FString AbsoluteUERootPath = FPaths::ConvertRelativePathToFull(FPaths::RootDir());
		if (AbsoluteTargetPath.StartsWith(AbsoluteUERootPath, ESearchCase::IgnoreCase))
		{
			// Project is within the UE source tree, so remove the UE root path
			int32 CookPathRelativeToTargetRootLen = CookSubPath.Len();
			CookPathRelativeToTargetRootLen += (AbsoluteTargetPath.Len() - AbsoluteUERootPath.Len());

			int32 InternalCookPathLen = FCString::Strlen(TEXT("d:/build/XXX+RXX.XX+Inc/Sync")) + CookPathRelativeToTargetRootLen + AssetPathRelativeToCookRootLen;

			// Only add game name padding to non-GameFeature project plugins so that they can ported to other projects.
			// GameFeaturePlugins are not designed to be ported between projects since they can depend on /Game/ assets and non-plugin code.
			const bool bIsGameFeaturePlugin = AbsolutePathToAsset.Contains(TEXT("/Plugins/GameFeatures/"));
			if (PluginContainingAsset && bIsProjectAsset && !bIsGameFeaturePlugin)
			{
				// We assume the game name is 20 characters (the maximum allowed) to make sure that content can be ported between projects
				constexpr int32 MaxGameNameLen = 20;
				InternalCookPathLen += FMath::Max(0, MaxGameNameLen - FCString::Strlen(GameName));
			}

			return InternalCookPathLen;
		}
	}
	
	// Test that the package can be cooked based on the current project path
	return AbsoluteTargetPath.Len() + CookSubPath.Len() + AssetPathRelativeToCookRootLen;
}

bool AssetViewUtils::IsValidPackageForCooking(const FString& PackageName, FText& OutErrorMessage)
{
	int32 AbsoluteCookPathToAssetLength = GetPackageLengthForCooking(PackageName, FEngineBuildSettings::IsInternalBuild());

	int32 MaxCookPathLen = GetMaxCookPathLen();
	if (AbsoluteCookPathToAssetLength > MaxCookPathLen)
	{
		// See TTP# 332328:
		// The following checks are done mostly to prevent / alleviate the problems that "long" paths are causing with the BuildFarm and cooked builds.
		// The BuildFarm uses a verbose path to encode extra information to provide more information when things fail, however this makes the path limitation a problem.
		//	- We assume a base path of length: D:/BuildFarm/buildmachine_++depot+UE-Releases+XX.XX/
		//	- We assume the game name is 20 characters (the maximum allowed) to make sure that content can be ported between projects
		//	- We calculate the cooked game path relative to the game root (eg, Showcases/Infiltrator/Saved/Cooked/Windows/Infiltrator)
		//	- We calculate the asset path relative to (and including) the Content directory (eg, Content/Environment/Infil1/Infil1_Underground/Infrastructure/Model/SM_Infil1_Tunnel_Ceiling_Pipes_1xEntryCurveOuter_Double.uasset)
		if (FEngineBuildSettings::IsInternalBuild())
		{
			// The projected length of the path for cooking is too long
			OutErrorMessage = FText::Format(LOCTEXT("AssetCookingPathTooLongForBuildMachine", "The path to the asset is too long '{0}' for cooking by the build machines, the maximum is '{1}'\nPlease choose a shorter name for the asset or create it in a shallower folder structure with shorter folder names."), FText::AsNumber(AbsoluteCookPathToAssetLength), FText::AsNumber(MaxCookPathLen));
		}
		else
		{
			// The projected length of the path for cooking is too long
			OutErrorMessage = FText::Format(LOCTEXT("AssetCookingPathTooLong", "The path to the asset is too long '{0}', the maximum for cooking is '{1}'\nPlease choose a shorter name for the asset or create it in a shallower folder structure with shorter folder names."), FText::AsNumber(AbsoluteCookPathToAssetLength), FText::AsNumber(MaxCookPathLen));
		}

		// Return false to indicate that the user should enter a new name
		return false;
	}

	return true;
}

int32 AssetViewUtils::GetMaxAssetPathLen()
{
	const int32 PathRootAffordance = 50;

	if ( GetDefault<UEditorExperimentalSettings>()->bEnableLongPathsSupport )
	{
		// Allow the longest path allowed by the system
		return FPlatformMisc::GetMaxPathLength() - PathRootAffordance;
	}
	else
	{
		// 260 characters is the limit on Windows, which is the shortest max path of any platforms that support cooking
		return 260 - PathRootAffordance;
	}
}

int32 AssetViewUtils::GetMaxCookPathLen()
{
	if (GetDefault<UEditorExperimentalSettings>()->bEnableLongPathsSupport)
	{
		// Allow the longest path allowed by the system
		return FPlatformMisc::GetMaxPathLength();
	}
	else
	{
		// 260 characters is the limit on Windows, which is the shortest max path of any platforms that support cooking
		return 260;
	}
}

/** Given an set of packages that will be synced by a SCC operation, report any dependencies that are out-of-date and aren't in the list of packages to be synced */
void GetOutOfDatePackageDependencies(const TArray<FString>& InPackagesThatWillBeSynced, TArray<FString>& OutDependenciesThatAreOutOfDate)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Build up the initial list of known packages
	// We add to these as we find new dependencies to process
	TSet<FName> AllPackages;
	TArray<FName> AllPackagesArray;
	{
		AllPackages.Reserve(InPackagesThatWillBeSynced.Num());
		AllPackagesArray.Reserve(InPackagesThatWillBeSynced.Num());

		for (const FString& PackageName : InPackagesThatWillBeSynced)
		{
			const FName PackageFName = *PackageName;
			AllPackages.Emplace(PackageFName);
			AllPackagesArray.Emplace(PackageFName);
		}
	}

	// Build up the complete set of package dependencies
	TArray<FString> AllDependencies;
	{
		for (int32 PackageIndex = 0; PackageIndex < AllPackagesArray.Num(); ++PackageIndex)
		{
			const FName PackageName = AllPackagesArray[PackageIndex];

			TArray<FName> PackageDependencies;
			AssetRegistryModule.GetDependencies(PackageName, PackageDependencies, UE::AssetRegistry::EDependencyCategory::Package);

			for (const FName& PackageDependency : PackageDependencies)
			{
				if (!AllPackages.Contains(PackageDependency))
				{
					AllPackages.Emplace(PackageDependency);
					AllPackagesArray.Emplace(PackageDependency);

					FString PackageDependencyStr = PackageDependency.ToString();
					if (!FPackageName::IsScriptPackage(PackageDependencyStr) && FPackageName::IsValidLongPackageName(PackageDependencyStr))
					{
						AllDependencies.Emplace(MoveTemp(PackageDependencyStr));
					}
				}
			}
		}
	}

	// Query SCC to see which dependencies are out-of-date
	if (AllDependencies.Num() > 0)
	{
		ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();

		TArray<FString> DependencyFilenames = SourceControlHelpers::PackageFilenames(AllDependencies);
		for (int32 DependencyIndex = 0; DependencyIndex < AllDependencies.Num(); ++DependencyIndex)
		{
			// Dependency data may contain files that no longer exist on disk; strip those from the list now
			if (!FPaths::FileExists(DependencyFilenames[DependencyIndex]))
			{
				AllDependencies.RemoveAt(DependencyIndex, EAllowShrinking::No);
				DependencyFilenames.RemoveAt(DependencyIndex, EAllowShrinking::No);
				--DependencyIndex;
			}
		}

		SCCProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), DependencyFilenames);
		for (int32 DependencyIndex = 0; DependencyIndex < AllDependencies.Num(); ++DependencyIndex)
		{
			const FString& DependencyName = AllDependencies[DependencyIndex];
			const FString& DependencyFilename = DependencyFilenames[DependencyIndex];

			FSourceControlStatePtr SCCState = SCCProvider.GetState(DependencyFilename, EStateCacheUsage::Use);
			if (SCCState.IsValid() && !SCCState->IsCurrent())
			{
				OutDependenciesThatAreOutOfDate.Emplace(DependencyName);
			}
		}
	}
}

void ShowSyncDependenciesDialog(const TArray<FString>& InDependencies, TArray<FString>& OutExtraPackagesToSync)
{
	if (InDependencies.Num() > 0)
	{
		FPackagesDialogModule& PackagesDialogModule = FModuleManager::LoadModuleChecked<FPackagesDialogModule>(TEXT("PackagesDialog"));

		PackagesDialogModule.CreatePackagesDialog(
			LOCTEXT("SyncAssetDependenciesTitle", "Sync Asset Dependencies"),
			LOCTEXT("SyncAssetDependenciesMessage", "The following assets have newer versions available, but aren't selected to be synced.\nSelect any additional dependencies you would like to sync in order to avoid potential issues loading the updated packages.")
		);

		PackagesDialogModule.AddButton(
			DRT_CheckOut,
			LOCTEXT("SyncDependenciesButton", "Sync"),
			LOCTEXT("SyncDependenciesButtonTip", "Sync with the selected dependencies included")
		);

		for (const FString& DependencyName : InDependencies)
		{
			if (UPackage* Package = FindPackage(nullptr, *DependencyName))
			{
				PackagesDialogModule.AddPackageItem(Package, ECheckBoxState::Checked);
			}
		}

		const EDialogReturnType UserResponse = PackagesDialogModule.ShowPackagesDialog();

		if (UserResponse == DRT_CheckOut)
		{
			TArray<UPackage*> SelectedPackages;
			PackagesDialogModule.GetResults(SelectedPackages, ECheckBoxState::Checked);

			for (UPackage* SelectedPackage : SelectedPackages)
			{
				if (SelectedPackage)
				{
					OutExtraPackagesToSync.Emplace(SelectedPackage->GetName());
				}
			}
		}
	}
}

void AssetViewUtils::SyncPackagesFromSourceControl(const TArray<FString>& PackageNames, bool bIsSyncLatestOperation)
{
	if (bIsSyncLatestOperation)
	{
		SyncLatestFromSourceControl();
	}
	else
	{
		SyncPackagesFromSourceControl(PackageNames);
	}
}

static void FullyLoadAndResetLoaderForPackages(TArray<FString>& InPackageNamesToLoad, TArray<UPackage*>& OutLoadedPackages)
{
	TArray<UObject*, TInlineAllocator<32>> LoadedObjects;
	TArray<int32, TInlineAllocator<32>> PendingPackageRequestIds;

	LoadedObjects.Reserve(InPackageNamesToLoad.Num());
	OutLoadedPackages.Reserve(InPackageNamesToLoad.Num());
	PendingPackageRequestIds.Reserve(InPackageNamesToLoad.Num());
	for (const FString& PackageName : InPackageNamesToLoad)
	{
		UPackage* Package = FindPackage(nullptr, *PackageName);
		if (Package)
		{
			LoadedObjects.Emplace(Package);
			OutLoadedPackages.Emplace(Package);

			if (!Package->IsFullyLoaded())
			{
				PendingPackageRequestIds.Emplace(LoadPackageAsync(PackageName));
			}
		}
	}

	// Detach the linkers of any loaded packages so that SCC can overwrite the files...
	FlushAsyncLoading(PendingPackageRequestIds);
	ResetLoaders(LoadedObjects);
}

bool AssetViewUtils::SyncPackagesFromSourceControl(const TArray<FString>& PackageNames)
{
	if (PackageNames.Num() > 0)
	{
		// Broadcast sync starting...
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		AssetViewUtils::OnSyncStart().Broadcast();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;

		TArray<FString> PackageNamesToSync = PackageNames;

		// Warn about any packages that are being synced without also getting the newest version of their dependencies...
		TArray<FString> OutOfDateDependencies;
		GetOutOfDatePackageDependencies(PackageNamesToSync, OutOfDateDependencies);

		TArray<FString> ExtraPackagesToSync;
		ShowSyncDependenciesDialog(OutOfDateDependencies, ExtraPackagesToSync);

		PackageNamesToSync.Append(ExtraPackagesToSync);

		ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();
		const TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(PackageNamesToSync);

		// Form a list of loaded packages to reload...
		TArray<UPackage*> LoadedPackages;
		FullyLoadAndResetLoaderForPackages(PackageNamesToSync, LoadedPackages);

		// Sync everything...
		TSharedRef<FSync> Operation = ISourceControlOperation::Create<FSync>();
		ECommandResult::Type SyncResult = SCCProvider.Execute(Operation, PackageFilenames);

		// Syncing may have deleted some packages, so we need to unload those rather than re-load them...
		// Note: we will store the package using weak pointers here otherwise we might have garbage collection issues after the ReloadPackages call
		TArray<TWeakObjectPtr<UPackage>> PackagesToUnload;
		LoadedPackages.RemoveAll([&](UPackage* InPackage) -> bool
		{
			const FString PackageExtension = InPackage->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			const FString PackageFilename = FPackageName::LongPackageNameToFilename(InPackage->GetName(), PackageExtension);
			if (!FPaths::FileExists(PackageFilename))
			{
				PackagesToUnload.Emplace(MakeWeakObjectPtr(InPackage));
				return true; // remove package
			}
			return false; // keep package
		});

		// Hot-reload the new packages...
		if (LoadedPackages.Num() > 0)
		{
			FBookmarkScoped BookmarkScoped;
			FScopedSlowTask SlowTaskScoped(1.f, NSLOCTEXT("SourceControl", "Reload", "Reloading..."));
			SlowTaskScoped.MakeDialog();
			SlowTaskScoped.EnterProgressFrame(1.f);

			UPackageTools::ReloadPackages(LoadedPackages);
		}

		// Unload any deleted packages...
		TArray<UPackage*> PackageRawPtrsToUnload;
		for (TWeakObjectPtr<UPackage>& PackageToUnload : PackagesToUnload)
		{
			if (PackageToUnload.IsValid())
			{
				PackageRawPtrsToUnload.Emplace(PackageToUnload.Get());
			}
		}

		UPackageTools::UnloadPackages(PackageRawPtrsToUnload);

		// Re-cache the SCC state...
		SCCProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), PackageFilenames);

		// Broadcast sync finished...
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		AssetViewUtils::OnSyncFinish().Broadcast(SyncResult == ECommandResult::Succeeded, &PackageFilenames);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;

		// Return result...
		return (SyncResult == ECommandResult::Succeeded);
	}

	return true;
}

static bool SyncPathsFromSourceControl(const FString& Revision, const TArray<FString>& Paths, bool bCheckDependencies)
{
	TArray<FString> PathsOnDisk;
	PathsOnDisk.Reserve(Paths.Num());
	for (const FString& Path : Paths)
	{
		FString PathOnDisk;
		if (FPackageName::TryConvertLongPackageNameToFilename(Path / TEXT(""), PathOnDisk))
		{
			// Given path was a content path.
		}
		else
		{
			// Treat path as a (relative or absolute) disk path.
			PathOnDisk = FPaths::ConvertRelativePathToFull(Path);
		}

		if (FPaths::DirectoryExists(PathOnDisk))
		{
			PathsOnDisk.Emplace(MoveTemp(PathOnDisk));
		}
	}

	if (PathsOnDisk.Num() > 0)
	{
		// Broadcast sync starting...
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		AssetViewUtils::OnSyncStart().Broadcast();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;

		ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();

		// Get all the assets about to be synced in those path(s) on disk...
		TArray<FString> AffectedFiles;
		TArray<FString> PackageNames;

		// Use FSyncPreview if possible.
		TSharedRef<FSyncPreview> PreviewOperation = ISourceControlOperation::Create<FSyncPreview>();
		PreviewOperation->SetRevision(Revision);
		if (SCCProvider.CanExecuteOperation(PreviewOperation) &&
			SCCProvider.Execute(PreviewOperation, PathsOnDisk) == ECommandResult::Succeeded)
		{
			// The source control provider supports sync previews which gets us the full list of
			// affected files, both packages and non-packages.
			AffectedFiles = PreviewOperation->GetAffectedFiles();

			// Determine which packages would be affected.
			for (const FString& FileName : AffectedFiles)
			{
				FString PackageName;
				if (FPackageName::TryConvertFilenameToLongPackageName(FileName, PackageName))
				{
					PackageNames.Add(PackageName);
				}
			}
		}
		else
		{
			// The source control provider does not support sync previews.
			// Fallback on using the AssetRegistry.
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			FARFilter Filter;
			Filter.bRecursivePaths = true;
			for (const FString& PathOnDisk : PathsOnDisk)
			{
				FString PackagePath;
				if (FPackageName::TryConvertFilenameToLongPackageName(PathOnDisk, PackagePath))
				{
					if (PackagePath.Len() > 1 && PackagePath[PackagePath.Len() - 1] == TEXT('/'))
					{
						// The filter path can't end with a trailing slash
						PackagePath.LeftChopInline(1, EAllowShrinking::No);
					}
					Filter.PackagePaths.Emplace(*PackagePath);
				}
			}

			TArray<FAssetData> AssetList;
			AssetRegistryModule.Get().GetAssets(Filter, AssetList);

			TSet<FName> UniquePackageNames;
			for (const FAssetData& Asset : AssetList)
			{
				bool bWasInSet = false;
				UniquePackageNames.Add(Asset.PackageName, &bWasInSet);
				if (!bWasInSet)
				{
					PackageNames.Add(Asset.PackageName.ToString());
				}
			}
		}

		// Warn about any packages that are being synced without also getting the newest version of their dependencies...
		TArray<FString> PackageNamesToSync = PackageNames;
		TArray<FString> ExtraPackagesToSync;
		if (bCheckDependencies)
		{
			TArray<FString> OutOfDateDependencies;
			GetOutOfDatePackageDependencies(PackageNamesToSync, OutOfDateDependencies);

			ShowSyncDependenciesDialog(OutOfDateDependencies, ExtraPackagesToSync);

			PackageNamesToSync.Append(ExtraPackagesToSync);
		}

		// Form a list of loaded packages to reload...
		TArray<UPackage*> LoadedPackages;
		FullyLoadAndResetLoaderForPackages(PackageNamesToSync, LoadedPackages);

		// Group everything...
		TArray<FString> PathsToSync;
		PathsToSync.Reserve(PathsOnDisk.Num() + ExtraPackagesToSync.Num());
		PathsToSync.Append(PathsOnDisk);
		PathsToSync.Append(SourceControlHelpers::PackageFilenames(ExtraPackagesToSync));

		// Sync everything...
		TSharedRef<FSync> Operation = ISourceControlOperation::Create<FSync>();
		Operation->SetRevision(Revision);
		ECommandResult::Type SyncResult = SCCProvider.Execute(Operation, PathsToSync);

		// Syncing may have deleted some packages, so we need to unload those rather than re-load them...
		// Note: we will store the package using weak pointers here otherwise we might have garbage collection issues after the ReloadPackages call
		TArray<TWeakObjectPtr<UPackage>> PackagesToUnload;
		LoadedPackages.RemoveAll([&](UPackage* InPackage) -> bool
		{
			const FString PackageExtension = InPackage->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			const FString PackageFilename = FPackageName::LongPackageNameToFilename(InPackage->GetName(), PackageExtension);
			if (!FPaths::FileExists(PackageFilename))
			{
				PackagesToUnload.Emplace(MakeWeakObjectPtr(InPackage));
				return true; // remove package
			}
			return false; // keep package
		});

		// Check if the world should be reloaded as well because one of its external packages got synced...
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
		{
			UPackage* EditorWorldPackage = EditorWorld->GetPackage();
			if (!LoadedPackages.Contains(EditorWorldPackage))
			{
				for (const FString& PackageName : PackageNamesToSync)
				{
					if (PackageName.Contains(FPackagePath::GetExternalActorsFolderName()) ||
						PackageName.Contains(FPackagePath::GetExternalObjectsFolderName()))
					{
						LoadedPackages.Add(EditorWorldPackage);
						break;
					}
				}
			}
		}

		// Hot-reload the new packages...
		if (LoadedPackages.Num() > 0)
		{
			FBookmarkScoped BookmarkScoped;
			FScopedSlowTask SlowTaskScoped(1.f, NSLOCTEXT("SourceControl", "Reload", "Reloading..."));
			SlowTaskScoped.MakeDialog();
			SlowTaskScoped.EnterProgressFrame(1.f);

			UPackageTools::ReloadPackages(LoadedPackages);
		}

		// Unload any deleted packages...
		TArray<UPackage*> PackageRawPtrsToUnload;
		for (TWeakObjectPtr<UPackage>& PackageToUnload : PackagesToUnload)
		{
			if (PackageToUnload.IsValid())
			{
				PackageRawPtrsToUnload.Emplace(PackageToUnload.Get());
			}
		}

		UPackageTools::UnloadPackages(PackageRawPtrsToUnload);

		// Re-cache the SCC state...
		SCCProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), AffectedFiles.Num() > 0 ? AffectedFiles : PathsOnDisk);

		// Broadcast sync finished...
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		AssetViewUtils::OnSyncFinish().Broadcast(SyncResult == ECommandResult::Succeeded, AffectedFiles.Num() > 0 ? &AffectedFiles : nullptr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;

		// Return result...
		return (SyncResult == ECommandResult::Succeeded);
	}

	return true;
}

static bool SyncPathsFromSourceControl(const TArray<FString>& Paths, bool bCheckDependencies)
{
	return SyncPathsFromSourceControl(TEXT(""), Paths, bCheckDependencies);
}

bool AssetViewUtils::SyncPathsFromSourceControl(const TArray<FString>& Paths)
{
	return SyncPathsFromSourceControl(Paths, /*bCheckDependencies=*/true);
}

bool AssetViewUtils::SyncRevisionFromSourceControl(const FString& Revision)
{
	return SyncPathsFromSourceControl(Revision, SourceControlHelpers::GetSourceControlLocations(), /*bCheckDependencies=*/false);
}

bool AssetViewUtils::SyncLatestFromSourceControl()
{
	return SyncRevisionFromSourceControl(TEXT(""));
}

void AssetViewUtils::ShowErrorNotifcation(const FText& InErrorMsg)
{
	if (!InErrorMsg.IsEmpty())
	{
		FNotificationInfo Info(InErrorMsg);
		Info.ExpireDuration = 5.0f;

		if (TSharedPtr<SNotificationItem> InfoItem = FSlateNotificationManager::Get().AddNotification(Info))
		{
			InfoItem->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

AssetViewUtils::FOnAlwaysShowPath& AssetViewUtils::OnAlwaysShowPath()
{
	return OnAlwaysShowPathDelegate;
}

AssetViewUtils::FOnFolderPathChanged& AssetViewUtils::OnFolderPathChanged()
{
	return OnFolderPathChangedDelegate;
}

AssetViewUtils::FOnSyncStart& AssetViewUtils::OnSyncStart()
{
	return FOnSyncStartDelegate;
}

AssetViewUtils::FOnSyncFinish& AssetViewUtils::OnSyncFinish()
{
	return FOnSyncFinishDelegate;
}

#undef LOCTEXT_NAMESPACE
