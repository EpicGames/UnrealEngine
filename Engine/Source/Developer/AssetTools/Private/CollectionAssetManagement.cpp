// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionAssetManagement.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

FCollectionAssetManagement::FCollectionAssetManagement()
	: FCollectionAssetManagement(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer())
{
}

FCollectionAssetManagement::FCollectionAssetManagement(const TSharedRef<ICollectionContainer>& InCollectionContainer)
	: CollectionContainer(InCollectionContainer)
{
	// Register the notifications we need in order to keep things up-to-date
	OnCollectionRenamedHandle = CollectionContainer->OnCollectionRenamed().AddRaw(this, &FCollectionAssetManagement::HandleCollectionRenamed);
	OnCollectionDestroyedHandle = CollectionContainer->OnCollectionDestroyed().AddRaw(this, &FCollectionAssetManagement::HandleCollectionDestroyed);
	OnCollectionUpdatedHandle = CollectionContainer->OnCollectionUpdated().AddRaw(this, &FCollectionAssetManagement::HandleCollectionUpdated);
	OnAssetsAddedHandle = CollectionContainer->OnAssetsAddedToCollection().AddRaw(this, &FCollectionAssetManagement::HandleAssetsAddedToCollection);
	OnAssetsRemovedHandle = CollectionContainer->OnAssetsRemovedFromCollection().AddRaw(this, &FCollectionAssetManagement::HandleAssetsRemovedFromCollection);
}

FCollectionAssetManagement::~FCollectionAssetManagement()
{
	CollectionContainer->OnCollectionRenamed().Remove(OnCollectionRenamedHandle);
	CollectionContainer->OnCollectionDestroyed().Remove(OnCollectionDestroyedHandle);
	CollectionContainer->OnCollectionUpdated().Remove(OnCollectionUpdatedHandle);
	CollectionContainer->OnAssetsAddedToCollection().Remove(OnAssetsAddedHandle);
	CollectionContainer->OnAssetsRemovedFromCollection().Remove(OnAssetsRemovedHandle);
}

void FCollectionAssetManagement::SetCurrentAssets(const TArray<FAssetData>& CurrentAssets)
{
	CurrentAssetPaths.Empty();
	for (const FAssetData& AssetData : CurrentAssets)
	{
		CurrentAssetPaths.Add(AssetData.GetSoftObjectPath());
	}

	UpdateAssetManagementState();
}

void FCollectionAssetManagement::SetCurrentAssetPaths(const TArray<FSoftObjectPath>& CurrentAssets)
{
	CurrentAssetPaths.Empty();
	CurrentAssetPaths.Append(CurrentAssets);

	UpdateAssetManagementState();
}

void FCollectionAssetManagement::AddCurrentAssetsToCollection(FCollectionNameType InCollectionKey)
{
	const TArray<FSoftObjectPath> ObjectPaths = CurrentAssetPaths.Array();

	FText ResultText;
	bool bSuccess = false;
	{
		int32 NumAdded = 0;
		if (CollectionContainer->AddToCollection(InCollectionKey.Name, InCollectionKey.Type, ObjectPaths, &NumAdded, &ResultText))
		{
			bSuccess = true;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Number"), NumAdded);
			Args.Add(TEXT("CollectionName"), FText::FromName(InCollectionKey.Name));
			ResultText = FText::Format(LOCTEXT("CollectionAssetsAdded", "Added {Number} asset(s) to {CollectionName}"), Args);
		}
	}

	if (!ResultText.IsEmpty())
	{
		FNotificationInfo Info(ResultText);
		Info.bFireAndForget = true;
		Info.bUseLargeFont = false;

		TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
		if (Item.IsValid())
		{
			Item->SetCompletionState((bSuccess) ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		}
	}
}

void FCollectionAssetManagement::RemoveCurrentAssetsFromCollection(FCollectionNameType InCollectionKey)
{
	const TArray<FSoftObjectPath> ObjectPaths = CurrentAssetPaths.Array();

	FText ResultText;
	bool bSuccess = false;
	{
		int32 NumRemoved = 0;
		if (CollectionContainer->RemoveFromCollection(InCollectionKey.Name, InCollectionKey.Type, ObjectPaths, &NumRemoved, &ResultText))
		{
			bSuccess = true;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Number"), NumRemoved);
			Args.Add(TEXT("CollectionName"), FText::FromName(InCollectionKey.Name));
			ResultText = FText::Format(LOCTEXT("CollectionAssetsRemoved", "Removed {Number} asset(s) from {CollectionName}"), Args);
		}
	}

	if (!ResultText.IsEmpty())
	{
		FNotificationInfo Info(ResultText);
		Info.bFireAndForget = true;
		Info.bUseLargeFont = false;

		TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
		if (Item.IsValid())
		{
			Item->SetCompletionState((bSuccess) ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		}
	}
}

bool FCollectionAssetManagement::IsCollectionEnabled(FCollectionNameType InCollectionKey) const
{
	if (CollectionContainer->IsReadOnly(InCollectionKey.Type))
	{
		return false;
	}

	// Non-local collections can only be changed if we have an available source control connection
	const bool bCollectionWritable = (InCollectionKey.Type == ECollectionShareType::CST_Local) || (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable());
	return bCollectionWritable && CurrentAssetPaths.Num() > 0;
}

ECheckBoxState FCollectionAssetManagement::GetCollectionCheckState(FCollectionNameType InCollectionKey) const
{
	// If the collection exists in the map, then it means that the current selection contains at least one asset using that collection
	const ECheckBoxState* FoundCheckState = AssetManagementState.Find(InCollectionKey);
	if (FoundCheckState)
	{
		return *FoundCheckState;
	}

	// If the collection doesn't exist in the map, then it's assumed to be unused by the current selection (and thus unchecked)
	return ECheckBoxState::Unchecked;
}

int32 FCollectionAssetManagement::GetCurrentAssetCount() const
{
	return CurrentAssetPaths.Num();
}

void FCollectionAssetManagement::UpdateAssetManagementState()
{
	AssetManagementState.Empty();

	if (CurrentAssetPaths.Num() == 0)
	{
		return;
	}

	// The logic below is much simpler when only a single object is selected as we don't need to deal with set intersection
	if (CurrentAssetPaths.Num() == 1)
	{
		TArray<FCollectionNameType> MatchedCollections;
		CollectionContainer->GetCollectionsContainingObject(*CurrentAssetPaths.CreateConstIterator(), MatchedCollections, ECollectionRecursionFlags::Self);

		for (const FCollectionNameType& CollectionKey : MatchedCollections)
		{
			AssetManagementState.Add(CollectionKey, ECheckBoxState::Checked);
		}

		MatchedCollections.Reset();
		CollectionContainer->GetCollectionsContainingObject(*CurrentAssetPaths.CreateConstIterator(), MatchedCollections, ECollectionRecursionFlags::Children);

		for (const FCollectionNameType& CollectionKey : MatchedCollections)
		{
			if (!AssetManagementState.Contains(CollectionKey))
			{
				AssetManagementState.Add(CollectionKey, ECheckBoxState::Undetermined);
			}
		}
	}
	else
	{
		const TArray<FSoftObjectPath> ObjectPaths = CurrentAssetPaths.Array();
		TMap<FCollectionNameType, TArray<FSoftObjectPath>> CollectionsAndMatchedObjects;
		CollectionContainer->GetCollectionsContainingObjects(ObjectPaths, CollectionsAndMatchedObjects);

		for (const TPair<FCollectionNameType, TArray<FSoftObjectPath>>& MatchedCollection : CollectionsAndMatchedObjects)
		{
			const FCollectionNameType& CollectionKey = MatchedCollection.Key;
			const TArray<FSoftObjectPath>& MatchedObjects = MatchedCollection.Value;

			// Collections that contain all of the selected assets are shown as checked, collections that only contain some of the selected assets are shown as undetermined
			AssetManagementState.Add(
				CollectionKey, 
				(MatchedObjects.Num() == CurrentAssetPaths.Num()) ? ECheckBoxState::Checked : ECheckBoxState::Undetermined
				);
		}
	}
}

void FCollectionAssetManagement::HandleCollectionRenamed(ICollectionContainer&, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection)
{
	if (AssetManagementState.Contains(OriginalCollection))
	{
		AssetManagementState.Add(NewCollection, AssetManagementState[OriginalCollection]);
		AssetManagementState.Remove(OriginalCollection);
	}
}

void FCollectionAssetManagement::HandleCollectionUpdated(ICollectionContainer&, const FCollectionNameType& Collection)
{
	// Collection has changed in an unknown way - we need to update everything to be sure
	UpdateAssetManagementState();
}

void FCollectionAssetManagement::HandleCollectionDestroyed(ICollectionContainer&, const FCollectionNameType& Collection)
{
	AssetManagementState.Remove(Collection);
}

void FCollectionAssetManagement::HandleAssetsAddedToCollection(ICollectionContainer&, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsAdded)
{
	// Only need to update if one of the added assets belongs to our current selection set
	bool bNeedsUpdate = false;
	for (const FSoftObjectPath& AssetPath : AssetsAdded)
	{
		if (CurrentAssetPaths.Contains(AssetPath))
		{
			bNeedsUpdate = true;
			break;
		}
	}

	if (bNeedsUpdate)
	{
		UpdateAssetManagementState();
	}
}

void FCollectionAssetManagement::HandleAssetsRemovedFromCollection(ICollectionContainer&, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsRemoved)
{
	// Only need to update if one of the removed assets belongs to our current selection set
	bool bNeedsUpdate = false;
	for (const FSoftObjectPath& AssetPath : AssetsRemoved)
	{
		if (CurrentAssetPaths.Contains(AssetPath))
		{
			bNeedsUpdate = true;
			break;
		}
	}

	if (bNeedsUpdate)
	{
		UpdateAssetManagementState();
	}
}

#undef LOCTEXT_NAMESPACE
