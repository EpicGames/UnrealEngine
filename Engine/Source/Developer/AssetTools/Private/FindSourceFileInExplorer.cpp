// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindSourceFileInExplorer.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"

namespace UE::AssetTools
{
	void ExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions)
	{
		for (TArray<FString>::TConstIterator FilenameIter(Filenames); FilenameIter; ++FilenameIter)
		{
			const FString CSVFilename = FPaths::ConvertRelativePathToFull(*FilenameIter);
			const FString RootPath = FPaths::GetPath(CSVFilename);
			const FString BaseFilename = FPaths::GetBaseFilename(CSVFilename, true);
		
			for (TArray<FString>::TConstIterator ExtensionItr(OverrideExtensions); ExtensionItr; ++ExtensionItr)
			{
				const FString FilenameWithExtension(FString::Printf(TEXT("%s/%s%s"), *RootPath, *BaseFilename, **ExtensionItr));
			
				if (!FilenameWithExtension.IsEmpty() && FPaths::FileExists(*FilenameWithExtension))
				{
					FPlatformProcess::LaunchFileInDefaultExternalApplication(*FilenameWithExtension, nullptr, ELaunchVerb::Edit);
					break;
				}
			}
		}
	}

	bool CanExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions)
	{
		// Verify that extensions were provided
		if (OverrideExtensions.Num() == 0)
		{
			return false;
		}

		// Verify that the file exists with any of the given extensions
		for (TArray<FString>::TConstIterator FilenameIter(Filenames); FilenameIter; ++FilenameIter)
		{
			const FString CSVFilename = FPaths::ConvertRelativePathToFull(*FilenameIter);
			const FString RootPath = FPaths::GetPath(CSVFilename);
			const FString BaseFilename = FPaths::GetBaseFilename(CSVFilename, true);

			for (TArray<FString>::TConstIterator ExtensionItr(OverrideExtensions); ExtensionItr; ++ExtensionItr)
			{
				const FString FilenameWithExtension(FString::Printf(TEXT("%s/%s%s"), *RootPath, *BaseFilename, **ExtensionItr));

				if (!FilenameWithExtension.IsEmpty() && FPaths::FileExists(*FilenameWithExtension))
				{
					return true;
				}
			}
		}

		return false;
	}
}