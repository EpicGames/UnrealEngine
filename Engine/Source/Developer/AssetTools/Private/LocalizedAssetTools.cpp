// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizedAssetTools.h"

#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetTools.h"
#include "AssetToolsLog.h"
#include "AssetToolsModule.h"
#include "Internationalization/PackageLocalizationUtil.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "SFileListReportDialog.h"
#include "SourceControlHelpers.h"
#include "SourceControlPreferences.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "LocalizedAssetTools"

class SIncludeLocalizedVariantsDialog : public SFileListReportDialog
{
private:
	bool bAllowOperationCanceling = true;
	ELocalizedVariantsInclusion RecommendedBehavior = ELocalizedVariantsInclusion::Include;
	ELocalizedVariantsInclusion Result = ELocalizedVariantsInclusion::Include;

public:
	static ELocalizedVariantsInclusion OpenIncludeListDialog(const FText& InTitle, const FText& InHeader, const TArray<FText>& InFiles, ELocalizedVariantsInclusion RecommendedBehavior = ELocalizedVariantsInclusion::Include, bool bAllowOperationCanceling = true, ELocalizedVariantsInclusion UnattendedDefaultBehavior = ELocalizedVariantsInclusion::Exclude)
	{
		// Make sure that if bAllowOperationCanceling is false then the default behavior should never be "Cancel".
		ensureAlways(bAllowOperationCanceling || (!bAllowOperationCanceling && RecommendedBehavior != ELocalizedVariantsInclusion::Cancel));

		if (FApp::IsUnattended() || GIsRunningUnattendedScript)
		{
			return UnattendedDefaultBehavior;
		}

		TSharedRef<SIncludeLocalizedVariantsDialog> FileListReportDialogRef = SNew(SIncludeLocalizedVariantsDialog).Header(InHeader).Files(InFiles);

		FileListReportDialogRef->bAllowOperationCanceling = bAllowOperationCanceling;
		FileListReportDialogRef->RecommendedBehavior = RecommendedBehavior;
		FileListReportDialogRef->Result = RecommendedBehavior;

		FileListReportDialogRef->bOpenAsModal = true;
		FileListReportDialogRef->bAllowTitleBarX = bAllowOperationCanceling;
		FileListReportDialogRef->Title = InTitle;
		CreateWindow(FileListReportDialogRef);

		// Modal window is closed so the result should be accessible now

		return FileListReportDialogRef->Result;
	}

protected:
	FReply OnIncludeSelected()
	{
		Result = ELocalizedVariantsInclusion::Include;
		return CloseWindow();
	}

	FReply OnExcludeSelected()
	{
		Result = ELocalizedVariantsInclusion::Exclude;
		return CloseWindow();
	}

	FReply OnCancelSelected()
	{
		Result = ELocalizedVariantsInclusion::Cancel;
		return CloseWindow();
	}

	virtual void OnClosedWithTitleBarX(const TSharedRef<SWindow>& Window)
	{
		if (bAllowOperationCanceling)
		{
			Result = ELocalizedVariantsInclusion::Cancel;
		}

		// Otherwise, Result is already defined, let's use that value
	}

	virtual TSharedRef<SHorizontalBox> ConstructButtons(const FArguments& InArgs) override
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
					.OnClicked(this, &SIncludeLocalizedVariantsDialog::OnIncludeSelected)
					.Text(LOCTEXT("IncludeLocalizedVariantsDialogIncludeButtonText", "Include"))
					.ButtonStyle(FAppStyle::Get(), RecommendedBehavior == ELocalizedVariantsInclusion::Include ? "FlatButton.Primary" : "FlatButton.Default")
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 0)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
					.OnClicked(this, &SIncludeLocalizedVariantsDialog::OnExcludeSelected)
					.Text(LOCTEXT("IncludeLocalizedVariantsDialogExcludeButtonText", "Exclude"))
					.ButtonStyle(FAppStyle::Get(), RecommendedBehavior == ELocalizedVariantsInclusion::Exclude ? "FlatButton.Primary" : "FlatButton.Default")
			];

		if (bAllowOperationCanceling)
		{
			HorizontalBox->AddSlot()
				.AutoWidth()
				.Padding(4, 0)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
						.OnClicked(this, &SIncludeLocalizedVariantsDialog::OnCancelSelected)
						.Text(LOCTEXT("IncludeLocalizedVariantsDialogCancelButtonText", "Cancel"))
						.ButtonStyle(FAppStyle::Get(), RecommendedBehavior == ELocalizedVariantsInclusion::Cancel ? "FlatButton.Primary" : "FlatButton.Default")
				];
		}

		return HorizontalBox;
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Enter)
		{
			switch (RecommendedBehavior)
			{
			case ELocalizedVariantsInclusion::Include:
				return OnIncludeSelected();
			case ELocalizedVariantsInclusion::Exclude:
				return OnExcludeSelected();
			case ELocalizedVariantsInclusion::Cancel:
			default:
				return OnCancelSelected();
			}
		}
		else if(InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancelSelected();
		}

		return SFileListReportDialog::OnKeyDown(MyGeometry, InKeyEvent);
	}
};

FLocalizedAssetTools::FLocalizedAssetTools() : 
	RevisionControlIsNotAvailableWarningText(LOCTEXT("RevisionControlIsRequiredToChangeLocalizableAssets", "Revision Control is required to move/rename/delete localizable assets for this project and it is currently not accessible."))
	, FilesNeedToBeOnDiskWarningText(LOCTEXT("FilesToSyncDialogTitle", "Files in Revision Control need to be on disk"))
	
{

}

bool FLocalizedAssetTools::CanLocalize(const UClass* Class) const
{
	if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(Class))
	{
		return AssetDefinition->CanLocalize(FAssetData()).IsSupported();
	}
	else
	{
		FAssetToolsModule& Module = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		if (TSharedPtr<IAssetTypeActions> AssetActions = Module.Get().GetAssetTypeActionsForClass(Class).Pin())
		{
			return AssetActions->CanLocalize();
		}
	}

	return false;
}

ELocalizedAssetsOnDiskResult FLocalizedAssetTools::GetLocalizedVariantsOnDisk(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySource, TArray<FName>* OutPackagesNotFound /*= nullptr*/) const
{
	FScopedSlowTask GettingLocalizedVariantsOnDiskSlowTask(1.0f, LOCTEXT("GettingLocalizedVariantsOnDiskSlowTask", "Getting localized variants on disk..."));

	OutLocalizedVariantsBySource.Reserve(InPackages.Num());
	if (OutPackagesNotFound != nullptr)
	{
		OutPackagesNotFound->Reserve(InPackages.Num());
	}

	TMap<FName, FAssetData> PackagesToAssetDataMap;
	UE::AssetRegistry::GetAssetForPackages(InPackages, PackagesToAssetDataMap);
	if (PackagesToAssetDataMap.Num() != InPackages.Num()) // There used to be an ensure here but there were edge cases not covered. It will be properly fixed soon.
	{
		for (const FName& OriginalAssetName : InPackages)
		{
			OutLocalizedVariantsBySource.Add({ OriginalAssetName, TArray<FName>() });
		}
		return ELocalizedAssetsOnDiskResult::PackageNamesError;
	}

	UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get();
	float ProgressStep = 1.0f / static_cast<float>(InPackages.Num());
	for (const FName& OriginalAssetName : InPackages)
	{
		GettingLocalizedVariantsOnDiskSlowTask.EnterProgressFrame(ProgressStep);
		FString SourceAssetPathStr = OriginalAssetName.ToString();
		FPackageLocalizationUtil::ConvertLocalizedToSource(SourceAssetPathStr, SourceAssetPathStr);
		const FName SourceAssetName(*SourceAssetPathStr);

		if (OutLocalizedVariantsBySource.Contains(SourceAssetName))
		{
			continue; // We want to avoid doing any unnecessary work if it was already processed
		}

		// We want to avoid doing any unnecessary work on assets that does not require checking for variants
		FAssetData SourceAssetData = PackagesToAssetDataMap[OriginalAssetName];
		UClass* SourceAssetClass = SourceAssetData.GetClass();
		const UAssetDefinition* SourceAssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForClass(SourceAssetClass);
		bool bShouldCheckForVariant = SourceAssetDefinition != nullptr && SourceAssetDefinition->CanLocalize(SourceAssetData).IsSupported();
		if (!bShouldCheckForVariant)
		{
			OutLocalizedVariantsBySource.Add({ SourceAssetName, TArray<FName>() });
			continue;
		}

		// Check on disk for localized variants first. Remember the assets that had no variants on disk
		// because we will then check in Revision Control if applicable
		TArray<FString> LocalizedVariantsPaths;
		FPackageLocalizationUtil::GetLocalizedVariantsAbsolutePaths(SourceAssetPathStr, LocalizedVariantsPaths);
		if (LocalizedVariantsPaths.IsEmpty())
		{
			if (OutPackagesNotFound != nullptr)
			{
				OutPackagesNotFound->Add(OriginalAssetName);
			}
			continue;
		}

		// If localized variants were found on disk, let's build renaming data for them too
		TArray<FName> LocalizedAssets;
		LocalizedAssets.Reserve(LocalizedVariantsPaths.Num());
		for (const FString& LocalizedVariantPath : LocalizedVariantsPaths)
		{
			FString Culture;
			FPackageLocalizationUtil::ExtractCultureFromLocalized(LocalizedVariantPath, Culture);

			FString LocalizedAsset;
			FPackageLocalizationUtil::ConvertSourceToLocalized(SourceAssetPathStr, Culture, LocalizedAsset);

			LocalizedAssets.Add(FName(LocalizedAsset));
		}

		OutLocalizedVariantsBySource.Add({ SourceAssetName, LocalizedAssets });
	}

	return ELocalizedAssetsOnDiskResult::Success;
}

ELocalizedAssetsInSCCResult FLocalizedAssetTools::GetLocalizedVariantsInRevisionControl(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySource, TArray<FName>* OutPackagesNotFound /*= nullptr*/) const
{
	FScopedSlowTask GetLocalizedVariantsInRevisionControlSlowTask(1.0f, LOCTEXT("GetLocalizedVariantsInRevisionControlSlowTask", "Querying Revision Control for localized variants... This could take a long time."));
	GetLocalizedVariantsInRevisionControlSlowTask.EnterProgressFrame(0.05f);

	OutLocalizedVariantsBySource.Reserve(InPackages.Num());

	// Modify data to be used by USourceControlHelpers
	TArray<FString> PackagesAsString;
	PackagesAsString.Reserve(InPackages.Num());
	for (const FName& InPackageName : InPackages)
	{
		PackagesAsString.Add(InPackageName.ToString());
	}

	// Let's check the packages presence in Revision Control in a single query
	TArray<FString> LocalizedVariantsInRevisionControl;
	GetLocalizedVariantsInRevisionControlSlowTask.EnterProgressFrame(0.9f);
	bool bOutRevisionControlWasNeeded = !GetLocalizedVariantsDepotPaths(PackagesAsString, LocalizedVariantsInRevisionControl);

	// Fill a proper structure with the results
	float ProgressStep = 0.03f / static_cast<float>(LocalizedVariantsInRevisionControl.Num());
	for (const FString& LocalizedVariantInRevisionControl : LocalizedVariantsInRevisionControl)
	{
		GetLocalizedVariantsInRevisionControlSlowTask.EnterProgressFrame(ProgressStep);
		FString SourceAsset;
		FPackageLocalizationUtil::ConvertToSource(LocalizedVariantInRevisionControl, SourceAsset);
		FName SourceAssetName(SourceAsset);
		TArray<FName>* VariantsBySourceFound = OutLocalizedVariantsBySource.Find(SourceAssetName);
		TArray<FName>& VariantsBySource = (VariantsBySourceFound != nullptr ? *VariantsBySourceFound : OutLocalizedVariantsBySource.Add({ SourceAssetName, TArray<FName>() }));
		VariantsBySource.Add(FName(LocalizedVariantInRevisionControl));
	}

	// Don't forget to return the information on the packages that found nothing in Revision Control
	if (OutPackagesNotFound != nullptr)
	{
		ProgressStep = 0.02f / static_cast<float>(InPackages.Num());
		for (const FName& PackageName : InPackages)
		{
			GetLocalizedVariantsInRevisionControlSlowTask.EnterProgressFrame(ProgressStep);
			FString SourcePackage;
			FPackageLocalizationUtil::ConvertToSource(PackageName.ToString(), SourcePackage);
			if (OutLocalizedVariantsBySource.Find(FName(SourcePackage)) == nullptr)
			{
				// Package not found
				OutPackagesNotFound->Add(PackageName);
			}
		}
	}

	return bOutRevisionControlWasNeeded ? ELocalizedAssetsInSCCResult::RevisionControlNotAvailable : ELocalizedAssetsInSCCResult::Success;
}

ELocalizedAssetsResult FLocalizedAssetTools::GetLocalizedVariants(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySourceOnDisk, bool bAlsoCheckInRevisionControl, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySourceInRevisionControl, TArray<FName>* OutPackagesNotFound /*= nullptr*/) const
{
	ELocalizedAssetsResult Result = ELocalizedAssetsResult::Success;

	// Check on disk first
	// Call the LocalizedAssetTools interface to GetLocalizedVariantsOnDisk of a list of packages
	TMap<FName, TArray<FName>> VariantsBySources;
	TArray<FName> VariantsMaybeInRevisionControl;
	ELocalizedAssetsOnDiskResult DiskResult = GetLocalizedVariantsOnDisk(InPackages, OutLocalizedVariantsBySourceOnDisk, bAlsoCheckInRevisionControl ? &VariantsMaybeInRevisionControl : OutPackagesNotFound);
	Result = (DiskResult == ELocalizedAssetsOnDiskResult::PackageNamesError ? ELocalizedAssetsResult::PackageNamesError : Result);

	// Check in Revision Control if applicable
	// Call the LocalizedAssetTools interface to GetLocalizedVariantsInRevisionControl of a list of packages
	bool bRevisionControlWasNeeded = false;
	if (!VariantsMaybeInRevisionControl.IsEmpty())
	{
		if (Result == ELocalizedAssetsResult::Success)
		{
			ELocalizedAssetsInSCCResult SCCResult = GetLocalizedVariantsInRevisionControl(VariantsMaybeInRevisionControl, OutLocalizedVariantsBySourceInRevisionControl, OutPackagesNotFound);
			Result = (SCCResult == ELocalizedAssetsInSCCResult::RevisionControlNotAvailable ? ELocalizedAssetsResult::RevisionControlNotAvailable : Result);
		}
		else
		{
			OutPackagesNotFound->Append(VariantsMaybeInRevisionControl);
		}
	}

	return Result;
}

void FLocalizedAssetTools::OpenRevisionControlRequiredDialog() const
{
	FText WarningText = RevisionControlIsNotAvailableWarningText;
	const FText AvoidWarningText = LOCTEXT("HowToFixRevisionControlIsRequiredToManageLocalizableAssets", "If you want to disable this project option, it is located under:\n\tProject Settings/\n\tEditor/\n\tRevision Control/\n\tRequires Revision Control To Manage Localizable Assets\n\nThis option is there to prevent breaking paths between a source asset and its localized variants if they are not on disk.");
	FMessageDialog::Open(EAppMsgType::Ok, WarningText.Format(LOCTEXT("RevisionControlIsRequiredToManageLocalizableAssetsDialog", "{0}\n\n{1}"), WarningText, AvoidWarningText));
}

void FLocalizedAssetTools::OpenFilesInRevisionControlRequiredDialog(const TArray<FText>& FileList) const
{
	OpenLocalizedVariantsListMessageDialog(FilesNeedToBeOnDiskWarningText,
		LOCTEXT("FilesToSyncDialogHeader", "The following assets were found only in Revision Control. They need to be on your disk to be renamed."),
		FileList);
}

void FLocalizedAssetTools::OpenLocalizedVariantsListMessageDialog(const FText& Header, const FText& Message, const TArray<FText>& FileList) const
{
	SFileListReportDialog::OpenListDialog(Header, Message, FileList, true);
}

ELocalizedVariantsInclusion FLocalizedAssetTools::OpenIncludeLocalizedVariantsListDialog(const TArray<FText>& FileList, ELocalizedVariantsInclusion RecommendedBehavior /*= ELocalizedVariantsInclusion::Include*/, bool bAllowOperationCanceling /*= true*/, ELocalizedVariantsInclusion UnattendedDefaultBehavior /*= ELocalizedVariantsInclusion::Exclude*/) const
{
	return SIncludeLocalizedVariantsDialog::OpenIncludeListDialog(LOCTEXT("IncludeLocalizedVariantsDialogTitle", "Include Localized Variants"),
		LOCTEXT("IncludeLocalizedVariantsDialogHeader", "The current operation could also apply to the following localized variants (or source asset). Do you want to include them in the current operation ?"),
		FileList,
		RecommendedBehavior,
		bAllowOperationCanceling,
		UnattendedDefaultBehavior);
}

const FText& FLocalizedAssetTools::GetRevisionControlIsNotAvailableWarningText() const
{
	return RevisionControlIsNotAvailableWarningText;
}

const FText& FLocalizedAssetTools::GetFilesNeedToBeOnDiskWarningText() const
{
	return FilesNeedToBeOnDiskWarningText;
}

bool FLocalizedAssetTools::GetLocalizedVariantsDepotPaths(const TArray<FString>& InPackagesNames, TArray<FString>& OutLocalizedVariantsPaths) const
{
	// Ensure source control system is up and running with a configured provider
	ISourceControlModule& SCModule = ISourceControlModule::Get();
	if (!SCModule.IsEnabled())
	{
		return false;
	}
	ISourceControlProvider& Provider = SCModule.GetProvider();
	if (!Provider.IsAvailable())
	{
		return false;
	}

	// Other providers don't work for now
	if (Provider.GetName() == "Perforce")
	{
		TArray<FString> LocalizedVariantsRegexPaths;
		LocalizedVariantsRegexPaths.Reserve(InPackagesNames.Num());
		for (const FString& InPackageName : InPackagesNames)
		{
			FString SourcePackageName;
			FPackageLocalizationUtil::ConvertToSource(InPackageName, SourcePackageName);
			FString LocalizedVariantsRegexPath;
			FPackageLocalizationUtil::ConvertSourceToRegexLocalized(SourcePackageName, LocalizedVariantsRegexPath);
			LocalizedVariantsRegexPath += FPackageName::GetAssetPackageExtension();
			LocalizedVariantsRegexPaths.Add(LocalizedVariantsRegexPath);
		}

		bool bSilent = true;
		bool bIncludeDeleted = true;
		USourceControlHelpers::GetFilesInDepotAtPaths(LocalizedVariantsRegexPaths, OutLocalizedVariantsPaths, bIncludeDeleted, bSilent, true);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

