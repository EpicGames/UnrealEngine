// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ILocalizedAssetTools.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

class FLocalizedAssetTools : public ILocalizedAssetTools
{
public:
	FLocalizedAssetTools();
	virtual ~FLocalizedAssetTools() = default;

	// ILocalizedAssetTools implementations

	virtual bool CanLocalize(const UClass* Class) const override;

	virtual ELocalizedAssetsOnDiskResult GetLocalizedVariantsOnDisk(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySource, TArray<FName>* OutPackagesNotFound = nullptr) const override;

	virtual ELocalizedAssetsInSCCResult GetLocalizedVariantsInRevisionControl(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySource, TArray<FName>* OutPackagesNotFound = nullptr) const override;

	virtual ELocalizedAssetsResult GetLocalizedVariants(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySourceOnDisk, bool bAlsoCheckInRevisionControl, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySourceInRevisionControl, TArray<FName>* OutPackagesNotFound = nullptr) const override;

	virtual void OpenRevisionControlRequiredDialog() const override;

	virtual void OpenFilesInRevisionControlRequiredDialog(const TArray<FText>& FileList) const override;

	virtual void OpenLocalizedVariantsListMessageDialog(const FText& Header, const FText& Message, const TArray<FText>& FileList) const override;

	virtual ELocalizedVariantsInclusion OpenIncludeLocalizedVariantsListDialog( const TArray<FText>& FileList, ELocalizedVariantsInclusion RecommendedBehavior = ELocalizedVariantsInclusion::Include, bool bAllowOperationCanceling = true, ELocalizedVariantsInclusion UnattendedDefaultBehavior = ELocalizedVariantsInclusion::Exclude) const override;

	virtual const FText& GetRevisionControlIsNotAvailableWarningText() const override;

	virtual const FText& GetFilesNeedToBeOnDiskWarningText() const override;

private:
	bool GetLocalizedVariantsDepotPaths(const TArray<FString>& InPackagesNames, TArray<FString>& OutLocalizedVariantsPaths) const;

private:
	FText RevisionControlIsNotAvailableWarningText;
	FText FilesNeedToBeOnDiskWarningText;
};
