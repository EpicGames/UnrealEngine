// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageMigrationContext.h"

#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"

namespace UE::AssetTools
{

namespace Private::Migration
{
	static FString MigrationDir(TEXT("Migration"));
}

FPackageMigrationContext::EPackageMigrationStep FPackageMigrationContext::GetCurrentStep() const
{
	return CurrentStep;
}

void FPackageMigrationContext::AddSucessfullMigrationMessage(const FText& InMessage)
{
	MigratedPackageMessages.Add(InMessage);
}

void FPackageMigrationContext::AddWarningMigrationMessage(const FText& InMessage)
{
	WarningMessage.Add(InMessage);
}

void FPackageMigrationContext::AddErrorMigrationMessage(const FText& InMessage)
{
	ErrorMessages.Add(InMessage);
}

const FPackageMigrationContext::FScopedMountPoint& FPackageMigrationContext::GetDestinationMountPoint() const
{
	return DestinationMountPoint;
}

const TArray<FPackageMigrationContext::FMigrationPackageData>& FPackageMigrationContext::GetMigrationPackagesData() const
{
	return MigrationPackagesData;
}

const TArray<UPackage*>& FPackageMigrationContext::GetMovedOutOfTheWayPackages() const
{
	return ObjectPtrDecay(PackagesThatWhereMoved);
}

const TArray<FString>& FPackageMigrationContext::GetExcludedDependencies() const
{
	checkf(CurrentStep < EPackageMigrationStep::PostExcludedDependenciesCreated, TEXT("The migration can't give the excluded packages before the where processed"));

	return ExcludedDependencies;
}

void FPackageMigrationContext::MoveInTheWayPackage(UPackage* Package)
{
	check(Package);

	checkf(CurrentStep < EPackageMigrationStep::InTheWayPackagesMoved, TEXT("The migration should not move existing pacakges after the broadcast of the in the way pacakge moved"));

	TemporalyMovedPackages.Emplace(Package);
	PackagesThatWhereMoved.Add(Package);
}

FPackageMigrationContext::FPackageMigrationContext(FScopedMountPoint&& InMountPoint)
	: DestinationMountPoint(MoveTemp(InMountPoint))
{
}

void FPackageMigrationContext::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(PackagesThatWhereMoved);

	for (FScopedTemporalyMovedPackage& MovedPackage : TemporalyMovedPackages)
	{
		Collector.AddReferencedObject(MovedPackage.PackageToMove);
	}

	for (FMigrationPackageData& PacakgeData : MigrationPackagesData)
	{
		Collector.AddReferencedObject(PacakgeData.InstancedPackage);
	}
}

FString FPackageMigrationContext::GetReferencerName() const
{
	return TEXT("FPackageMigrationContext");
}

const FString& FPackageMigrationContext::FScopedMountPoint::GetRootPath() const
{
	return RootPath;
}

FString FPackageMigrationContext::FScopedMountPoint::GetMigratedPackageFilename(const FString& LongPackageName, const FStringView& InExtension) const
{
	FString PackageRoot;
	FString PackagePath;
	FString PackageName;

	FPackageName::SplitLongPackageName(LongPackageName, PackageRoot, PackagePath, PackageName);

	FString PackageFilename;
	PackageFilename.Reserve(ContentPath.Len() + PackagePath.Len() + PackageName.Len() + InExtension.Len());
	PackageFilename.Append(ContentPath);
	PackageFilename.Append(PackagePath);
	PackageFilename.Append(PackageName);
	PackageFilename.Append(InExtension);

	return PackageFilename;
}

FString FPackageMigrationContext::FScopedMountPoint::GetNewPackageNameForMigration(const FString& LongPackageName) const
{
	const bool InWithoutSlashes = false;
	FName CurrentPackageMountPoint = FPackageName::GetPackageMountPoint(LongPackageName, InWithoutSlashes);

	FStringView CurrentPackageWithoutMountPoint = LongPackageName;
	CurrentPackageWithoutMountPoint.RightChopInline(CurrentPackageMountPoint.GetStringLength());


	FString NewPackageName;
	NewPackageName.Reserve(GetRootPath().Len() + CurrentPackageWithoutMountPoint.Len());
	NewPackageName.Append(GetRootPath());
	NewPackageName.Append(CurrentPackageWithoutMountPoint);

	return NewPackageName;
}

FPackageMigrationContext::FScopedMountPoint::FScopedMountPoint(FString&& InRootPath, FString&& InContentPath)
	: RootPath(MoveTemp(InRootPath))
	, ContentPath(MoveTemp(InContentPath))
{
	if (!FPackageName::MountPointExists(RootPath))
	{
		bHasMountedANewMountPoint = true;
		// Use a temp folder to avoid letting the asset registry scan the destination
		FPackageName::RegisterMountPoint(RootPath, FPaths::Combine(FPaths::ProjectIntermediateDir(), UE::AssetTools::Private::Migration::MigrationDir));
	}
}

FPackageMigrationContext::FScopedMountPoint::~FScopedMountPoint()
{
	// Empty root path means that we didn't add or own a new mount point
	if (bHasMountedANewMountPoint && !RootPath.IsEmpty() && !ContentPath.IsEmpty())
	{
		FPackageName::UnRegisterMountPoint(RootPath, FPaths::Combine(FPaths::ProjectIntermediateDir(), UE::AssetTools::Private::Migration::MigrationDir));
	}
}

FPackageMigrationContext::FScopedTemporalyMovedPackage::FScopedTemporalyMovedPackage(UPackage* InPackageToMove)
	: PackageToMove(InPackageToMove)
	, OriginalName(PackageToMove->GetName())
{
	const ERenameFlags PkgRenameFlags = REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_SkipGeneratedClasses | REN_AllowPackageLinkerMismatch;
	verify(PackageToMove->Rename(*MakeUniqueObjectName(nullptr, UPackage::StaticClass(), *FString::Printf(TEXT("%s_MOVEDFORMIGRATION"), *PackageToMove->GetName())).ToString(), nullptr, PkgRenameFlags));
}

FPackageMigrationContext::FScopedTemporalyMovedPackage::~FScopedTemporalyMovedPackage()
{
	const ERenameFlags PkgRenameFlags = REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_SkipGeneratedClasses | REN_AllowPackageLinkerMismatch;
	verify(PackageToMove->Rename(*OriginalName, nullptr, PkgRenameFlags));
}

const FString& FPackageMigrationContext::FMigrationPackageData::GetInstancedPackageName() const
{
	return InstancedPackageName;
}

const FString& FPackageMigrationContext::FMigrationPackageData::GetOriginalPackageName() const
{
	return OriginalPackageName;
}

const FString& FPackageMigrationContext::FMigrationPackageData::GetDestinationFilename() const
{
	return DestinationFilename;
}

UPackage* FPackageMigrationContext::FMigrationPackageData::GetInstancedPackage() const
{
	return InstancedPackage;
}

FPackageMigrationContext::FMigrationPackageData::FMigrationPackageData(const FString& InInstancedPackageName, const FString& InOriginalPackageName, const FString& InDestinationFilename)
	: InstancedPackageName(InInstancedPackageName)
	, OriginalPackageName(InOriginalPackageName)
	, DestinationFilename(InDestinationFilename)
{
}

}
