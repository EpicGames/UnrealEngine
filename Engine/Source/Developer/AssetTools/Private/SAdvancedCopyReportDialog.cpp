// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAdvancedCopyReportDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SCheckBox.h"
#include "AssetToolsModule.h"
#include "AssetToolsSettings.h"
#include "Misc/PackageName.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "AdvancedCopyReportDialog"

static const FName AssetColumnLabel = TEXT("Asset");
static const FName SourceColumnLabel = TEXT("Source");
static const FName RelativeDestinationColumnLabel = TEXT("RelativeDestination");

struct FCompareFAdvancedCopyReportNodeByName
{
	FORCEINLINE bool operator()( TSharedPtr<FAdvancedCopyReportNode> A, TSharedPtr<FAdvancedCopyReportNode> B ) const
	{
		return A->Source < B->Source;
	}
};

FAdvancedCopyReportNode::FAdvancedCopyReportNode()
{
}

FAdvancedCopyReportNode::FAdvancedCopyReportNode(const FString& InSource, const FString& InDestination, TSharedPtr<TSet<FString>> InIncludedSet)
	: Source(InSource)
	, Destination(InDestination)
	, IncludedSet(InIncludedSet)
{
}

void FAdvancedCopyReportNode::AddPackage(const FString& InSource, const FString& InDestination, const FString& DependencyOf)
{
	AddPackage_Recursive(InSource, InDestination, DependencyOf);
}

void FAdvancedCopyReportNode::ExpandChildrenRecursively(const TSharedRef<SAdvancedCopyReportTree>& TreeView)
{
	ForAllDescendants([TreeView](const TSharedPtr<FAdvancedCopyReportNode>& Node) {
		TreeView->SetItemExpansion(Node, false);
	});
}

void FAdvancedCopyReportNode::ForAllDescendants(TFunctionRef<void(const TSharedPtr<FAdvancedCopyReportNode>& /*Node*/)> RecursiveAction)
{
	for (auto ChildIt = Children.CreateIterator(); ChildIt; ++ChildIt)
	{
		RecursiveAction(*ChildIt);
		(*ChildIt)->ForAllDescendants(RecursiveAction);
	}
}

bool FAdvancedCopyReportNode::GetWillCopy() const
{
	if (IncludedSet.IsValid())
	{
		return IncludedSet->Contains(Source);
	}

	return false;
}

void FAdvancedCopyReportNode::SetWillCopy(bool bCopy)
{
	if (bCopy)
	{
		IncludedSet->Add(Source);
	}
	else
	{
		IncludedSet->Remove(Source);
	}
}

bool FAdvancedCopyReportNode::AddPackage_Recursive(const FString& InSource, const FString& InDestination, const FString& DependencyOf)
{
	TSharedPtr<FAdvancedCopyReportNode> Child;
	// If this is not a dependency of an asset, add it to the top of the tree
	if (DependencyOf.IsEmpty())
	{
		int32 ChildIdx = Children.Add(MakeShareable(new FAdvancedCopyReportNode(InSource, InDestination, IncludedSet)));
		Child = Children[ChildIdx];
		Children.Sort(FCompareFAdvancedCopyReportNodeByName());
		return true;
	}
	else if (Source == DependencyOf)
	{
		for (auto ChildIt = Children.CreateConstIterator(); ChildIt; ++ChildIt)
		{
			if ((*ChildIt)->Source == InSource)
			{
				Child = (*ChildIt);
				break;
			}
		}

		// If one was not found, create it
		if (!Child.IsValid())
		{
			int32 ChildIdx = Children.Add(MakeShareable(new FAdvancedCopyReportNode(InSource, InDestination, IncludedSet)));
			Child = Children[ChildIdx];
			Children.Sort(FCompareFAdvancedCopyReportNodeByName());
			return true;
		}
	}
	else
	{
		bool bFoundDependencyInChild = false;
		for (auto ChildIt = Children.CreateConstIterator(); ChildIt; ++ChildIt)
		{
			bFoundDependencyInChild = ChildIt->Get()->AddPackage_Recursive(InSource, InDestination, DependencyOf);
			if (bFoundDependencyInChild)
			{
				return true;
			}
		}
		return false;
	}
	return false;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAdvancedCopyReportDialog::Construct( const FArguments& InArgs, const FAdvancedCopyParams& InParams, const FString& InDestination, const TArray<TMap<FString, FString>>& InDestinationMap, const TArray<TMap<FName, FName>>& DependencyMap, const SAdvancedCopyReportDialog::FOnReportConfirmed& InOnReportConfirmed )
{
	OnReportConfirmed = InOnReportConfirmed;
	InitialDestinationMap = InDestinationMap;
	CloneSet = MakeShared<TSet<FString>>();
	PackageReportRootNode.IncludedSet = CloneSet;
	CurrentCopyParams = InParams;
	Destination = InDestination;
	ConstructNodeTree(InDestinationMap, DependencyMap);

	TSharedRef< SHeaderRow > HeaderRowWidget = SNew(SHeaderRow);

	TSharedPtr<SAdvancedCopyColumn> PackageColumn = MakeShareable(new SAdvancedCopyColumn(AssetColumnLabel));
	Columns.Add(PackageColumn->GetColumnID(), PackageColumn);
	HeaderRowWidget->AddColumn(PackageColumn->ConstructHeaderRowColumn());

	TSharedPtr<SAdvancedCopyColumn> SourceColumn = MakeShareable(new SAdvancedCopyColumn(SourceColumnLabel));
	Columns.Add(SourceColumn->GetColumnID(), SourceColumn);
	HeaderRowWidget->AddColumn(SourceColumn->ConstructHeaderRowColumn());

	TSharedPtr<SAdvancedCopyColumn> DestinationColumn = MakeShareable(new SAdvancedCopyColumn(RelativeDestinationColumnLabel));
	Columns.Add(DestinationColumn->GetColumnID(), DestinationColumn);
	HeaderRowWidget->AddColumn(DestinationColumn->ConstructHeaderRowColumn());


	ChildSlot
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush") )
		.Padding(FMargin(4, 8, 4, 4))
		[
			SNew(SVerticalBox)

			// Report Message
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(STextBlock)
				.Text(this, &SAdvancedCopyReportDialog::GetHeaderText, FText::FromString(IAssetTools::Get().GetUserFacingLongPackagePath(Destination)))
				.TextStyle( FAppStyle::Get(), "PackageMigration.DialogTitle" )
				.AutoWrapText(true)
			]

			// Tree of packages in the report
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				[
					SAssignNew( ReportTreeView, SAdvancedCopyReportTree )
					.HeaderRow(HeaderRowWidget)
					.TreeItemsSource(&PackageReportRootNode.Children)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow( this, &SAdvancedCopyReportDialog::GenerateTreeRow )
					.OnGetChildren( this, &SAdvancedCopyReportDialog::GetChildrenForTree )
					.OnSetExpansionRecursive(this, &SAdvancedCopyReportDialog::SetItemExpansionRecursive)
				]
			]

			// Options
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("GenerateDependenciesToCopyTooltip", "Toggle whether or not to search for dependencies. Toggling this will rebuild the destination list."))
				.Type(ESlateCheckBoxType::CheckBox)
				.IsChecked(this, &SAdvancedCopyReportDialog::IsGeneratingDependencies)
				.OnCheckStateChanged(this, &SAdvancedCopyReportDialog::ToggleGeneratingDependencies)
				.Padding(4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GenerateDependenciesToCopy", "Generate Dependencies to Copy"))
				]
			]

			// Find...
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(SEditableTextBox)
				.HintText(LOCTEXT("FindHintText", "Find..."))
				.Text_Lambda([this]() { return FText::FromString(FindString); })
				.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
				{
					FindString = NewText.ToString();
				})
			]

			// Replace...
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(SEditableTextBox)
				.HintText(LOCTEXT("ReplaceHintText", "Replace..."))
				.Text_Lambda([this]() { return FText::FromString(ReplaceString); })
				.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
				{
					ReplaceString = NewText.ToString();
				})
			]

			// Ok/Cancel buttons
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0,4,0,0)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
					.OnClicked(this, &SAdvancedCopyReportDialog::OkClicked)
					.Text(LOCTEXT("OkButton", "OK"))
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
					.OnClicked(this, &SAdvancedCopyReportDialog::CancelClicked)
					.Text(LOCTEXT("CancelButton", "Cancel"))
				]
			]
		]
	];

	// Make sure the initially selected packages begin as part of the set we're definitely cloning.
	IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();

	for (FName OriginalName : InParams.GetSelectedPackageOrFolderNames())
	{
		const FString& OriginalNameString = OriginalName.ToString();
		if (!FPackageName::DoesPackageExist(OriginalNameString))
		{
			TArray<FAssetData> AssetsInFolder;
			AssetRegistry.GetAssetsByPath(OriginalName, AssetsInFolder, true, false);
			for (const FAssetData& Asset : AssetsInFolder)
			{
				CloneSet->Add(Asset.PackageName.ToString());
			}
		}
		else
		{
			CloneSet->Add(OriginalName.ToString());
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SAdvancedCopyReportDialog::GetHeaderText(const FText InReportMessage) const
{
	if (PackageReportRootNode.Children.Num() == 0)
	{
		return LOCTEXT("NoValidSources", "You have not selected any valid sources for advanced copying.");
	}
	return FText::Format(LOCTEXT("AdvancedCopyDesc", "The following files will be copied to {0} and references to copied files will be fixed up."), InReportMessage);
}

ECheckBoxState SAdvancedCopyReportDialog::IsGeneratingDependencies() const
{
	return CurrentCopyParams.bShouldCheckForDependencies ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SAdvancedCopyReportDialog::ToggleGeneratingDependencies(ECheckBoxState NewState)
{
	CurrentCopyParams.bShouldCheckForDependencies = NewState == ECheckBoxState::Checked;
	CloseDialog();
	IAssetTools& AssetTools = IAssetTools::Get();
	AssetTools.InitAdvancedCopyFromCopyParams(CurrentCopyParams);
}

void SAdvancedCopyReportDialog::OpenPackageReportDialog(const FAdvancedCopyParams& InParams, const FString& Destination, const TArray<TMap<FString, FString>>& InDestinationMap, const TArray<TMap<FName, FName>>& DependencyMap, const SAdvancedCopyReportDialog::FOnReportConfirmed& InOnReportConfirmed)
{
	TSharedRef<SWindow> ReportWindow = SNew(SWindow)
		.Title(LOCTEXT("AdvancedCopyReportWindowTitle", "Advanced Copy Asset Report"))
		.ClientSize( FVector2D(800, 600) )
		.SupportsMaximize(true)
		.SupportsMinimize(true)
		[
			SNew(SAdvancedCopyReportDialog, InParams, Destination, InDestinationMap, DependencyMap, InOnReportConfirmed)
		];

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	if ( MainFrameModule.GetParentWindow().IsValid() )
	{
		FSlateApplication::Get().AddWindowAsNativeChild(ReportWindow, MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(ReportWindow);
	}
}

void SAdvancedCopyReportDialog::CloseDialog()
{
	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if ( Window.IsValid() )
	{
		Window->RequestDestroyWindow();
	}
}

TSharedRef<ITableRow> SAdvancedCopyReportDialog::GenerateTreeRow( TSharedPtr<FAdvancedCopyReportNode> TreeItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SAdvancedCopyTreeRow, ReportTreeView.ToSharedRef(), SharedThis(this)).Item(TreeItem);
}

void SAdvancedCopyReportDialog::GetChildrenForTree( TSharedPtr<FAdvancedCopyReportNode> TreeItem, TArray< TSharedPtr<FAdvancedCopyReportNode> >& OutChildren )
{
	OutChildren = TreeItem->Children;
}

void SAdvancedCopyReportDialog::ConstructNodeTree(const TArray<TMap<FString, FString>>& DestinationMap, const TArray<TMap<FName, FName>>& DependencyMap)
{
	for (int32 MapIndex = 0; MapIndex < DestinationMap.Num(); MapIndex++)
	{
		TMap<FString, FString> SingleDestinationMap = DestinationMap[MapIndex];
		TMap<FName, FName> SingleDependencyMap = DependencyMap[MapIndex];
		for (auto PackageIt = SingleDestinationMap.CreateConstIterator(); PackageIt; ++PackageIt)
		{
			FString SourceString = *PackageIt.Key();
			const FName* DependencyOfPtr = SingleDependencyMap.Find(FName(*SourceString));
			FString DependencyOf = FString();
			if (DependencyOfPtr)
			{
				DependencyOf = DependencyOfPtr->ToString();
			}
			PackageReportRootNode.AddPackage(SourceString, *PackageIt.Value(), DependencyOf);
		}
	}
}

void SAdvancedCopyReportDialog::SetItemExpansionRecursive(TSharedPtr<FAdvancedCopyReportNode> TreeItem, bool bInExpansionState)
{
	if (TreeItem.IsValid())
	{
		ReportTreeView->SetItemExpansion(TreeItem, bInExpansionState);

		for (TSharedPtr<FAdvancedCopyReportNode>& ChildModel : TreeItem->Children)
		{
			SetItemExpansionRecursive(ChildModel, bInExpansionState);
		}
	}
}

FReply SAdvancedCopyReportDialog::OkClicked()
{
	CloseDialog();

	TArray<TMap<FString, FString>> FilteredDestinationMap = InitialDestinationMap;
	for (auto& DestinationsMap : FilteredDestinationMap)
	{
		for ( TMap< FString, FString >::TIterator It = DestinationsMap.CreateIterator(); It; ++It )
		{
			if (!CloneSet->Contains(It.Key()))
			{
				It.RemoveCurrent();
			}
			else
			{
				if (!FindString.IsEmpty())
				{
					FString OutPackageRoot, OutPackagePath, OutPackageName;
					FPackageName::SplitLongPackageName(It.Value(), OutPackageRoot, OutPackagePath, OutPackageName);

					FString CroppedDestination = It.Value();
					const bool bDidRemovePrefix = CroppedDestination.RemoveFromStart(Destination);
					if (bDidRemovePrefix)
					{
						It.Value() = Destination / CroppedDestination.Replace(*FindString, *ReplaceString);
					}
				}
			}
		}
	}

	FilteredDestinationMap.Shrink();

	OnReportConfirmed.ExecuteIfBound(CurrentCopyParams, FilteredDestinationMap);

	return FReply::Handled();
}

FReply SAdvancedCopyReportDialog::CancelClicked()
{
	CloseDialog();

	return FReply::Handled();
}

void SAdvancedCopyTreeRow::Construct(const FArguments& InArgs, const TSharedRef<SAdvancedCopyReportTree>& OutlinerTreeView, TSharedRef<SAdvancedCopyReportDialog> AdvancedCopyReport)
{
	Item = InArgs._Item;
	ReportDialogWeak = AdvancedCopyReport;
	auto Args = FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

	SMultiColumnTableRow<TSharedPtr<FAdvancedCopyReportNode>>::Construct(Args, OutlinerTreeView);
}

TSharedRef<SWidget> SAdvancedCopyTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	auto ItemPtr = Item.Pin();
	if (!ItemPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	// Create the widget for this item
	TSharedRef<SWidget> NewItemWidget = SNullWidget::NullWidget;

	auto Column = ReportDialogWeak.Pin()->GetColumns().FindRef(ColumnName);
	if (Column.IsValid())
	{
		NewItemWidget = Column->ConstructRowWidget(ItemPtr.ToSharedRef(), *this);
	}

	if (ColumnName == AssetColumnLabel)
	{
		// The first column gets the tree expansion arrow for this row
		return
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			[
				SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SAdvancedCopyTreeRow::GetWillCopyCheckedState)
				.OnCheckStateChanged(this, &SAdvancedCopyTreeRow::ApplyWillCopyCheckedState)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				NewItemWidget
			];
	}
	else
	{
		// Other columns just get widget content -- no expansion arrow needed
		return NewItemWidget;
	}
}

ECheckBoxState SAdvancedCopyTreeRow::GetWillCopyCheckedState() const
{
	auto ItemPtr = Item.Pin();
	if (ItemPtr.IsValid())
	{
		return ItemPtr->GetWillCopy() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}

void SAdvancedCopyTreeRow::ApplyWillCopyCheckedState(const ECheckBoxState NewCheckState)
{
	TSharedPtr<FAdvancedCopyReportNode> ItemPtr = Item.Pin();
	if (ItemPtr.IsValid())
	{
		const bool bWillCopy = NewCheckState == ECheckBoxState::Checked ? true : false;
		ItemPtr->SetWillCopy(bWillCopy);

		const bool bRecursive = FSlateApplication::Get().GetModifierKeys().IsShiftDown() ? true : false;
		if (bRecursive)
		{
			ItemPtr->ForAllDescendants([bWillCopy](const TSharedPtr<FAdvancedCopyReportNode>& Node) {
				Node->SetWillCopy(bWillCopy);
			});
		}
	}
}

SHeaderRow::FColumn::FArguments SAdvancedCopyColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FillWidth(2.f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FName::NameToDisplayString(ColumnName.ToString(), false)))
		];
}

const TSharedRef< SWidget > SAdvancedCopyColumn::ConstructRowWidget(TSharedPtr<struct FAdvancedCopyReportNode> TreeItem, const SAdvancedCopyTreeRow& Row)
{
	if (ColumnName == AssetColumnLabel)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FPaths::GetBaseFilename(TreeItem.Get()->Source)));
	}
	else if (ColumnName == SourceColumnLabel)
	{
		FString Source;
		if (FPackageName::DoesPackageExist(TreeItem.Get()->Source))
		{
			Source = IAssetTools::Get().GetUserFacingLongPackageName(FName(TreeItem.Get()->Source));
		}
		else
		{
			Source = IAssetTools::Get().GetUserFacingLongPackagePath(TreeItem.Get()->Source);
		}
		return SNew(STextBlock)
			.Text(FText::FromString(MoveTemp(Source)));
	}
	else if (ColumnName == RelativeDestinationColumnLabel)
	{
		FString CroppedDestination = TreeItem.Get()->Destination;
		const bool bDidRemovePrefix = CroppedDestination.RemoveFromStart(Row.GetReportDialog()->GetDestination());
		if (bDidRemovePrefix)
		{
			CroppedDestination = TEXT(".") + CroppedDestination;

		}
		return SNew(STextBlock)
			.Text(FText::FromString(FPaths::GetPath(CroppedDestination)));
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE