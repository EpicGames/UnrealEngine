// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "IAssetTools.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Containers/Array.h"

struct FAdvancedCopyReportNode;
class SAdvancedCopyTreeRow;

typedef STreeView< TSharedPtr<struct FAdvancedCopyReportNode> > SAdvancedCopyReportTree;

class SAdvancedCopyColumn : public TSharedFromThis< SAdvancedCopyColumn >
{
public:
	SAdvancedCopyColumn(FName InColumnName)
	{
		ColumnName = InColumnName;
	}
	virtual ~SAdvancedCopyColumn() = default;

	virtual FName GetColumnID() { return ColumnName; };

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn();

	virtual const TSharedRef< SWidget > ConstructRowWidget(TSharedPtr<struct FAdvancedCopyReportNode> TreeItem, const SAdvancedCopyTreeRow& Row);

	FName ColumnName;
};

struct FAdvancedCopyReportNode
{
	/** The name of the tree node without the path */
	FString Source;
	/** The name of the tree node without the path */
	FString Destination;
	/** The children of this node */
	TArray< TSharedPtr<FAdvancedCopyReportNode> > Children;
	/** The included set */
	TSharedPtr<TSet<FString>> IncludedSet;

	/** Constructor */
	FAdvancedCopyReportNode();
	FAdvancedCopyReportNode(const FString& InSource, const FString& InDestination, TSharedPtr<TSet<FString>> InIncludedSet);

	/** Adds the path to the tree relative to this node, creating nodes as needed. */
	void AddPackage(const FString& Source, const FString& Destination, const FString& DependencyOf);

	/** Expands this node and all its children */
	void ExpandChildrenRecursively(const TSharedRef<SAdvancedCopyReportTree>& TreeView);

	/** Perform recursive action on all descendants. */
	void ForAllDescendants(TFunctionRef<void(const TSharedPtr<FAdvancedCopyReportNode>& /*Node*/)> RecursiveAction);

	bool GetWillCopy() const;
	void SetWillCopy(bool bCopy);

private:
	bool AddPackage_Recursive(const FString& Source, const FString& Destination, const FString& DependencyOf);
};

class SAdvancedCopyReportDialog : public SCompoundWidget
{
public:
	typedef TArray<TMap<FString, FString>> FCopyDestinationMap;
	DECLARE_DELEGATE_TwoParams(FOnReportConfirmed, const FAdvancedCopyParams&, const FCopyDestinationMap&)

	SLATE_BEGIN_ARGS(SAdvancedCopyReportDialog){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const FAdvancedCopyParams& InParams, const FString& InDestination, const TArray<TMap<FString, FString>>& DestinationMap, const TArray<TMap<FName, FName>>& DependencyMap, const FOnReportConfirmed& InOnReportConfirmed );
		
	ECheckBoxState IsGeneratingDependencies() const;

	void ToggleGeneratingDependencies(ECheckBoxState NewState);

	/** Opens the dialog in a new window */
	static void OpenPackageReportDialog(const FAdvancedCopyParams& InParams, const FString& Destination, const TArray<TMap<FString, FString>>& DestinationMap, const TArray<TMap<FName, FName>>& DependencyMap, const FOnReportConfirmed& InOnReportConfirmed);

	/** Closes the dialog. */
	void CloseDialog();

	TMap<FName, TSharedPtr<SAdvancedCopyColumn>> GetColumns() const { return Columns; }

	const FString& GetDestination() const { return Destination; }

private:
	/** Constructs the node tree given the list of package names */
	void ConstructNodeTree(const TArray<TMap<FString, FString>>& DestinationMap, const TArray<TMap<FName, FName>>& DependencyMap);

	/** Handler to generate a row in the report tree */
	TSharedRef<ITableRow> GenerateTreeRow( TSharedPtr<FAdvancedCopyReportNode> TreeItem, const TSharedRef<STableViewBase>& OwnerTable );

	/** Gets the children for the specified tree item */
	void GetChildrenForTree( TSharedPtr<FAdvancedCopyReportNode> TreeItem, TArray< TSharedPtr<FAdvancedCopyReportNode> >& OutChildren );

	void SetItemExpansionRecursive(TSharedPtr<FAdvancedCopyReportNode> TreeItem, bool bInExpansionState);

	/** Handler for when "Ok" is clicked */
	FReply OkClicked();

	/** Handler for when "Cancel" is clicked */
	FReply CancelClicked();

	FText GetHeaderText(const FText InReportMessage) const;

private:
	FOnReportConfirmed OnReportConfirmed;
	FAdvancedCopyReportNode PackageReportRootNode;
	TSharedPtr<SAdvancedCopyReportTree> ReportTreeView;
	/** The set of things to actually clone. */
	TSharedPtr<TSet<FString>> CloneSet;
	/** Map of columns that are shown on this report. */
	TMap<FName, TSharedPtr<SAdvancedCopyColumn>> Columns;
	FString Destination;
	FAdvancedCopyParams CurrentCopyParams;
	TArray<TMap<FString, FString>> InitialDestinationMap;
	FString FindString;
	FString ReplaceString;
};


/** Widget that represents a row in the outliner's tree control.  Generates widgets for each column on demand. */
class SAdvancedCopyTreeRow
	: public SMultiColumnTableRow< TSharedPtr<struct FAdvancedCopyReportNode> >
{

public:

	SLATE_BEGIN_ARGS(SAdvancedCopyTreeRow) {}

	/** The list item for this row */
	SLATE_ARGUMENT(TSharedPtr<struct FAdvancedCopyReportNode>, Item)

	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<SAdvancedCopyReportTree>& OutlinerTreeView, TSharedRef<SAdvancedCopyReportDialog> AdvancedCopyReport);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	TSharedPtr<SAdvancedCopyReportDialog> GetReportDialog() const
	{
		return ReportDialogWeak.IsValid() ? ReportDialogWeak.Pin() : nullptr;
	}

private:
	ECheckBoxState GetWillCopyCheckedState() const;
	void ApplyWillCopyCheckedState(const ECheckBoxState NewCheckState);

	/** Weak reference to the outliner widget that owns our list */
	TWeakPtr< SAdvancedCopyReportDialog > ReportDialogWeak;

	/** The item associated with this row of data */
	TWeakPtr<struct FAdvancedCopyReportNode> Item;
};
