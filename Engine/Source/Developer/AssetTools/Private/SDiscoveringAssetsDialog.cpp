// Copyright Epic Games, Inc. All Rights Reserved.


#include "SDiscoveringAssetsDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorWidgetsModule.h"
#include "Interfaces/IMainFrameModule.h"

#define LOCTEXT_NAMESPACE "DiscoveringAssetsDialog"

SDiscoveringAssetsDialog::~SDiscoveringAssetsDialog()
{
	if ( FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")) )
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDiscoveringAssetsDialog::Construct( const FArguments& InArgs )
{
	OnAssetsDiscovered = InArgs._OnAssetsDiscovered;
	
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
		.Padding(FMargin(4, 8, 4, 4))
		[
			SNew(SVerticalBox)

			// "Discovering Assets" UI
			+SVerticalBox::Slot()
			.Padding(16, 0)
			.FillHeight(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 8)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DiscoveringAssets", "Please wait while assets are being discovered."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				[
					EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_None, FMargin(0), false)
				]
			]

			// Cancel button
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.OnClicked(this, &SDiscoveringAssetsDialog::CancelClicked)
				.Text(LOCTEXT("CancelButton", "Cancel"))
			]
		]
	];

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if ( AssetRegistryModule.Get().IsLoadingAssets() )
	{
		AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SDiscoveringAssetsDialog::AssetRegistryLoadComplete);
	}
	else
	{
		OnAssetsDiscovered.ExecuteIfBound();
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDiscoveringAssetsDialog::OpenDiscoveringAssetsDialog(const FOnAssetsDiscovered& InOnAssetsDiscovered)
{
	TSharedRef<SWindow> RenameWindow = SNew(SWindow)
		.Title(LOCTEXT("DiscoveringAssetsDialog", "Discovering Assets..."))
		.SizingRule( ESizingRule::Autosized )
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SDiscoveringAssetsDialog)
			.OnAssetsDiscovered(InOnAssetsDiscovered)
		];

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));

	if ( MainFrameModule.GetParentWindow().IsValid() )
	{
		FSlateApplication::Get().AddWindowAsNativeChild(RenameWindow, MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(RenameWindow);
	}
}

FReply SDiscoveringAssetsDialog::CancelClicked()
{
	CloseDialog();

	return FReply::Handled();
}

void SDiscoveringAssetsDialog::AssetRegistryLoadComplete()
{
	OnAssetsDiscovered.ExecuteIfBound();

	CloseDialog();
}

void SDiscoveringAssetsDialog::CloseDialog()
{
	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if ( Window.IsValid() )
	{
		Window->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
