// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SDiscoveringAssetsDialog : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnAssetsDiscovered)

	SLATE_BEGIN_ARGS( SDiscoveringAssetsDialog ){}

		SLATE_EVENT( FOnAssetsDiscovered, OnAssetsDiscovered)

	SLATE_END_ARGS()

	/** Destructor */
	~SDiscoveringAssetsDialog();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Opens the dialog in a new window */
	static void OpenDiscoveringAssetsDialog(const FOnAssetsDiscovered& InOnAssetsDiscovered);

private:
	/** Handler for when "Cancel" is clicked */
	FReply CancelClicked();

	/** Called when the asset registry initial load has completed */
	void AssetRegistryLoadComplete();

	/** Closes the dialog. */
	void CloseDialog();

private:
	FOnAssetsDiscovered OnAssetsDiscovered;
};
