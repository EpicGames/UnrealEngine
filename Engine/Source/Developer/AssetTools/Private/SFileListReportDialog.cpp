// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFileListReportDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/App.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "FileListReportDialog"

void SFileListReportDialog::Construct(const FArguments& InArgs)
{
	for (const FText& File : InArgs._Files)
	{
		Files.Add(MakeShareable(new FText(File)));
	}

	ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
				.Padding(FMargin(4, 8, 4, 4))
				[
					SNew(SVerticalBox)

						// Title text
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock).Text(InArgs._Header)
								.AutoWrapText(true)
						]

						// Files To Sync list
						+ SVerticalBox::Slot()
						.Padding(0, 8)
						.FillHeight(1.f)
						[
							SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								[
									SNew(SListView<TSharedRef<FText>>)
										.ListItemsSource(&Files)
										.SelectionMode(ESelectionMode::None)
										.OnGenerateRow(this, &SFileListReportDialog::MakeListViewWidget)
								]
						]

						// Buttons
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0, 4)
						.HAlign(HAlign_Right)
						[
							ConstructButtons(InArgs)
						]
				]
		];
}

void SFileListReportDialog::OpenListDialog(const FText& InTitle, const FText& InHeader, const TArray<FText>& InFiles, bool bOpenAsModal /*= false*/)
{
	if (FApp::IsUnattended() || GIsRunningUnattendedScript)
	{
		return;
	}

	TSharedRef<SFileListReportDialog> FileListReportDialogRef = SNew(SFileListReportDialog).Header(InHeader).Files(InFiles);
	FileListReportDialogRef->bOpenAsModal = bOpenAsModal;
	FileListReportDialogRef->bAllowTitleBarX = true;
	FileListReportDialogRef->Title = InTitle;
	CreateWindow(FileListReportDialogRef);
}

TSharedRef<ITableRow> SFileListReportDialog::MakeListViewWidget(TSharedRef<FText> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(STableRow< TSharedRef<FText> >, OwnerTable)
		[
			SNew(STextBlock).Text(Item.Get())
		];
}

TSharedRef<SHorizontalBox> SFileListReportDialog::ConstructButtons(const FArguments& InArgs)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
				.OnClicked(this, &SFileListReportDialog::CloseWindow)
				.Text(LOCTEXT("WindowCloseButton", "Close"))
		];
}

FReply SFileListReportDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter || InKeyEvent.GetKey() == EKeys::Escape)
	{
		CloseWindow();
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

bool SFileListReportDialog::SupportsKeyboardFocus() const
{
	return true;
}

void SFileListReportDialog::CreateWindow(TSharedRef<SFileListReportDialog> InFileListReportDialogRef)
{
	TSharedRef<SWindow> FileListReportWindow = SNew(SWindow)
		.Title(InFileListReportDialogRef->Title)
		.ClientSize(FVector2D(800, 400))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.HasCloseButton(InFileListReportDialogRef->bAllowTitleBarX)
		[
			InFileListReportDialogRef
		];

	FileListReportWindow->GetOnWindowActivatedEvent().AddLambda([InFileListReportDialogRef]()
		{
			FSlateApplication::Get().SetKeyboardFocus(InFileListReportDialogRef);
		});

	FileListReportWindow->SetOnWindowClosed(FOnWindowClosed::CreateRaw(&InFileListReportDialogRef.Get(), &SFileListReportDialog::OnWindowClosed));

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));

	if (MainFrameModule.GetParentWindow().IsValid())
	{
		if (InFileListReportDialogRef->bOpenAsModal)
		{
			FSlateApplication::Get().AddModalWindow(FileListReportWindow, MainFrameModule.GetParentWindow().ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindowAsNativeChild(FileListReportWindow, MainFrameModule.GetParentWindow().ToSharedRef());
		}
	}
	else
	{
		if (InFileListReportDialogRef->bOpenAsModal)
		{
			FSlateApplication::Get().AddModalWindow(FileListReportWindow, nullptr);
		}
		else
		{
			FSlateApplication::Get().AddWindow(FileListReportWindow);
		};
	}
}

FReply SFileListReportDialog::CloseWindow()
{
	bClosingWithoutTitleBarX = true;

	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (Window.IsValid())
	{
		Window->RequestDestroyWindow();
	}

	return FReply::Handled();
}

void SFileListReportDialog::OnWindowClosed(const TSharedRef<SWindow>& Window)
{
	if (!bClosingWithoutTitleBarX)
	{
		OnClosedWithTitleBarX(Window);
	}
}

void SFileListReportDialog::SetModal(bool bInIsModal)
{
	bOpenAsModal = bInIsModal;
}

void SFileListReportDialog::SetAllowTitleBarX(bool bInAllowTitleBarX)
{
	bAllowTitleBarX = bInAllowTitleBarX;
}

#undef LOCTEXT_NAMESPACE
