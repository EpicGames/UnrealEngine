// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SHorizontalBox;
class STableViewBase;

class SFileListReportDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFileListReportDialog) {}

		SLATE_ARGUMENT(FText, Header)
		SLATE_ARGUMENT(TArray<FText>, Files)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	static void OpenListDialog(const FText& InTitle, const FText& InHeader, const TArray<FText>& InFiles, bool bOpenAsModal = false);

protected:
	TSharedRef<ITableRow> MakeListViewWidget(TSharedRef<FText> Item, const TSharedRef<STableViewBase>& OwnerTable);
	virtual TSharedRef<SHorizontalBox> ConstructButtons(const FArguments& InArgs);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override;

	static void CreateWindow(TSharedRef<SFileListReportDialog> InFileListReportDialogRef);

	FReply CloseWindow();
	void OnWindowClosed(const TSharedRef<SWindow>& Window);
	virtual void OnClosedWithTitleBarX(const TSharedRef<SWindow>& Window) {}

	void SetModal(bool bIsModal);
	void SetAllowTitleBarX(bool bInAllowTitleBarX);

protected:
	bool bOpenAsModal = false;
	bool bAllowTitleBarX = true;
	FText Title;

private:
	FText Header;
	TArray< TSharedRef<FText> > Files;

	bool bClosingWithoutTitleBarX = false;
};
