// Copyright Epic Games, Inc. All Rights Reserved.


#include "SPackageReportDialog.h"
#include "IAssetTools.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/IPluginManager.h"

#define LOCTEXT_NAMESPACE "PackageReportDialog"

FPackageReportNode::FPackageReportNode()
	: CheckedState(ECheckBoxState::Undetermined)
	, bShouldMigratePackage(nullptr)
	, Parent(nullptr)
{}

FPackageReportNode::FPackageReportNode(FString&& InNodeName, FText&& InNodeText, FText&& InToolTipText, bool* bInShouldMigratePackage, FPackageReportNode& Parent)
	: NodeName(MoveTemp(InNodeName))
	, NodeText(MoveTemp(InNodeText))
	, ToolTipText(MoveTemp(InToolTipText))
	, CheckedState(bInShouldMigratePackage != nullptr ? (*bInShouldMigratePackage ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined)
	, bShouldMigratePackage(bInShouldMigratePackage)
	, Parent(&Parent)
{
}

void FPackageReportNode::AddPackage(const FString& PackageName, bool* bInShouldMigratePackage)
{
	TArray<FString> PathElements;
	PackageName.ParseIntoArray(PathElements, TEXT("/"), /*InCullEmpty=*/true);

	if (PathElements.Num() < 2)
	{
		return;
	}

	const IAssetTools& AssetTools = IAssetTools::Get();

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PathElements[0]);
	bool bFirstElementIsMountPoint = true;
	if (Plugin.IsValid() && !Plugin->GetVersePath().IsEmpty() && AssetTools.ShowingContentVersePath())
	{
		TArray<FString> VersePathElements;
		Plugin->GetVersePath().ParseIntoArray(VersePathElements, TEXT("/"), /*InCullEmpty=*/true);

		if (ensure(!VersePathElements.IsEmpty()))
		{
			bFirstElementIsMountPoint = false;

			// Replace the mount point with the Verse path.
			PathElements.RemoveAt(0);
			PathElements.Insert(VersePathElements, 0);
		}
	}

	FPackageReportNode* ChildParent = this;
	for (int32 Index = 0; Index < PathElements.Num(); ++Index)
	{
		TSharedPtr<FPackageReportNode>* Child = ChildParent->Children.FindByPredicate([&PathElements, Index](const TSharedPtr<FPackageReportNode>& Child)
		{
			return Child->NodeName == PathElements[Index];
		});
		if (Child == nullptr)
		{
			FString ChildNodeName = MoveTemp(PathElements[Index]);

			FText ChildNodeText;
			FText ChildToolTipText;
			bool* bChildShouldMigratePackage = nullptr;
			if (bFirstElementIsMountPoint && Index == 0 && Plugin.IsValid())
			{
				ChildNodeText = FText::FromString(Plugin->GetFriendlyName());
				ChildToolTipText = FText::FromString(ChildNodeName);
			}
			else
			{
				ChildNodeText = FText::FromString(ChildNodeName);

				if (Index + 1 == PathElements.Num())
				{
					ChildToolTipText = FText::FromString(AssetTools.GetUserFacingLongPackageName(FName(PackageName)));
					bChildShouldMigratePackage = bInShouldMigratePackage;
				}
			}

			Child = &ChildParent->Children.Add_GetRef(MakeShared<FPackageReportNode>(MoveTemp(ChildNodeName), MoveTemp(ChildNodeText), MoveTemp(ChildToolTipText), bChildShouldMigratePackage, *ChildParent));
		}
		ChildParent = Child->Get();
	}
}

void FPackageReportNode::ExpandChildrenRecursively(const TSharedRef<PackageReportTree>& Treeview)
{
	for ( auto ChildIt = Children.CreateConstIterator(); ChildIt; ++ChildIt )
	{
		Treeview->SetItemExpansion(*ChildIt, (*ChildIt)->CheckedState != ECheckBoxState::Unchecked);
		(*ChildIt)->ExpandChildrenRecursively(Treeview);
	}
}

void FPackageReportNode::Finalize()
{
	for (const TSharedPtr<FPackageReportNode>& Child : Children)
	{
		Child->Finalize();
	}

	Children.Sort([](const TSharedPtr<FPackageReportNode>& A, const TSharedPtr<FPackageReportNode>& B)
	{
		return A->NodeText.CompareTo(B->NodeText) < 0;
	});

	UpdateCheckedStateFromChildren();
}

void FPackageReportNode::UpdateCheckedStateFromChildren()
{
	if (bShouldMigratePackage == nullptr)
	{
		int32 NumChecked = 0;
		for (const TSharedPtr<FPackageReportNode>& Child : Children)
		{
			switch (Child->CheckedState)
			{
			case ECheckBoxState::Checked:
				++NumChecked;
				break;
			case ECheckBoxState::Undetermined:
				CheckedState = ECheckBoxState::Undetermined;
				return;
			}
		}

		if (NumChecked == Children.Num())
		{
			CheckedState = ECheckBoxState::Checked;
		}
		else if (NumChecked == 0)
		{
			CheckedState = ECheckBoxState::Unchecked;
		}
		else
		{
			CheckedState = ECheckBoxState::Undetermined;
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SPackageReportDialog::Construct( const FArguments& InArgs, const FText& InReportMessage, TArray<ReportPackageData>& InPackageNames, const FOnReportConfirmed& InOnReportConfirmed )
{
	OnReportConfirmed = InOnReportConfirmed;
	FolderOpenBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderOpen");
	FolderClosedBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderClosed");
	PackageBrush = FAppStyle::GetBrush("ContentBrowser.ColumnViewAssetIcon");

	ConstructNodeTree(InPackageNames);
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush") )
		.Padding(FMargin(4, 8, 4, 4))
		[
			SNew(SVerticalBox)

			// Report Message
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(STextBlock)
				.Text(InReportMessage)
				.TextStyle( FAppStyle::Get(), "PackageMigration.DialogTitle" )
			]

			// Tree of packages in the report
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				[
					SAssignNew( ReportTreeView, PackageReportTree )
					.TreeItemsSource(&PackageReportRootNode.Children)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow( this, &SPackageReportDialog::GenerateTreeRow )
					.OnGetChildren( this, &SPackageReportDialog::GetChildrenForTree )
				]
			]

			// Ok/Cancel buttons
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0,4,0,0)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
					.OnClicked(this, &SPackageReportDialog::OkClicked)
					.Text(LOCTEXT("OkButton", "OK"))
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
					.OnClicked(this, &SPackageReportDialog::CancelClicked)
					.Text(LOCTEXT("CancelButton", "Cancel"))
				]
			]
		]
	];

	if ( ensure(ReportTreeView.IsValid()) )
	{
		PackageReportRootNode.ExpandChildrenRecursively(ReportTreeView.ToSharedRef());
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SPackageReportDialog::OpenPackageReportDialog(const FText& ReportMessage, TArray<ReportPackageData>& PackageNames, const FOnReportConfirmed& InOnReportConfirmed)
{
	TSharedRef<SWindow> ReportWindow = SNew(SWindow)
		.Title(LOCTEXT("ReportWindowTitle", "Asset Report"))
		.ClientSize( FVector2D(600, 500) )
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SPackageReportDialog, ReportMessage, PackageNames, InOnReportConfirmed)
		];
		
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	if ( MainFrameModule.GetParentWindow().IsValid() )
	{
		FSlateApplication::Get().AddWindowAsNativeChild(ReportWindow, MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(ReportWindow);
	}
}

void SPackageReportDialog::CloseDialog()
{
	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if ( Window.IsValid() )
	{
		Window->RequestDestroyWindow();
	}
}

TSharedRef<ITableRow> SPackageReportDialog::GenerateTreeRow( TSharedPtr<FPackageReportNode> TreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(TreeItem.IsValid());

	return SNew( STableRow< TSharedPtr<FPackageReportNode> >, OwnerTable )
		[
			// Icon
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SPackageReportDialog::CheckBoxStateChanged, TreeItem, OwnerTable)
				.IsChecked(this, &SPackageReportDialog::GetEnabledCheckState, TreeItem)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage).Image(this, &SPackageReportDialog::GetNodeIcon, TreeItem)
			]
			// Name
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(TreeItem->NodeText)
				.ToolTipText(TreeItem->ToolTipText)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
}

ECheckBoxState SPackageReportDialog::GetEnabledCheckState(TSharedPtr<FPackageReportNode> TreeItem) const
{
	return TreeItem.Get()->CheckedState;
}

void SPackageReportDialog::SetStateRecursive(TSharedPtr<FPackageReportNode> TreeItem, bool bIsChecked)
{
	if (TreeItem.Get() == nullptr)
	{
		return;
	}

	TreeItem.Get()->CheckedState = bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

	if (TreeItem.Get()->bShouldMigratePackage)
	{
		*(TreeItem.Get()->bShouldMigratePackage) = bIsChecked;
	}

	TArray< TSharedPtr<FPackageReportNode> > Children;
	GetChildrenForTree(TreeItem, Children);
	for (int i = 0; i < Children.Num(); i++)
	{
		if (Children[i].Get() == nullptr)
		{
			continue;
		}

		SetStateRecursive(Children[i], bIsChecked);
	}
}

void SPackageReportDialog::CheckBoxStateChanged(ECheckBoxState InCheckBoxState, TSharedPtr<FPackageReportNode> TreeItem, TSharedRef<STableViewBase> OwnerTable)
{
	SetStateRecursive(TreeItem, InCheckBoxState == ECheckBoxState::Checked);

	FPackageReportNode* CurrentParent = TreeItem->Parent;
	while (CurrentParent != nullptr)
	{
		CurrentParent->UpdateCheckedStateFromChildren();
		CurrentParent = CurrentParent->Parent;
	}

	OwnerTable.Get().RebuildList();
}

void SPackageReportDialog::GetChildrenForTree( TSharedPtr<FPackageReportNode> TreeItem, TArray< TSharedPtr<FPackageReportNode> >& OutChildren )
{
	OutChildren = TreeItem->Children;
}

void SPackageReportDialog::ConstructNodeTree(TArray<ReportPackageData>& PackageNames)
{
	for (ReportPackageData& Package : PackageNames)
	{
		PackageReportRootNode.AddPackage(Package.Name, &Package.bShouldMigratePackage);
	}
	PackageReportRootNode.Finalize();
}

const FSlateBrush* SPackageReportDialog::GetNodeIcon(TSharedPtr<FPackageReportNode> ReportNode) const
{
	if ( ReportNode->bShouldMigratePackage != nullptr )
	{
		return PackageBrush;
	}
	else if ( ReportTreeView->IsItemExpanded(ReportNode) )
	{
		return FolderOpenBrush;
	}
	else
	{
		return FolderClosedBrush;
	}
}

FReply SPackageReportDialog::OkClicked()
{
	CloseDialog();
	OnReportConfirmed.ExecuteIfBound();

	return FReply::Handled();
}

FReply SPackageReportDialog::CancelClicked()
{
	CloseDialog();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
