// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

struct FPackageReportNode;

typedef STreeView< TSharedPtr<struct FPackageReportNode> > PackageReportTree;

struct ReportPackageData
{
	FString Name;
	bool bShouldMigratePackage;
};

struct FPackageReportNode
{
	/** The name of the tree node without the path */
	FString NodeName;
	/** The display name of the tree node */
	FText NodeText;
	/** The tooltip of the tree node */
	FText ToolTipText;
	/** A user-exposed flag determining whether the content of this node and its children should be migrated or not. */
	ECheckBoxState CheckedState;
	/** A pointer to an external bool describing whether this node ultimately should be migrated or not. Is only non-null for leaf nodes.*/
	bool* bShouldMigratePackage;
	
	/** The parent of this node */
	FPackageReportNode* Parent;

	/** The children of this node */
	TArray< TSharedPtr<FPackageReportNode> > Children;

	/** Constructor */
	FPackageReportNode();
	FPackageReportNode(FString&& InNodeName, FText&& InNodeText, FText&& InToolTipText, bool* bInShouldMigratePackage, FPackageReportNode& Parent);

	/** Adds the path to the tree relative to this node, creating nodes as needed. */
	void AddPackage(const FString& PackageName, bool* bInIsPackageIncluded);

	/** Expands this node and all its children */
	void ExpandChildrenRecursively(const TSharedRef<PackageReportTree>& Treeview);

	/** Call after all nodes have been created to order the child nodes for display and calculate their initial CheckedState. */
	void Finalize();

	/** Updates CheckedState from this node's child CheckedStates if bShouldMigratePackage is null */
	void UpdateCheckedStateFromChildren();
};

class SPackageReportDialog : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnReportConfirmed)

	SLATE_BEGIN_ARGS( SPackageReportDialog ){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const FText& InReportMessage, TArray<ReportPackageData>& InPackageNames, const FOnReportConfirmed& InOnReportConfirmed );

	/** Opens the dialog in a new window */
	static void OpenPackageReportDialog(const FText& ReportMessage, TArray<ReportPackageData>& PackageNames, const FOnReportConfirmed& InOnReportConfirmed);

	/** Closes the dialog. */
	void CloseDialog();

private:
	/** Recursively sets the checked/active state of every child of this node in the tree when a checkbox is toggled. */
	void SetStateRecursive(TSharedPtr<FPackageReportNode> TreeItem, bool bIsChecked);

	/** Callback to check whether a checkbox is checked or not. */
	ECheckBoxState GetEnabledCheckState(TSharedPtr<FPackageReportNode> TreeItem) const;

	/** Callback called whenever a checkbox is toggled. */
	void CheckBoxStateChanged(ECheckBoxState InCheckBoxState, TSharedPtr<FPackageReportNode> TreeItem, TSharedRef<STableViewBase> OwnerTable);

	/** Constructs the node tree given the list of package names */
	void ConstructNodeTree(TArray<ReportPackageData>& PackageNames);

	/** Handler to generate a row in the report tree */
	TSharedRef<ITableRow> GenerateTreeRow( TSharedPtr<FPackageReportNode> TreeItem, const TSharedRef<STableViewBase>& OwnerTable );

	/** Gets the children for the specified tree item */
	void GetChildrenForTree( TSharedPtr<FPackageReportNode> TreeItem, TArray< TSharedPtr<FPackageReportNode> >& OutChildren );

	/** Determines which image to display next to a node */
	const FSlateBrush* GetNodeIcon(TSharedPtr<FPackageReportNode> ReportNode) const;

	/** Handler for when "Ok" is clicked */
	FReply OkClicked();

	/** Handler for when "Cancel" is clicked */
	FReply CancelClicked();

private:
	FOnReportConfirmed OnReportConfirmed;
	FPackageReportNode PackageReportRootNode;
	TSharedPtr<PackageReportTree> ReportTreeView;

	/** Brushes for the different node states */
	const FSlateBrush* FolderOpenBrush;
	const FSlateBrush* FolderClosedBrush;
	const FSlateBrush* PackageBrush;
};
