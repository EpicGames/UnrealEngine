// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AssetRegistry/ARFilter.h"
#include "IAssetTools.h"
#include "AdvancedCopyCustomization.generated.h"

#define UE_API ASSETTOOLS_API



UCLASS(MinimalAPI)
class UAdvancedCopyCustomization : public UObject
{
	GENERATED_BODY()

public:
	UE_API UAdvancedCopyCustomization(const class FObjectInitializer& ObjectInitializer);

	/* Getter for whether or not we should generate relative paths for this advanced copy */
	bool GetShouldGenerateRelativePaths() const
	{
		return bShouldGenerateRelativePaths;
	}

	/* Returns the ARFilter for this advanced copy */
	virtual FARFilter GetARFilter() const
	{
		return FilterForExcludingDependencies;
	}

	/* Allows the customization to edit the parameters for the whole copy operation */
	virtual void EditCopyParams(FAdvancedCopyParams& CopyParams) const {};

	/* Apply any additional filtering after the ARFilter is run on the packages to copy */
	virtual void ApplyAdditionalFiltering(TArray<FName>& PackagesToCopy) const {};

	/* Once the destination map is generated for the set of assets, the destinations can be manipulated for renaming, restructuring, etc. */
	virtual void TransformDestinationPaths(TMap<FString, FString>& OutPackagesAndDestinations) const {};

	/* Allows for additional validation of the packages to be copied and their destination. Returns false if anything doesn't pass validation */
	virtual bool CustomCopyValidate(const TMap<FString, FString>& OutPackagesAndDestinations) const { return true; };

	/* Store the path of the package that caused this customization to be used */
	UE_API void SetPackageThatInitiatedCopy(const FString& InBasePackage);

	const FString GetPackageThatInitiatedCopy() const { return PackageThatInitiatedCopy; }

protected:
	/* Whether or not the destinations for copy should be relative to the package that initiated the copy */
	bool bShouldGenerateRelativePaths;

	/* The filter to use when finding valid dependencies to also copy */
	FARFilter FilterForExcludingDependencies;

	/* The path of the package that caused this customization to be used */
	FString PackageThatInitiatedCopy;
};

#undef UE_API
