// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IAssetTools.h"

class IAssetTools;

class FAssetToolsModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	virtual IAssetTools& Get() const;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAssetToolsModule& GetModule()
	{
		static const FName ModuleName = "AssetTools";
		return FModuleManager::LoadModuleChecked< FAssetToolsModule >(ModuleName);
	}

	static inline bool IsModuleLoaded()
	{
		static const FName ModuleName = "AssetTools";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

private:
	TWeakObjectPtr<class UAssetToolsImpl> AssetToolsPtr;
	class FAssetToolsConsoleCommands* ConsoleCommands = nullptr;
};
