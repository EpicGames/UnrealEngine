// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "AdvancedCopyCustomization.h"
#include "Engine/AssetUserData.h"
#include "Engine/EngineTypes.h"
#include "AssetToolsSettings.generated.h"

USTRUCT()
struct FAdvancedCopyMap
{
	GENERATED_BODY()

public:
	/** When copying this class, use a particular set of dependency and destination rules */
	UPROPERTY(EditAnywhere, Category = "Asset Tools", meta = (MetaClass = "/Script/CoreUObject.Object"))
	FSoftClassPath ClassToCopy;

	/** The set of dependency and destination rules to use for advanced copy */
	UPROPERTY(EditAnywhere, Category = "Asset Tools", meta = (MetaClass = "/Script/AssetTools.AdvancedCopyCustomization"))
	FSoftClassPath AdvancedCopyCustomization;
};

UCLASS(MinimalAPI, config = Game, defaultconfig, meta = (DisplayName = "Asset Tools"))
class UAssetToolsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAssetToolsSettings() {};

	/** List of rules to use when advanced copying assets */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Copy", Meta = (TitleProperty = "ClassToCopy"))
	TArray<FAdvancedCopyMap> AdvancedCopyCustomizations;
};

USTRUCT()
struct FAssetNamingConventionEntry
{
	GENERATED_BODY()

	/**
	 * An optional prefix to include when naming an asset
	 */
	UPROPERTY(EditAnywhere, Category = "Asset Naming")
	TOptional<FString> Prefix;

	/* Supports {source} when used in the context of creating assets from existing uobjects instead of purely from factory
	 * For instance, to override "NewBlueprint" with "BP_", "BP_{source}_" to preserve existing source object names,
	 * or "BP_" to always reset the link
	 * 
	 * Note - if using a prefix or suffix, you must also set this (todo this is terrible)
	 */
	UPROPERTY(EditAnywhere, Category = "Asset Naming")
	TOptional<FString> AssetName;

	/**
	 * An optional suffix to include when naming an asset
	 */
	UPROPERTY(EditAnywhere, Category = "Asset Naming")
	TOptional<FString> Suffix;
};

UCLASS(config = EditorSettings, defaultconfig, meta = (DisplayName = "Asset Tools"))
class UAssetToolsEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAssetToolsEditorSettings() {};

	/**
	 * Override default name for new assets, by asset type.
	 * Use of an override will override any factory setting
	 */
	UPROPERTY(EditAnywhere, config, Category = "Asset Naming")
	TMap<TSoftClassPtr<UObject>, FAssetNamingConventionEntry> AssetNamingConventionOverrides;
};
