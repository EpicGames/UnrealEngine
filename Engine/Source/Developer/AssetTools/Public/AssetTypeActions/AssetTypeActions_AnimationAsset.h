// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions_Base.h"
#include "Animation/AnimationAsset.h"
#include "EditorAnimUtils.h"
#include "SSkeletonWidget.h"

#define UE_API ASSETTOOLS_API

class
// UE_DEPRECATED(5.2, "The AssetDefinition system is replacing AssetTypeActions and UAssetDefinition_AnimationAsset replaced this.  Please see the Conversion Guide in AssetDefinition.h")
FAssetTypeActions_AnimationAsset : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimationAsset", "AnimationAsset"); }
	virtual FColor GetTypeColor() const override { return FColor(80,123,72); }
	virtual UClass* GetSupportedClass() const override { return UAnimationAsset::StaticClass(); }
	UE_API virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override;
	virtual bool CanFilter() override { return false; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	UE_API virtual class UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override;

private:

	/** Handler for when FindSkeleton is selected */
	UE_API void ExecuteFindSkeleton(TArray<TWeakObjectPtr<UAnimationAsset>> Objects);

	/** Handler for when ReplaceSkeleton is selected */
	UE_API void ExecuteReplaceSkeleton(TArray<TWeakObjectPtr<UAnimationAsset>> Objects);

	/** Handle menu item for wanting to open asset in a new editor */
	UE_API void ExecuteOpenInNewWindow(TArray<TWeakObjectPtr<UAnimationAsset>> Objects);

	/** Open animation asset, will find existing editor if desired. */
	UE_API void OpenAnimAssetEditor(const TArray<UObject*>& InObjects, bool bForceNewEditor, TSharedPtr<IToolkitHost> EditWithinLevelEditor);
	
	/** Replace skeleton when USkeleton is missing. Returns true only if Skeleton was replaced. */
	UE_API bool ReplaceMissingSkeleton(TArray<TObjectPtr<UObject>> InAnimationAssets) const;
};

#undef UE_API
