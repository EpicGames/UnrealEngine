// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions/AssetTypeActions_ClassTypeBase.h"

#define UE_API ASSETTOOLS_API

struct FAssetData;
class IClassTypeActions;
class UFactory;

class
// UE_DEPRECATED(5.2, "The AssetDefinition system is replacing AssetTypeActions and UAssetDefinition_Blueprint replaced this.  Please see the Conversion Guide in AssetDefinition.h")
FAssetTypeActions_Blueprint : public FAssetTypeActions_ClassTypeBase
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Blueprint", "Blueprint Class"); }
	virtual FColor GetTypeColor() const override { return FColor( 63, 126, 255 ); }
	virtual UClass* GetSupportedClass() const override { return UBlueprint::StaticClass(); }
	
	// AssetDefinition - The Blueprint actions are now in the UAssetDefinition_Blueprint,
	// can't use false here, since FAssetTypeActions_Blueprint is inherited.  Need the others to function until they can
	// move.
	virtual bool ShouldCallGetActions() const override { return GetSupportedClass() != UBlueprint::StaticClass(); }

	UE_API virtual void GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section) override;
	UE_API virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	UE_API virtual bool CanMerge() const override;
	UE_API virtual void Merge(UObject* InObject) override;
	UE_API virtual void Merge(UObject* BaseAsset, UObject* RemoteAsset, UObject* LocalAsset, const FOnMergeResolved& ResolutionCallback) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Blueprint | EAssetTypeCategories::Basic; }
	UE_API virtual void PerformAssetDiff(UObject* Asset1, UObject* Asset2, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override;
	UE_API virtual class UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override;
	UE_API virtual FText GetAssetDescription(const FAssetData& AssetData) const override;

	// FAssetTypeActions_ClassTypeBase Implementation
	UE_API virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const override;

protected:
	/** Whether or not this asset can create derived blueprints */
	UE_API virtual bool CanCreateNewDerivedBlueprint() const;

	/** Return the factory responsible for creating this type of Blueprint */
	UE_API virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const;

	/** Returns the tooltip to display when attempting to derive a Blueprint */
	UE_API FText GetNewDerivedBlueprintTooltip(TWeakObjectPtr<UBlueprint> InObject);

	/** Returns TRUE if you can derive a Blueprint */
	UE_API bool CanExecuteNewDerivedBlueprint(TWeakObjectPtr<UBlueprint> InObject);

private:
	/** Handler for when EditDefaults is selected */
	UE_API void ExecuteEditDefaults(TArray<TWeakObjectPtr<UBlueprint>> Objects);

	/** Handler for when NewDerivedBlueprint is selected */
	UE_API void ExecuteNewDerivedBlueprint(TWeakObjectPtr<UBlueprint> InObject);

	/** Returns true if the blueprint is data only */
	UE_API bool ShouldUseDataOnlyEditor( const UBlueprint* Blueprint ) const;
};

#undef UE_API
