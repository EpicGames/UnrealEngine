// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions/AssetTypeActions_ClassTypeBase.h"

#define UE_API ASSETTOOLS_API

class UBlueprintGeneratedClass;
struct FAssetData;
class IClassTypeActions;
class UFactory;

class
// UE_DEPRECATED(5.2, "The AssetDefinition system is replacing AssetTypeActions and UAssetDefinition_BlueprintGeneratedClass replaced this.  Please see the Conversion Guide in AssetDefinition.h")
FAssetTypeActions_BlueprintGeneratedClass : public FAssetTypeActions_ClassTypeBase
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_BlueprintGeneratedClass", "Compiled Blueprint Class"); }
	virtual FColor GetTypeColor() const override { return FColor(133, 173, 255); }
	UE_API virtual UClass* GetSupportedClass() const override;

	// We don't want BPGCs to show up in CB/Ref viewer filters since the BP filters already include them
	virtual uint32 GetCategories() override { return EAssetTypeCategories::None; }
	// @todo
	//virtual class UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override;
	// End IAssetTypeActions Implementation

	// FAssetTypeActions_ClassTypeBase Implementation
	UE_API virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const override;
	// End FAssetTypeActions_ClassTypeBase Implementation
};

#undef UE_API
