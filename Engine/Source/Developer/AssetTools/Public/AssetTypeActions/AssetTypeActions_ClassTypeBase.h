// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "AssetTypeActions_Base.h"

struct FAssetData;
class IClassTypeActions;

/** Base class for "class type" assets (C++ classes and Blueprints */
class
// UE_DEPRECATED(5.2, "The AssetDefinition system is replacing AssetTypeActions and this was replaced by AssetDefinition_ClassTypeBase.  Please see the Conversion Guide in AssetDefinition.h")
FAssetTypeActions_ClassTypeBase : public FAssetTypeActions_Base
{
public:
	virtual bool CanLocalize() const override { return false; }

	/** Get the class type actions for this asset */
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const = 0;
};
