// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"
#include "Engine/DataAsset.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API ASSETTOOLS_API

class
UE_DEPRECATED(5.8, "The AssetDefinition system is replacing AssetTypeActions and UAssetDefinition_DataAsset replaced this.  Please see the Conversion Guide in AssetDefinition.h")
FAssetTypeActions_DataAsset : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DataAsset", "Data Asset"); }
	virtual FColor GetTypeColor() const override { return FColor(201, 29, 85); }
	virtual UClass* GetSupportedClass() const override { return UDataAsset::StaticClass(); }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
	UE_API virtual FText GetDisplayNameFromAssetData(const FAssetData& AssetData) const override;
	UE_API virtual void PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision) const override;
};

#undef UE_API
