// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "Widgets/SWidget.h"
#include "Toolkits/IToolkitHost.h"
#include "Modules/ModuleManager.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "AssetTypeActivationOpenedMethod.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "AssetRegistry/ARFilter.h"

struct FAssetData;
struct FARFilter;
class FMenuBuilder;
struct FToolMenuSection;

/** A base class for all AssetTypeActions. Provides helper functions useful for many types. Deriving from this class is optional. */
class FAssetTypeActions_Base : public IAssetTypeActions
{
public:

	// IAssetTypeActions interface
	virtual FString GetObjectDisplayName(UObject* Object) const override
	{
		return Object->GetName();
	}

	virtual bool ShouldFindEditorForAsset() const override
	{
		return true;
	}

	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, InObjects);
	}

	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, const EAssetTypeActivationOpenedMethod OpenedMethod, TSharedPtr<IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override
	{
		if (OpenedMethod == EAssetTypeActivationOpenedMethod::Edit)
		{
			OpenAssetEditor(InObjects, EditWithinLevelEditor);
		}
	}

	virtual bool AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType) override
	{
		return false;
	}

	virtual TArray<FAssetData> GetValidAssetsForPreviewOrEdit(TArrayView<const FAssetData> InAssetDatas, bool bIsPreview) override
	{
		return TArray<FAssetData>(InAssetDatas);
	}

	virtual bool CanRename(const FAssetData& InAsset, FText* OutErrorMsg) const override
	{
		return true;
	}

	virtual bool CanDuplicate(const FAssetData& InAsset, FText* OutErrorMsg) const override
	{
		return true;
	}

	virtual bool CanFilter() override
	{
		return true;
	}

	virtual bool CanLocalize() const override
	{
		return true;
	}

	virtual bool CanMerge() const override
	{
		return false;
	}

	virtual void Merge(UObject* InObject) override
	{
		check(false); // no generic merge operation exists yet, did you override CanMerge but not Merge?
	}

	virtual void Merge(UObject* BaseAsset, UObject* RemoteAsset, UObject* LocalAsset, const FOnMergeResolved& ResolutionCallback) override
	{
		check(false); // no generic merge operation exists yet, did you override CanMerge but not Merge?
	}

	virtual bool ShouldForceWorldCentric() override
	{
		return false;
	}

	virtual void PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override
	{
		check(OldAsset != nullptr || NewAsset != nullptr);

		// Dump assets to temp text files
		FString OldTextFilename = DumpAssetToTempFile(OldAsset);
		FString NewTextFilename = DumpAssetToTempFile(NewAsset);
		FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateDiffProcess(DiffCommand, OldTextFilename, NewTextFilename);
	}

	virtual class UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override
	{
		return nullptr;
	}

	virtual EThumbnailPrimType GetDefaultThumbnailPrimitiveType(UObject* Asset) const override
	{
		return TPT_None;
	}

	virtual const TArray<FText>& GetSubMenus() const override
	{
		static const TArray<FText> SubMenus;
		return SubMenus;
	}

	virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override
	{
		return nullptr;
	}

	virtual bool IsImportedAsset() const override
	{
		return false;
	}

	virtual void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const override
	{
	}

	virtual void GetSourceFileLabels(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFileLabels) const override
	{
		TArray<FString> SourceFilePaths;
		OutSourceFileLabels.Reset();
		GetResolvedSourceFilePaths(TypeAssets, SourceFilePaths);
		if (SourceFilePaths.Num() > 0)
		{
			OutSourceFileLabels.AddDefaulted(SourceFilePaths.Num());
		}
	}

	virtual void BuildBackendFilter(FARFilter& InFilter) override
	{
		// Add the supported class for this type to a filter
		InFilter.ClassPaths.Add(GetClassPathName());
		InFilter.bRecursiveClasses = true;
	}
	
	virtual FText GetDisplayNameFromAssetData(const FAssetData& AssetData) const override
	{
		return FText::GetEmpty();
	}

	virtual bool SupportsOpenedMethod(const EAssetTypeActivationOpenedMethod OpenedMethod) const override
	{
		return OpenedMethod == EAssetTypeActivationOpenedMethod::Edit;
	}

	virtual FName GetFilterName() const override
	{
		return FName(*GetSupportedClass()->GetPathName());
	}

	virtual FTopLevelAssetPath GetClassPathName() const override
	{
		return GetSupportedClass()->GetClassPathName();
	}

	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override { return nullptr; }

	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override { return nullptr; }

protected:

	// Here are some convenience functions for common asset type actions logic

	/** Creates a unique package and asset name taking the form InBasePackageName+InSuffix */
	virtual void CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName) const
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(InBasePackageName, InSuffix, OutPackageName, OutAssetName);
	}

	/** Util for dumping an asset to a temporary text file. Returns absolute filename to temp file */
	virtual FString DumpAssetToTempFile(UObject* Asset) const
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		return AssetToolsModule.Get().DumpAssetToTempFile(Asset);
	}

	/** Returns additional tooltip information for the specified asset, if it has any (otherwise return the null widget) */
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override
	{
		return FText::GetEmpty();
	}

	/** Helper function to convert the input for GetActions to a list that can be used for delegates */
	template <typename T>
	static TArray<TWeakObjectPtr<T>> GetTypedWeakObjectPtrs(const TArray<UObject*>& InObjects)
	{
		check(InObjects.Num() > 0);

		TArray<TWeakObjectPtr<T>> TypedObjects;
		for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			TypedObjects.Add( CastChecked<T>(*ObjIt) );
		}

		return TypedObjects;
	}
	
	template <typename T>
    static TArray<TSoftObjectPtr<T>> GetTypedSoftObjectPtrs(const TArray<UObject*>& InObjects)
    {
    	check(InObjects.Num() > 0);

    	TArray<TSoftObjectPtr<T>> TypedObjects;
    	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
    	{
    		TypedObjects.Add( CastChecked<T>(*ObjIt) );
    	}

    	return TypedObjects;
    }

	template <typename T>
	static TArray<T*> GetTypedObjectPtrs(const TArray<UObject*>& InObjects)
	{
		check(InObjects.Num() > 0);

		TArray<T*> TypedObjects;
		for (UObject* ObjIt : InObjects)
		{
			TypedObjects.Add(CastChecked<T>(ObjIt));
		}

		return TypedObjects;
	}
};
