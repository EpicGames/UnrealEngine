// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"

#define UE_API ASSETTOOLS_API

class 
// UE_DEPRECATED(5.2, "The AssetDefinition system is replacing AssetTypeActions and nothing replaced this, just subclass from UAssetDefinitionDefault.  If you needed the ExecuteFindSourceFileInExplorer, you can now find that in FindSourceFileInExplorer.h.  Please see the Conversion Guide in AssetDefinition.h")
FAssetTypeActions_CSVAssetBase : public FAssetTypeActions_Base
{
public:
	virtual FColor GetTypeColor() const override { return FColor(62, 140, 35); }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
	virtual bool IsImportedAsset() const override { return true; }

protected:
	/** Handler for opening the source file for this asset */
	UE_API void ExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions);

	/** Determine whether the find source file in explorer editor command can execute or not */
	UE_API bool CanExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions) const;

	/** Verify the specified filename exists */
	UE_API bool VerifyFileExists(const FString& InFileName) const;
};

#undef UE_API
