// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "AssetTypeActivationOpenedMethod.generated.h"

/**
 *	(jcotton) This enum has been extracted into a separate header as it would ideally live in IAssetTypeActions.h or AssetTypeActions_Base.h, 
 *  however these files are included without module linking (IncludePathModuleNames) in several other modules which makes adding a UENUM() not 
 *  possible without refactoring.
 */ 

// Types of permissions allowed when attempting to open an asset in editor via activation (EAssetTypeActivationMethod)
UENUM()
enum class EAssetTypeActivationOpenedMethod : uint8
{
	Edit,
	View
};