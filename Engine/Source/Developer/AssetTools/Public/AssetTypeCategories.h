// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 *	this enum has been extracted into a separate header for easier inclusion in
 *	code in other modules
 */ 

// The list of categories for Asset Type Actions and UFactory subclasses
namespace EAssetTypeCategories
{
	enum Type
	{
		None = 0,
		Basic = 1 << 0,
		Animation = 1 << 1,
		Materials = 1 << 2,
		Sounds = 1 << 3,
		Physics = 1 << 4,
		UI = 1 << 5,
		Misc = 1 << 6,
		Gameplay = 1 << 7,
		Blueprint = 1 << 8,
		Media = 1 << 9,
		Textures = 1 << 10,
		World = 1 << 11,
		FX = 1 << 12,

		// Items below this will be allocated at runtime via RegisterAdvancedAssetCategory
		FirstUser = 1 << 13,
		LastUser = 1 << 31,
		// Last allowed value is 1 << 31
	};
}
