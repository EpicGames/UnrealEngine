// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Interfaces/IPluginManager.h"

class FViewport;

namespace AssetViewUtils
{
	/** Loads the specified object if needed and opens the asset editor for it */
	ASSETTOOLS_API bool OpenEditorForAsset(const FString& ObjectPath);

	/** Opens the asset editor for the specified asset */
	ASSETTOOLS_API bool OpenEditorForAsset(UObject* Asset);

	/** Opens the asset editor for the specified assets */
	ASSETTOOLS_API bool OpenEditorForAsset(const TArray<UObject*>& Assets);

	struct FLoadAssetsSettings
	{
		// Prompt the user with the number of unloaded assets before continuing to load
		bool bAlwaysPromptBeforeLoading = false;
		// Show a notification to the user that some assets failed to load
		bool bShowFailureNotification = true;
		// Follow redirectors and return the target assets.
		// We usually don't want to follow redirects when loading objects for the Content Browser.  It would
		// allow a user to interact with a ghost/unverified asset as if it were still alive.
		bool bFollowRedirectors = false;
		// Allow the user to cancel the load operation
		bool bAllowCancel = false;
		// Advanced setting: allow loading of world partition map packages.
		bool bLoadWorldPartitionMaps = false;
		// Advanced setting: load all external objects/actors with e.g. world partition map packages.
		bool bLoadAllExternalObjects = false;
	};

	enum class ELoadAssetsResult : uint8
	{
		Success,    // The user didn't cancel and all requested assets were successfully loaded
		Cancelled,  // The user cancelled the load operation
		SomeFailed, // Some or all of the assets failed to load
	};

	/**
	 * Makes sure the specified assets are loaded into memory, with displayed progress and optional cancellation for the user.
	 * Prefer the overload taking FAssetData if possible.
	 *
	 * @param Assets The assets to load
	 * @param LoadedObjects The returned list of objects that were already loaded or loaded by this method.
	 * @return enum specifying whether the load was successful, cancelled or if some or all assets failed to load
	 */
	ASSETTOOLS_API ELoadAssetsResult LoadAssetsIfNeeded(TConstArrayView<FAssetData> Assets, TArray<UObject*>& LoadedObjects, const FLoadAssetsSettings& InSettings);
	ASSETTOOLS_API ELoadAssetsResult LoadAssetsIfNeeded(TConstArrayView<FString> ObjectPaths, TArray<UObject*>& LoadedObjects, const FLoadAssetsSettings& InSettings);
	ASSETTOOLS_API ELoadAssetsResult LoadAssetsIfNeeded(TConstArrayView<FSoftObjectPath> ObjectPaths, TArray<UObject*>& LoadedObjects, const FLoadAssetsSettings& InSettings);

	/**
	  * Makes sure the specified assets are loaded into memory.
	  * 
	  * @param ObjectPaths The paths to the objects to load.
	  * @param LoadedObjects The returned list of objects that were already loaded or loaded by this method.
	  * @return false if user canceled after being warned about loading very many packages.
	  */
	UE_DEPRECATED(5.5, "Use LoadAssetsIfNeeded taking struct FLoadAssetsSettings instead.")
	ASSETTOOLS_API bool LoadAssetsIfNeeded(const TArray<FString>& ObjectPaths, TArray<UObject*>& LoadedObjects, bool bAllowedToPromptToLoadAssets = true, bool bLoadRedirects = false);

	/**
	 * Determines the unloaded assets that need loading
	 *
	 * @param ObjectPaths		Paths to assets that may need to be loaded
	 * @param OutUnloadedObjects	List of the unloaded object paths
	 * @return true if the user should be prompted to load assets
	 */
	ASSETTOOLS_API void GetUnloadedAssets(const TArray<FString>& ObjectPaths, TArray<FString>& OutUnloadedObjects);

	/**
	 * Prompts the user to load the list of unloaded objects
 	 *
	 * @param UnloadedObjects	The list of unloaded objects that we should prompt for loading
	 * @param true if the user allows the objects to be loaded
	 */
	ASSETTOOLS_API bool PromptToLoadAssets(const TArray<FString>& UnloadedObjects);

	/** 
	 * Copies assets to a new path 
	 * @param Assets The assets to copy
	 * @param DestPath The destination folder in which to copy the assets
	 */
	ASSETTOOLS_API void CopyAssets(const TArray<UObject*>& Assets, const FString& DestPath);

	/**
	  * Moves assets to a new path
	  * 
	  * @param Assets The assets to move
	  * @param DestPath The destination folder in which to move the assets
	  * @param SourcePath If non-empty, this will specify the base folder which will cause the move to maintain folder structure
	  */
	ASSETTOOLS_API void MoveAssets(const TArray<UObject*>& Assets, const FString& DestPath, const FString& SourcePath = FString());

	/** Attempts to deletes the specified assets. Returns the number of assets deleted */
	ASSETTOOLS_API int32 DeleteAssets(const TArray<UObject*>& AssetsToDelete);

	/** Attempts to delete the specified folders and all assets inside them. Returns true if the operation succeeded. */
	ASSETTOOLS_API bool DeleteFolders(const TArray<FString>& PathsToDelete);
	
	/** Attempts to delete the specified folders and all assets inside them. Returns true if the operation succeeded. */
	struct FDeleteFolderParameters
	{
		TArray<FString> PathsToDelete;
		bool bShowConfirmationBeforeLoadingAssets = true;
	};
	ASSETTOOLS_API bool DeleteFolders(const FDeleteFolderParameters& InOptions);

	/** Gets an array of assets inside the specified folders */
	ASSETTOOLS_API void GetAssetsInPaths(const TArray<FString>& InPaths, TArray<FAssetData>& OutAssetDataList);

	/** Saves all the specified packages */
	ASSETTOOLS_API bool SavePackages(const TArray<UPackage*>& Packages);

	/** Prompts to save all modified packages */
	ASSETTOOLS_API bool SaveDirtyPackages();

	/** Loads all the specified packages */
	ASSETTOOLS_API TArray<UPackage*> LoadPackages(const TArray<FString>& PackageNames);

	/** Moves all assets from the source path to the destination path, preserving path structure, deletes source path afterwards if possible */
	ASSETTOOLS_API bool RenameFolder(const FString& DestPath, const FString& SourcePath);

	/** Copies all assets in all source paths to the destination path, preserving path structure */
	ASSETTOOLS_API bool CopyFolders(const TArray<FString>& InSourcePathNames, const FString& DestPath);

	/** Moves all assets in all source paths to the destination path, preserving path structure */
	ASSETTOOLS_API bool MoveFolders(const TArray<FString>& InSourcePathNames, const FString& DestPath);

	/**
	  * A helper function for folder drag/drop which loads all assets in a path (including sub-paths) and returns the assets found
	  * 
	  * @param SourcePathNames				The paths to the folders to drag/drop
	  * @param OutSourcePathToLoadedAssets	The map of source folder paths to assets found
	  */
	ASSETTOOLS_API bool PrepareFoldersForDragDrop(const TArray<FString>& SourcePathNames, TMap< FString, TArray<UObject*> >& OutSourcePathToLoadedAssets);

	/**
	 * Capture active viewport to thumbnail and assigns that thumbnail to incoming assets
	 *
	 * @param InViewport - viewport to sample from
	 * @param InAssetsToAssign - assets that should receive the new thumbnail ONLY if they are assets that use GenericThumbnails
	 */
	ASSETTOOLS_API void CaptureThumbnailFromViewport(FViewport* InViewport, const TArray<FAssetData>& InAssetsToAssign);

	/**
	 * Clears custom thumbnails for the selected assets
	 *
	 * @param InAssetsToAssign - assets that should have their thumbnail cleared
	 */
	ASSETTOOLS_API void ClearCustomThumbnails(const TArray<FAssetData>& InAssetsToAssign);

	/** Returns true if the specified asset that uses shared thumbnails has a thumbnail assigned to it */
	ASSETTOOLS_API bool AssetHasCustomThumbnail( const FAssetData& AssetData );

	/** Returns true if the passed-in path is a project folder */
	ASSETTOOLS_API bool IsProjectFolder(const FStringView InPath, const bool bIncludePlugins = false);

	/** Returns true if the passed-in path is a engine folder */
	ASSETTOOLS_API bool IsEngineFolder(const FStringView InPath, const bool bIncludePlugins = false);

	/** Returns true if the passed-in path is a developers folder */
	ASSETTOOLS_API bool IsDevelopersFolder( const FStringView InPath );

	/** Returns true if the passed-in path is a plugin folder, optionally reporting where the plugin was loaded from */
	ASSETTOOLS_API bool IsPluginFolder(const FStringView InPath, EPluginLoadedFrom* OutPluginSource = nullptr);
	
	/** If the passed-in path is a plugin folder, then return its associated plugin */
	ASSETTOOLS_API TSharedPtr<IPlugin> GetPluginForFolder(const FStringView InPath);

	/** Get all the objects in a list of asset data */
	ASSETTOOLS_API void GetObjectsInAssetData(const TArray<FAssetData>& AssetList, TArray<UObject*>& OutDroppedObjects);

	/** Returns true if the supplied folder name can be used as part of a package name */
	ASSETTOOLS_API bool IsValidFolderName(const FString& FolderName, FText& Reason);

	/** Returns true if the path specified exists as a folder in the asset registry */
	ASSETTOOLS_API bool DoesFolderExist(const FString& FolderPath);

	/**
	 * Loads the color of this path from the config
	 *
	 * @param FolderPath - The path to the folder
	 * @return The color the folder should appear as, will be NULL if not customized
	 */
	UE_DEPRECATED(5.0, "LoadColor deprecated, please use GetPathColor")
	ASSETTOOLS_API const TSharedPtr<FLinearColor> LoadColor(const FString& FolderPath);

	/**
	 * Returns the color of this path from the config
	 *
	 * @param FolderPath - The path to the folder
	 * @return The color the folder should appear as, will be invalid if not customized
	 */
	ASSETTOOLS_API TOptional<FLinearColor> GetPathColor(const FString& FolderPath);

	/**
	 * Saves the color of the path to the config
	 *
	 * @param FolderPath - The path to the folder
	 * @param FolderColor - The color the folder should appear as
	 * @param bForceAdd - If true, force the color to be added for the path
	 */
	UE_DEPRECATED(5.0, "SaveColor deprecated, please use SetPathColor")
	ASSETTOOLS_API void SaveColor(const FString& FolderPath, const TSharedPtr<FLinearColor>& FolderColor, bool bForceAdd = false);

	/**
	 * Saves the color of the path to the config
	 *
	 * @param FolderPath - The path to the folder
	 * @param FolderColor - The color the folder should appear as
	 */
	ASSETTOOLS_API void SetPathColor(const FString& FolderPath, TOptional<FLinearColor> FolderColor);

	/**
	 * Checks to see if any folder has a custom color, optionally outputs them to a list
	 *
	 * @param OutColors - If specified, returns all the custom colors being used
	 * @return true, if custom colors are present
	 */
	ASSETTOOLS_API bool HasCustomColors( TArray< FLinearColor >* OutColors = NULL );

	/** Gets the default color the folder should appear as */
	ASSETTOOLS_API FLinearColor GetDefaultColor();

	/** Returns true if the specified path is available for object creation */
	ASSETTOOLS_API bool IsValidObjectPathForCreate(const FString& ObjectPath, FText& OutErrorMessage, bool bAllowExistingAsset = false);
	
	/** Returns true if the specified path is available for object creation */
	ASSETTOOLS_API bool IsValidObjectPathForCreate(const FString& ObjectPath, const UClass* ObjectClass, FText& OutErrorMessage, bool bAllowExistingAsset = false);

	/** Returns true if the specified folder name in the specified path is available for folder creation */
	ASSETTOOLS_API bool IsValidFolderPathForCreate(const FString& FolderPath, const FString& NewFolderName, FText& OutErrorMessage, bool bAllowExistingFolder = false);

	/** Returns the relative path, from the workspace root, of the package */
	ASSETTOOLS_API FString GetPackagePathWithinRoot(const FString& PackageName);

	/**
	 * Returns the length of the computed cooked package name and path whether it's run on a build machine or locally.
	 *
	 * @param bIsInternalBuild Whether to behave like this being used within Epic and make certain assumptions about paths
	 * 	 					   instead of using the local machine's workspace root path.
	 * @param CookSubPath Path relative to project root e.g. Saved/Cooked/SomePlatform
	 */
	ASSETTOOLS_API int32 GetPackageLengthForCooking(const FString& PackageName, bool bIsInternalBuild, FString CookSubPath={});

	/** Checks to see whether the path is within the size restrictions for cooking */
	ASSETTOOLS_API bool IsValidPackageForCooking(const FString& PackageName, FText& OutErrorMessage);

	/** Gets the maximum path length for an asset package file. Changes behavior based on whether the editor experimental setting for long paths is enabled. */
	ASSETTOOLS_API int32 GetMaxAssetPathLen();

	/** Gets the maximum path length for a cooked file. Changes behavior based on whether the editor experimental setting for long paths is enabled. */
	ASSETTOOLS_API int32 GetMaxCookPathLen();

	/** Syncs the specified packages from source control. */
	UE_DEPRECATED(5.3, "The bIsSyncLatestOperation parameter is deprecated. Use SyncLatestFromSourceControl instead.")
	ASSETTOOLS_API void SyncPackagesFromSourceControl(const TArray<FString>& PackageNames, bool bIsSyncLatestOperation);

	/** Syncs the specified packages from source control. */
	ASSETTOOLS_API bool SyncPackagesFromSourceControl(const TArray<FString>& PackageNames);

	/** Syncs the specified paths from source control. */
	ASSETTOOLS_API bool SyncPathsFromSourceControl(const TArray<FString>& Paths);

	/** Syncs a specific revision from source control. */
	ASSETTOOLS_API bool SyncRevisionFromSourceControl(const FString& Revision);

	/** Syncs latest from source control. */
	ASSETTOOLS_API bool SyncLatestFromSourceControl();

	/** Show an error notification toast if the given error message is not empty */
	ASSETTOOLS_API void ShowErrorNotifcation(const FText& InErrorMsg);

	/** Callback used when a folder should be forced to be visible in the Content Browser */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAlwaysShowPath, const FString& /*InPath*/);
	ASSETTOOLS_API FOnAlwaysShowPath& OnAlwaysShowPath();

	/** Called when a folder is moved or renamed */
	using FMovedContentFolder = TTuple<FString /*OldPath*/, FString /*NewPath*/>;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFolderPathChanged, TArrayView<const FMovedContentFolder> /*ChangedPaths*/);
	ASSETTOOLS_API FOnFolderPathChanged& OnFolderPathChanged();

	/** Called when a sync from AssetViewUtils starts */
	DECLARE_MULTICAST_DELEGATE(FOnSyncStart);
	UE_DEPRECATED(5.8, "This is no longer supported")
	ASSETTOOLS_API FOnSyncStart& OnSyncStart();

	/** 
	 * Called when a sync from AssetViewUtils finishes
	 * If Files is non-nullptr, those are the files affected by the sync.
	 * If Files is nullptr, assume any file might have been changed by the sync.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSyncFinish, bool /*bSucceeded*/, const TArray<FString>* /*Files*/);
	UE_DEPRECATED(5.8, "This is no longer supported")
	ASSETTOOLS_API FOnSyncFinish& OnSyncFinish();
}
