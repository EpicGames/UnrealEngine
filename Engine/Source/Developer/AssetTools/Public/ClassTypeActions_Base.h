// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IClassTypeActions.h"

struct FAssetData;

/** A base class for all ClassTypeActions. Provides helper functions useful for many types. Deriving from this class is optional. */
class FClassTypeActions_Base : public IClassTypeActions
{
public:
	// Begin IClassTypeActions implementation
	virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override
	{
		return nullptr;
	}
	// End IAssetTypeActions implementation
};
