// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "CollectionManagerTypes.h"
#include "Styling/SlateTypes.h"

#define UE_API ASSETTOOLS_API

class ICollectionContainer;

/** Handles the collection management for the given assets */
class FCollectionAssetManagement
{
public:
	/** Constructor */
	UE_DEPRECATED(5.6, "Use the ICollectionContainer constructor instead.")
	UE_API FCollectionAssetManagement();
	UE_API explicit FCollectionAssetManagement(const TSharedRef<ICollectionContainer>& InCollectionContainer);
	FCollectionAssetManagement(const FCollectionAssetManagement&) = delete;

	/** Destructor */
	UE_API ~FCollectionAssetManagement();

	FCollectionAssetManagement& operator=(const FCollectionAssetManagement&) = delete;

	/** Set the assets that we are currently observing and managing the collection state of */
	UE_API void SetCurrentAssets(const TArray<FAssetData>& CurrentAssets);

	/** Set the asset paths that we are currently observing and managing the collection state of */
	UE_API void SetCurrentAssetPaths(const TArray<FSoftObjectPath>& CurrentAssets);

	/** Get the number of assets in the current set */
	UE_API int32 GetCurrentAssetCount() const;

	/** Add the current assets to the given collection */
	UE_API void AddCurrentAssetsToCollection(FCollectionNameType InCollectionKey);

	/** Remove the current assets from the given collection */
	UE_API void RemoveCurrentAssetsFromCollection(FCollectionNameType InCollectionKey);

	/** Return whether or not the given collection should be enabled in any management UIs */
	UE_API bool IsCollectionEnabled(FCollectionNameType InCollectionKey) const;

	/** Get the check box state the given collection should use in any management UIs */
	UE_API ECheckBoxState GetCollectionCheckState(FCollectionNameType InCollectionKey) const;

private:
	/** Update the internal state used to track the check box status for each collection */
	UE_API void UpdateAssetManagementState();

	/** Handles an on collection renamed event */
	UE_API void HandleCollectionRenamed(ICollectionContainer&, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection);

	/** Handles an on collection updated event */
	UE_API void HandleCollectionUpdated(ICollectionContainer&, const FCollectionNameType& Collection);

	/** Handles an on collection destroyed event */
	UE_API void HandleCollectionDestroyed(ICollectionContainer&, const FCollectionNameType& Collection);

	/** Handles assets being added to a collection */
	UE_API void HandleAssetsAddedToCollection(ICollectionContainer&, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsAdded);

	/** Handles assets being removed from a collection */
	UE_API void HandleAssetsRemovedFromCollection(ICollectionContainer&, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsRemoved);

	TSharedRef<ICollectionContainer> CollectionContainer;

	/** Set of asset paths that we are currently observing and managing the collection state of */
	TSet<FSoftObjectPath> CurrentAssetPaths;

	/** Mapping between a collection and its asset management state (based on the current assets). A missing item is assumed to be unchecked */
	TMap<FCollectionNameType, ECheckBoxState> AssetManagementState;

	/** Delegate handles */
	FDelegateHandle OnCollectionRenamedHandle;
	FDelegateHandle OnCollectionDestroyedHandle;
	FDelegateHandle OnCollectionUpdatedHandle;
	FDelegateHandle OnAssetsAddedHandle;
	FDelegateHandle OnAssetsRemovedHandle;
};

#undef UE_API
