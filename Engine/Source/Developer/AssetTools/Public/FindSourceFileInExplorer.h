// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

//virtual FColor GetTypeColor() const override { return FColor(62, 140, 35); }
//virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
//virtual bool IsImportedAsset() const override { return true; }

namespace UE::AssetTools
{
	ASSETTOOLS_API void ExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions);
	ASSETTOOLS_API bool CanExecuteFindSourceFileInExplorer(TArray<FString> Filenames, TArray<FString> OverrideExtensions);
}
