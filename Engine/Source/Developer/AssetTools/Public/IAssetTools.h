// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/VersePathFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/SoftObjectPath.h"
#include "AssetTypeCategories.h"
#include "IAssetTypeActions.h"
#include "AutomatedAssetImportData.h"
#include "AssetRegistry/ARFilter.h"
#include "Engine/Blueprint.h"
#include "Logging/TokenizedMessage.h"
#include "IAssetTools.generated.h"

struct FAssetData;
class IAssetTools;
class IAssetTypeActions;
class IClassTypeActions;
class ILocalizedAssetTools;
class IPlugin;
class UFactory;
class UAssetImportTask;
class UAdvancedCopyCustomization;
class FNamePermissionList;
class FPathPermissionList;

namespace UE::AssetTools
{
	// Declared in PackageMigrationContext.h
	struct FPackageMigrationContext;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageMigration, FPackageMigrationContext&);

	DECLARE_DELEGATE_RetVal_OneParam(bool, FCanMigrateAsset, FName);

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FCanAssetBePublic, FStringView /*AssetPath*/, FText* /*DisallowReason*/);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FShouldCreateAssetsAsExternallyReferenceableForPath, FStringView /*AssetPath*/, TOptional<bool>& /*ExternallyReferenceable*/);

	DECLARE_DELEGATE_RetVal(bool, FShowingContentVersePath);
}

UENUM()
enum class EAssetClassAction : uint8
{
	/** Whether an action can be created by an AssetTypeAction */
	CreateAsset,
	/** Whether an asset can be viewed in the content browser */
	ViewAsset,
	/** Whether an asset can be imported */
	ImportAsset,
	/** Whether an asset can be exported */
	ExportAsset,
	AllAssetActions
};

UENUM()
enum class EAssetRenameResult : uint8
{
	/** The asset rename failed */
	Failure,
	/** The asset rename succeeded */
	Success,
	/** The asset rename is still pending, likely due to outstanding asset discovery */
	Pending,
};

UENUM()
enum class ERedirectFixupMode
{
	// Remove fully fixed up redirectors after the fixup process is done
	DeleteFixedUpRedirectors,

	// Leave the redirectors around even if no longer locally referenced
	LeaveFixedUpRedirectors,

	// If permitted, prompt the user to delete any redirectors that are no longer locally referenced.
	// Projects may disable deletion if it interferes with their workflow/branch setup/etc.
	PromptForDeletingRedirectors,
};

USTRUCT(BlueprintType)
struct FAssetRenameData
{
	GENERATED_BODY()

	/** Object being renamed */
	UPROPERTY(BlueprintReadWrite, Category=AssetRenameData)
	TWeakObjectPtr<UObject> Asset;

	/** New path to package without package name, ie /Game/SubDirectory */
	UPROPERTY(BlueprintReadWrite, Category = AssetRenameData)
	FString NewPackagePath;

	/** New package and asset name, new object path will be PackagePath/NewName.NewName */
	UPROPERTY(BlueprintReadWrite, Category = AssetRenameData)
	FString NewName;

	/** Full path to old name, in form /Game/SubDirectory/OldName.OldName:SubPath*/
	UPROPERTY()
	FSoftObjectPath OldObjectPath;

	/** New full path, may be a SubObject */
	UPROPERTY()
	FSoftObjectPath NewObjectPath;

	/** If true, only fix soft references. This will work even if Asset is null because it has already been renamed */
	UPROPERTY()
	bool bOnlyFixSoftReferences;
	
	/** If true, will also try to rename all localized variants with corresponding paths */
	UPROPERTY()
	bool bAlsoRenameLocalizedVariants;

	FAssetRenameData()
		: bOnlyFixSoftReferences(false)
		, bAlsoRenameLocalizedVariants(false) // This should probably always be true but it is false for backward compatibility reasons
	{}

	/** These constructors leave some fields empty, they are fixed up inside AssetRenameManager */
	FAssetRenameData(const TWeakObjectPtr<UObject>& InAsset, const FString& InNewPackagePath, const FString& InNewName)
		: Asset(InAsset)
		, NewPackagePath(InNewPackagePath)
		, NewName(InNewName)
		, bOnlyFixSoftReferences(false)
		, bAlsoRenameLocalizedVariants(false) // This should probably always be true but it is false for backward compatibility reasons
	{
	}
	/** These constructors leave some fields empty, they are fixed up inside AssetRenameManager */
	FAssetRenameData(const TWeakObjectPtr<UObject>& InAsset, const FString& InNewPackagePath, const FString& InNewName, bool bInOnlyFixSoftReferences, bool bInAlsoRenameLocalizedVariants)
		: Asset(InAsset)
		, NewPackagePath(InNewPackagePath)
		, NewName(InNewName)
		, bOnlyFixSoftReferences(bInOnlyFixSoftReferences)
		, bAlsoRenameLocalizedVariants(bInAlsoRenameLocalizedVariants)
	{
	}
	
	FAssetRenameData(const FSoftObjectPath& InOldObjectPath, const FSoftObjectPath& InNewObjectPath, bool bInOnlyFixSoftReferences = false, bool bInAlsoRenameLocalizedVariants = false)
		: OldObjectPath(InOldObjectPath)
		, NewObjectPath(InNewObjectPath)
		, bOnlyFixSoftReferences(bInOnlyFixSoftReferences)
		, bAlsoRenameLocalizedVariants(bInAlsoRenameLocalizedVariants) // This should probably be true by default but it is false for backward compatibility reasons
	{
	}
};

DECLARE_DYNAMIC_DELEGATE_TwoParams(FAdvancedCopyCompletedEvent, bool, bSuccess, const TArray<FAssetRenameData>&, AllCopiedAssets);

DECLARE_MULTICAST_DELEGATE_OneParam(FAssetPostRenameEvent, const TArray<FAssetRenameData>&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsNameAllowed, const FString& /*Name*/, FText* /*OutErrorMessage*/);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsObjectPathAllowed, const FString& /*FullPathAndName*/, FText* /*OutErrorMessage*/);
DECLARE_DELEGATE_OneParam(FSanitizeName, FString& /*NameToSanitize*/);

struct FAdvancedAssetCategory
{
	EAssetTypeCategories::Type CategoryType;
	FText CategoryName;

	FAdvancedAssetCategory(EAssetTypeCategories::Type InCategoryType, FText InCategoryName)
		: CategoryType(InCategoryType)
		, CategoryName(InCategoryName)
	{
	}
};

USTRUCT()
struct FAdvancedCopyParams
{
	GENERATED_USTRUCT_BODY()

	bool bShouldForceSave = false;
	bool bCopyOverAllDestinationOverlaps = false;
	bool bShouldSuppressUI = false;
	bool bShouldCheckForDependencies = false;
	FAdvancedCopyCompletedEvent OnCopyComplete;
	
	UE_DEPRECATED(5.0, "This function has been deprecated, use GetSelectedPackageOrFolderNames")
	const TArray<FName>& GetSelectedPackageNames() const
	{
		return SelectedPackageOrFolderNames;
	}

	const TArray<FName>& GetSelectedPackageOrFolderNames() const
	{
		return SelectedPackageOrFolderNames;
	}

	const FString& GetDropLocationForAdvancedCopy() const
	{
		return DropLocationForAdvancedCopy;
	}

	const TArray<UAdvancedCopyCustomization*>& GetCustomizationsToUse() const
	{
		return CustomizationsToUse;
	}

	void AddCustomization(UAdvancedCopyCustomization* InCustomization)
	{
		CustomizationsToUse.Add(InCustomization);
	}

	FAdvancedCopyParams(TArray<FName> InSelectedPackageOrFolderNames, FString InDropLocationForAdvancedCopy)
		: bShouldForceSave(false)
		, bCopyOverAllDestinationOverlaps(true)
		, bShouldSuppressUI(false)
		, bShouldCheckForDependencies(true)
		, SelectedPackageOrFolderNames(InSelectedPackageOrFolderNames)
		, DropLocationForAdvancedCopy(InDropLocationForAdvancedCopy)
	{
	}

	FAdvancedCopyParams() {}

private:
	TArray<FName> SelectedPackageOrFolderNames;
	TArray<UAdvancedCopyCustomization*> CustomizationsToUse;
	FString DropLocationForAdvancedCopy;
};


UENUM()
enum class EAssetMigrationConflict : uint8
{
	// Skip Assets
	Skip,
	// Overwrite Assets
	Overwrite,
	// Cancel the whole Migration
	Cancel
};

USTRUCT(BlueprintType)
struct FMigrationOptions
{
	GENERATED_BODY()

	/** Prompt user for confirmation (always false through scripting) */
	UPROPERTY(BlueprintReadWrite, Category = MigrationOptions)
	bool bPrompt;

	/** Ignore dependencies of assets, only migrate the given assets. usefull for automation. This will not migrate the actors of a OFPA (one file per actor) level */
	UPROPERTY(BlueprintReadWrite, Category = MigrationOptions)
	bool bIgnoreDependencies;

	/** What to do when Assets are conflicting on the destination */
	UPROPERTY(BlueprintReadWrite, Category = MigrationOptions)
	EAssetMigrationConflict AssetConflict;

	/** Destination for assets that don't have a corresponding content folder. If left empty those assets are not migrated. (Not used by the new migration)*/
	UPROPERTY(BlueprintReadWrite, Category = MigrationOptions)
	FString OrphanFolder;

	FMigrationOptions()
		: bPrompt(false)
		, bIgnoreDependencies(false)
		, AssetConflict(EAssetMigrationConflict::Skip)
	{}
};


// An array of maps each storing pairs of original object -> duplicated object
using FDuplicatedObjects = TArray<TMap<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>>>;

UINTERFACE(MinimalApi, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UAssetTools : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

DECLARE_DYNAMIC_DELEGATE_TwoParams(FAssetCreateCompleteDynamic, UFactory*, Factory, UObject*, NewObject);
DECLARE_DYNAMIC_DELEGATE_OneParam(FAssetCreateCancelledDynamic, UFactory*, Factory);
DECLARE_DELEGATE_TwoParams(FAssetCreateComplete, UFactory* /* Factory */, UObject* /* NewObject */);
DECLARE_DELEGATE_OneParam(FAssetCreateCancelled, UFactory* /* Factory */);

class IAssetTools
{
	GENERATED_IINTERFACE_BODY()

public:
	ASSETTOOLS_API static IAssetTools& Get();

	DECLARE_DELEGATE(FOnAssetsDiscovered);

	/** Registers an asset type actions object so it can provide information about and actions for asset types. */
	virtual void RegisterAssetTypeActions(const TSharedRef<IAssetTypeActions>& NewActions) = 0;

	/** Unregisters an asset type actions object. It will no longer provide information about or actions for asset types. */
	virtual void UnregisterAssetTypeActions(const TSharedRef<IAssetTypeActions>& ActionsToRemove) = 0;

	/** Generates a list of currently registered AssetTypeActions */
	virtual void GetAssetTypeActionsList(TArray<TWeakPtr<IAssetTypeActions>>& OutAssetTypeActionsList) const = 0;

	/** Gets the appropriate AssetTypeActions for the supplied class */
	virtual TWeakPtr<IAssetTypeActions> GetAssetTypeActionsForClass(const UClass* Class) const = 0;

	virtual TSharedPtr<ILocalizedAssetTools> GetLocalizedAssetTools() const = 0;

	virtual bool CanLocalize(const UClass* Class) const = 0;

	virtual TOptional<FLinearColor> GetTypeColor(const UClass* Class) const = 0;

	/** Gets the list of appropriate AssetTypeActions for the supplied class */
	virtual TArray<TWeakPtr<IAssetTypeActions>> GetAssetTypeActionsListForClass(const UClass* Class) const = 0;

	/**
	* Allocates a Category bit for a user-defined Category, or EAssetTypeCategories::Misc if all available bits are allocated.
	* Ignores duplicate calls with the same CategoryKey (returns the existing bit but does not change the display name).
	*/
	virtual EAssetTypeCategories::Type RegisterAdvancedAssetCategory(FName CategoryKey, FText CategoryDisplayName) = 0;

	/** Returns the allocated Category bit for a user-specified Category, or EAssetTypeCategories::Misc if it doesn't exist */
	virtual EAssetTypeCategories::Type FindAdvancedAssetCategory(FName CategoryKey) const = 0;

	/** Returns the list of all advanced asset categories */
	virtual void GetAllAdvancedAssetCategories(TArray<FAdvancedAssetCategory>& OutCategoryList) const = 0;

	/** Registers a class type actions object so it can provide information about and actions for class asset types. */
	virtual void RegisterClassTypeActions(const TSharedRef<IClassTypeActions>& NewActions) = 0;

	/** Unregisters a class type actions object. It will no longer provide information about or actions for class asset types. */
	virtual void UnregisterClassTypeActions(const TSharedRef<IClassTypeActions>& ActionsToRemove) = 0;

	/** Generates a list of currently registered ClassTypeActions */
	virtual void GetClassTypeActionsList(TArray<TWeakPtr<IClassTypeActions>>& OutClassTypeActionsList) const = 0;

	/** Gets the appropriate ClassTypeActions for the supplied class */
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActionsForClass(UClass* Class) const = 0;

	/**
	 * Creates an asset with the specified name, path, and factory
	 *
	 * @param AssetName the name of the new asset
	 * @param PackagePath the package that will contain the new asset
	 * @param AssetClass the class of the new asset
	 * @param Factory the factory that will build the new asset
	 * @param CallingContext optional name of the module or method calling CreateAsset() - this is passed to the factory
	 * @param bOverwriteExisting if set to true, this will overwrite an existing object without prompting
	 * @return the new asset or NULL if it fails
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual UObject* CreateAsset(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FName CallingContext = NAME_None, bool bOverwriteExisting = false) = 0;

	/**
	 * Creates an asset with the specified name, path, and factory asynchronously
	 *
	 * @param AssetName the name of the new asset
	 * @param PackagePath the package that will contain the new asset
	 * @param AssetClass the class of the new asset
	 * @param Factory the factory that will build the new asset
	 * @param OnComplete called when the factory has successfully created the asset
	 * @param OnCancelled called if the asynchronous method is cancelled
	 * @param CallingContext optional name of the module or method calling CreateAsset() - this is passed to the factory
	 * @param bOverwriteExisting if set to true, this will overwrite an existing object without prompting
	 * @return the new asset or NULL if it fails
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual bool CreateAssetAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateCompleteDynamic OnComplete, FAssetCreateCancelledDynamic OnCancelled, FName CallingContext = NAME_None, bool bOverwriteExisting = false)
	{
		return CreateAssetAsync(
			AssetName, 
			PackagePath, 
			AssetClass, 
			Factory,
			FAssetCreateComplete::CreateLambda([OnComplete](UFactory* Factory, UObject* NewObject) { OnComplete.ExecuteIfBound(Factory, NewObject); }),
			FAssetCreateCancelled::CreateLambda([OnCancelled](UFactory* Factory) { OnCancelled.ExecuteIfBound(Factory); }),
			CallingContext,
			bOverwriteExisting
		);
	}

	virtual bool CreateAssetAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete = FAssetCreateComplete(), FAssetCreateCancelled OnCancelled = FAssetCreateCancelled(), FName CallingContext = NAME_None, bool bOverwriteExisting = false) = 0;

	/**
	 * Creates one or more assets using the source objects as the basis for the next type.  This is a common enough operation
	 * it needed a utility.  In the case that you only have a single SourceObject, we'll lean on the content browser
	 * to create the asset and focus you to it so you can rename it inline.  However in the case that multiple assets
	 * get created we'll construct each one and then sync the content browser to them.
	 *
	 * You can return null for a factory if you need to skip a given SourceObject.
	 */
	virtual void CreateAssetsFrom(TConstArrayView<UObject*> SourceObjects, UClass* CreateAssetType, const FString& DefaultSuffix, TFunctionRef<UFactory*(UObject*)> FactoryConstructor, FName CallingContext = NAME_None) = 0;

	/**
	 * Creates one or more assets using the source objects as the basis for the next type.  This is a common enough operation
	 * it needed a utility.  In the case that you only have a single SourceObject, we'll lean on the content browser
	 * to create the asset and focus you to it so you can rename it inline.  However in the case that multiple assets
	 * get created we'll construct each one and then sync the content browser to them.
	 *
	 * You can return null for a factory if you need to skip a given SourceObject.
	 */
	template<typename SourceObjectType, typename = typename TEnableIf<TIsDerivedFrom<SourceObjectType, UObject>::Value>::Type>
	void CreateAssetsFrom(TConstArrayView<SourceObjectType*> SourceObjects, UClass* CreateAssetType, const FString& DefaultSuffix, TFunctionRef<UFactory*(SourceObjectType*)> FactoryConstructor, FName CallingContext = NAME_None)
	{
		CreateAssetsFrom(TConstArrayView<UObject*>(reinterpret_cast<UObject* const*>(SourceObjects.GetData()), SourceObjects.Num()), CreateAssetType, DefaultSuffix, [FactoryConstructor](UObject* SourceObject){ return FactoryConstructor(Cast<SourceObjectType>(SourceObject)); }, CallingContext);
	}

	/** Opens an asset picker dialog and creates an asset with the specified name and path */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual UObject* CreateAssetWithDialog(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FName CallingContext = NAME_None, const bool bCallConfigureProperties = true) = 0;

	/** Opens an asset picker dialog and creates an asset with the path chosen in the dialog */
	virtual UObject* CreateAssetWithDialog(UClass* AssetClass, UFactory* Factory, FName CallingContext = NAME_None) = 0;

	/**
	 * Opens an async dialog to create an asset.
	 * @returns true if the process was started successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual bool CreateAssetWithDialogAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateCompleteDynamic OnComplete, FAssetCreateCancelledDynamic OnCancelled, FName CallingContext = NAME_None)
	{
		return CreateAssetWithDialogAsync(
			AssetName, 
			PackagePath, 
			AssetClass, 
			Factory,
			FAssetCreateComplete::CreateLambda([OnComplete](UFactory* Factory, UObject* NewObject) { OnComplete.ExecuteIfBound(Factory, NewObject); }),
			FAssetCreateCancelled::CreateLambda([OnCancelled](UFactory* Factory) { OnCancelled.ExecuteIfBound(Factory); }),
			CallingContext
		);
	}

	virtual bool CreateAssetWithDialogAsync(const FString& AssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete = FAssetCreateComplete(), FAssetCreateCancelled OnCancelled = FAssetCreateCancelled(), FName CallingContext = NAME_None) = 0;

	/**
	 * Opens an async dialog to create an asset.
	 * @returns true if the process was started successfully.
	 */
	virtual bool CreateAssetWithDialogAsync(UClass* AssetClass, UFactory* Factory, FAssetCreateComplete OnComplete = FAssetCreateComplete(), FAssetCreateCancelled OnCancelled = FAssetCreateCancelled(), FName CallingContext = NAME_None) = 0;

	/** Opens an asset picker dialog and creates an asset with the specified name and path. Uses OriginalObject as the duplication source. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual UObject* DuplicateAssetWithDialog(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject) = 0;

	/** Opens an asset picker dialog and creates an asset with the specified name and path. 
	 * Uses OriginalObject as the duplication source.
	 * Uses DialogTitle as the dialog's title.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual UObject* DuplicateAssetWithDialogAndTitle(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject, FText DialogTitle) = 0;

	/** Creates an asset with the specified name and path. Uses OriginalObject as the duplication source. */
	UFUNCTION(BlueprintCallable, Category="Editor Scripting | Asset Tools")
	virtual UObject* DuplicateAsset(const FString& AssetName, const FString& PackagePath, UObject* OriginalObject) = 0;

	/** Controls whether or not newly created assets are made externally referneceable or not */
	virtual void SetCreateAssetsAsExternallyReferenceable(bool bValue) = 0;

	/** Gets whether all assets being made at any location are externally referenceable or not */
	virtual bool GetCreateAssetsAsExternallyReferenceable() = 0;

	/** Gets whether assets being made at this location are externally referenceable or not */
	virtual bool ShouldCreateAssetsAsExternallyReferenceableForPath(const FStringView AssetPath) const = 0;

	/** Delegate to control whether assets being made at a specific location are externally referenceable or not */
	virtual UE::AssetTools::FShouldCreateAssetsAsExternallyReferenceableForPath& GetOnShouldCreateAssetsAsExternallyReferenceableForPath() = 0;

	/** Gets whether assets registry is still loading assets or not */
	virtual bool IsDiscoveringAssetsInProgress() const = 0;

	/** Opens a dialog asking the user to wait while assets are being discovered */
	virtual void OpenDiscoveringAssetsDialog(const FOnAssetsDiscovered& InOnAssetsDiscovered) = 0;

	/** Renames assets using the specified names. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual bool RenameAssets(const TArray<FAssetRenameData>& AssetsAndNames) = 0;

	/** Renames assets using the specified names. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual EAssetRenameResult RenameAssetsWithDialog(const TArray<FAssetRenameData>& AssetsAndNames, bool bAutoCheckout = false) = 0;

	/** Returns list of objects that soft reference the given soft object path. This will load assets into memory to verify */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void FindSoftReferencesToObject(FSoftObjectPath TargetObject, TArray<UObject*>& ReferencingObjects) = 0;

	/** Returns list of objects that soft reference the given soft object paths. This will load assets into memory to verify */
	virtual void FindSoftReferencesToObjects(const TArray<FSoftObjectPath>& TargetObjects, TMap<FSoftObjectPath, TArray<UObject*>>& ReferencingObjects) = 0;

	/**
	 * Function that renames all FSoftObjectPath object with the old asset path to the new one.
	 *
	 * @param PackagesToCheck Packages to check for referencing FSoftObjectPath.
	 * @param AssetRedirectorMap Map from old asset path to new asset path
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void RenameReferencingSoftObjectPaths(const TArray<UPackage *> PackagesToCheck, const TMap<FSoftObjectPath, FSoftObjectPath>& AssetRedirectorMap) = 0;

	/** Event issued at the end of the rename process */
	virtual FAssetPostRenameEvent& OnAssetPostRename() = 0;

	/**
	 * Opens a file open dialog to choose files to import to the destination path.
	 *
	 * @param DestinationPath	Path to import files to
	 * @return list of successfully imported assets
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual TArray<UObject*> ImportAssetsWithDialog(const FString& DestinationPath) = 0;

	/**
	 * Opens a file open dialog to choose files to import to the destination path.
	 * It differ from ImportAssetsWithDialog by allowing an async import when it's available
	 *
	 * @param DestinationPath	Path to import files to
	 */
	virtual void ImportAssetsWithDialogAsync(const FString& DestinationPath) = 0;

	/**
	 * Imports the specified files to the destination path.
	 *
	 * @param Files				Files to import
	 * @param DestinationPath	destination path for imported files
	 * @param ChosenFactory		Specific factory to use for object creation
	 * @param bSyncToBrowser	If true sync content browser to first imported asset after import
	 * @param bAllowAsyncImport	This allow the import code to use a async importer if enabled. (Note doing so will ignore the ChosenFactory arguments (If you want a async import prefer the InterchangeManager api))
	 * @return list of successfully imported assets
	 */
	virtual TArray<UObject*> ImportAssets(const TArray<FString>& Files, const FString& DestinationPath, UFactory* ChosenFactory = NULL, bool bSyncToBrowser = true, TArray<TPair<FString, FString>>* FilesAndDestinations = nullptr, bool bAllowAsyncImport = false, bool bSceneImport = false) const = 0;

	/**
	 * Imports assets using data specified completely up front.  Does not ever ask any questions of the user or show any modal error messages
	 *
	 * @param AutomatedImportData	Data that specifies how to import each file
	 * @return list of successfully imported assets
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual TArray<UObject*> ImportAssetsAutomated( const UAutomatedAssetImportData* ImportData) = 0;

	/**
	* Imports assets using tasks specified.
	*
	* @param ImportTasks	Tasks that specify how to import each file
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void ImportAssetTasks(const TArray<UAssetImportTask*>& ImportTasks) = 0;

	/**
	 * Exports the specified objects to file.
	 *
	 * @param	AssetsToExport					List of full asset names (e.g /Game/Path/Asset) to export
	 * @param	ExportPath						The directory path to export to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void ExportAssets(const TArray<FString>& AssetsToExport, const FString& ExportPath) = 0;

	/**
	 * Exports the specified objects to file.
	 *
	 * @param	AssetsToExport					List of assets to export 
	 * @param	ExportPath						The directory path to export to.
	 */
	virtual void ExportAssets(const TArray<UObject*>& AssetsToExport, const FString& ExportPath) const = 0;
	
	/**
	 * Exports the specified objects to file using the clean filename as the saved filename.
	 *
	 * @param	AssetsToExport					List of assets to export
	 * @param	ExportPath						The directory path to export to.
	 */
	virtual void ExportAssetsWithCleanFilename(const TArray<UObject*>& AssetsToExport, const FString& ExportPath) const = 0;

	/**
	 * Exports the specified objects to file. First prompting the user to pick an export directory and optionally prompting the user to pick a unique directory per file
	 *
	 * @param	AssetsToExport					List of assets to export
	 * @param	ExportPath						The directory path to export to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void ExportAssetsWithDialog(const TArray<FString>& AssetsToExport, bool bPromptForIndividualFilenames) = 0;
	
	/**
	 * Exports the specified objects to file. First prompting the user to pick an export directory and optionally prompting the user to pick a unique directory per file
	 *
	 * @param	AssetsToExport					List of full asset names (e.g /Game/Path/Asset) to export
	 * @param	ExportPath						The directory path to export to.
	 */
	virtual void ExportAssetsWithDialog(const TArray<UObject*>& AssetsToExport, bool bPromptForIndividualFilenames) = 0;

	/**
	 * Check if specified assets can be exported.
	 *
	 * @param	AssetsToExport					List of assets to export
	 * @return true if all assets specified can be exported
	 */
	virtual bool CanExportAssets(const TArray<FAssetData>& AssetsToExport) const = 0;

	/** Creates a unique package and asset name taking the form InBasePackageName+InSuffix */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName) = 0;

	/** Returns true if the specified asset uses a stock thumbnail resource */
	virtual bool AssetUsesGenericThumbnail(const FAssetData& AssetData) const = 0;

	/**
	 * Try to diff the local version of an asset against the latest one from the depot
	 *
	 * @param InObject - The object we want to compare against the depot
	 * @param InPackagePath - The fullpath to the package
	 * @param InPackageName - The name of the package
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void DiffAgainstDepot(UObject* InObject, const FString& InPackagePath, const FString& InPackageName) const = 0;

	/** Try and diff two assets using class-specific tool. Will do nothing if either asset is NULL, or they are not the same class. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual void DiffAssets(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const = 0;

	/** Util for dumping an asset to a temporary text file. Returns absolute filename to temp file */
	virtual FString DumpAssetToTempFile(UObject* Asset) const = 0;

	/** Attempt to spawn Diff tool as external process
	 *
	 * @param DiffCommand -		Command used to launch the diff tool
	 * @param OldTextFilename - File path to original file
	 * @param NewTextFilename - File path to new file
	 * @param DiffArgs -		Any extra command line arguments (defaulted to empty)
	 *
	 * @return Returns true if the process has successfully been created.
	 */
	virtual bool CreateDiffProcess(const FString& DiffCommand, const FString& OldTextFilename, const FString& NewTextFilename, const FString& DiffArgs = FString("")) const = 0;

	/* Migrate packages to another game content folder */
	virtual void MigratePackages(const TArray<FName>& PackageNamesToMigrate) const = 0;

	/* Migrate packages and dependencies to another folder */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools", BlueprintPure = false)
	virtual void MigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& DestinationPath, const struct FMigrationOptions& Options = FMigrationOptions()) const = 0;
	
	/* Migrate packages and dependencies to another folder */
	virtual void MigratePackages(const TArray<FName>& PackageNamesToMigrate, const FString& DestinationPath, TSet<FName> AllDependenciesToMigrate, TSet<FName> ExcludedDependencies, const struct FMigrationOptions& Options = FMigrationOptions()) const = 0;

	/* Build dependency packages and excluded dependencies from a list of packages to migrate */
	virtual void BuildMigrateDependencies(const TArray<FName>& PackageNamesToMigrate, TSet<FName>& OutAllDependenciesToMigrate, TSet<FName>& OutExcludedDependencies) const = 0;
	
	/**
	 * Event called when some packages are migrated
	 * Note this is only true when AssetTools.UseNewPackageMigration is true
	 */
	virtual UE::AssetTools::FOnPackageMigration& GetOnPackageMigration() = 0;

	/* Copy packages and dependencies to another folder */
	virtual void BeginAdvancedCopyPackages(const TArray<FName>& InputNamesToCopy, const FString& TargetPath) const = 0;

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools", BlueprintPure = false, meta = (AutoCreateRefTerm = "OnCopyComplete"))
	virtual void BeginAdvancedCopyPackages(const TArray<FName>& InputNamesToCopy, const FString& TargetPath, const FAdvancedCopyCompletedEvent& OnCopyComplete) const = 0;

	/**
	 * Fix up references to the specified redirectors.
	 * @param bCheckoutDialogPrompt indicates whether to prompt the user with files checkout dialog or silently attempt to checkout all necessary files.
	 */
	virtual void FixupReferencers(const TArray<UObjectRedirector*>& Objects, bool bCheckoutDialogPrompt = true, ERedirectFixupMode FixupMode = ERedirectFixupMode::DeleteFixedUpRedirectors) const = 0;

	/** Returns whether redirectors are being fixed up. */
	virtual bool IsFixupReferencersInProgress() const = 0;

	/** Expands any folders found in the files list, and returns a flattened list of destination paths and files.  Mirrors directory structure. */
	virtual void ExpandDirectories(const TArray<FString>& Files, const FString& DestinationPath, TArray<TPair<FString, FString>>& FilesAndDestinations) const = 0;

	/** Copies files after the final set of maps of sources and destinations was confirmed */
	virtual bool AdvancedCopyPackages(const FAdvancedCopyParams& CopyParams, const TArray<TMap<FString, FString>>& PackagesAndDestinations) const = 0;

	/** Copies files after the flattened map of sources and destinations was confirmed */
	virtual bool AdvancedCopyPackages(const TMap<FString, FString>& SourceAndDestPackages, const bool bForceAutosave = false, const bool bCopyOverAllDestinationOverlaps = true, FDuplicatedObjects* OutDuplicatedObjects = nullptr, EMessageSeverity::Type NotificationSeverityFilter = EMessageSeverity::Info) const = 0;

	/* Given a set of packages to copy, generate the map of those packages to destination filenames */
	virtual void GenerateAdvancedCopyDestinations(FAdvancedCopyParams& InParams, const TArray<FName>& InPackageNamesToCopy, const UAdvancedCopyCustomization* CopyCustomization, TMap<FString, FString>& OutPackagesAndDestinations) const = 0;

	/* Flattens the maps for each selected package into one complete map to pass to the final copy function while checking for collisions */
	virtual bool FlattenAdvancedCopyDestinations(const TArray<TMap<FString, FString>>& PackagesAndDestinations, TMap<FString, FString>& FlattenedPackagesAndDestinations) const = 0;

	/* Validate the destinations for advanced copy once the map has been flattened */
	virtual bool ValidateFlattenedAdvancedCopyDestinations(const TMap<FString, FString>& FlattenedPackagesAndDestinations) const = 0;

	/* Find all the dependencies that also need to be copied in the advanced copy, mapping them to the file that depends on them and excluding any that don't pass the ARFilter stored on CopyParams */
	virtual void GetAllAdvancedCopySources(FName SelectedPackage, FAdvancedCopyParams& CopyParams, TArray<FName>& OutPackageNamesToCopy, TMap<FName, FName>& DependencyMap, const class UAdvancedCopyCustomization* CopyCustomization) const = 0;

	/* Given a complete set of copy parameters, which includes the selected package set, start the advanced copy process */
	virtual void InitAdvancedCopyFromCopyParams(FAdvancedCopyParams CopyParams) const = 0;

	/** Opens editor for assets */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools", meta = (DeprecatedFunction, DeprecationMessage = "Please use UAssetEditorSubsystem::OpenEditorForAssets instead."))
	virtual void OpenEditorForAssets(const TArray<UObject*>& Assets) = 0;

	/** Converts the given UTexture2D to virtual textures or converts virtual textures back to standard textures and updates the related UMaterials 
	 * @param Textures					The given textures to convert.
	 * @param bConvertBackToNonVirtual	If true, virtual textures will be converted back to standard textures.
	 * @param RelatedMaterials			An optional array of materials to update after the conversion, this is useful during import when not all dependencies and caches are up to date.
	 */
	virtual void ConvertVirtualTextures(const TArray<UTexture2D*>& Textures, bool bConvertBackToNonVirtual, const TArray<UMaterial*>* RelatedMaterials = nullptr) const = 0;

	/** Is the given asset class supported? */
	virtual bool IsAssetClassSupported(const UClass* AssetClass) const = 0;

	/** Find all supported asset factories. */
	virtual TArray<UFactory*> GetNewAssetFactories() const = 0;

	/** Get asset class permission list for content browser and other systems */
	virtual const TSharedRef<FPathPermissionList>& GetAssetClassPathPermissionList(EAssetClassAction AssetClassAction) const = 0;

	/** Get extension permission list allowed for importer */
	virtual const TSharedRef<FNamePermissionList>& GetImportExtensionPermissionList() const = 0;

	virtual bool IsImportExtensionAllowed(const FStringView& Extension) const = 0;

	/** Which BlueprintTypes are allowed to be created. An empty list should allow everything. */
	virtual TSet<EBlueprintType>& GetAllowedBlueprintTypes() = 0;

	/** Get folder permission list for content browser and other systems */
	virtual TSharedRef<FPathPermissionList>& GetFolderPermissionList() = 0;

	/** Get writable folder permission list for content browser and other systems */
	virtual TSharedRef<FPathPermissionList>& GetWritableFolderPermissionList() = 0;

	/** Determines whether an asset has a viewable asset class and its folder is allowed. If not, it can also check if it's aliased to an allowed location. */
	virtual bool IsAssetVisible(const FAssetData& AssetData, bool bCheckAliases = true) const = 0;

	/**
	 * Determines whether an asset is considered read-only for editing, based on its flags and the current permissions (@see GetWritableFolderPermissionList).
	 * @note This does not check the read-only state of the file itself or anything regarding source control, as it's only checking if the asset can be edited in-memory, not that any edits can be saved to disk.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	virtual bool IsAssetReadOnly(const FAssetData& AssetData) const = 0;

	/**
	 * Find the asset data for the specified Verse path
	 * @note Verse paths to objects can be ambiguous. This function can be expensive for certain Verse paths.
	 *
	 * @param VersePath the Verse path of the object to be looked up
	 * @param bIncludeOnlyOnDiskAssets If true, use only DiskGatheredData, do not calculate from UObjects.
	 *        @see IAssetRegistry class header for bIncludeOnlyOnDiskAssets.
	 * @param bSkipARFilteredAssets If true, skips Objects that return true for IsAsset but are not assets in the current platform.
	 * @return the assets data; Will be invalid if object could not be found
	 */
	UE_INTERNAL virtual FAssetData FindAssetByVersePath(const UE::Core::FVersePath& VersePath, bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets = true) const = 0;

	/** Return whether to surface Content Verse paths throughout the editor */
	UE_INTERNAL virtual bool ShowingContentVersePath() const = 0;

	/** Delegate to control whether we are showing content Verse paths or not */
	UE_INTERNAL virtual UE::AssetTools::FShowingContentVersePath& GetOnShowingContentVersePath() = 0;

	/**
	 * Return the path to display to the user in editor UI for the given plugin.
	 * Attempts to convert the folder to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual const FString& GetUserFacingPluginName(const IPlugin& Plugin) const = 0;

	/**
	 * Return the path to display to the user in editor UI for the given plugin.
	 * Attempts to convert the folder to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingPluginName(FStringView PluginName) const = 0;

	/**
	 * Return the path to display to the user in editor UI for the given package path.
	 * Attempts to convert the folder to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingLongPackagePath(FStringView LongPackagePath) const = 0;

	/**
	 * Return the path to display to the user in editor UI for the given long package name.
	 * Attempts to convert the primary asset in the package to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingLongPackageName(FName PackageName) const = 0;

	/**
	 * Return the paths to display to the user in editor UI for the given long package names.
	 * Attempts to convert the primary asset in each package to a Verse path if ShowingContentVersePath is true.
	 * @note The array return value is guaranteed to have the same size as PackageNames.
	 */
	UE_INTERNAL virtual TArray<FString> GetUserFacingLongPackageNames(TConstArrayView<FName> PackageNames) const = 0;

	/**
	* Return the path to display to the user in editor UI for the given soft object path.
	* Attempts to convert the object path to a Verse path if ShowingContentVersePath is true.
	*/
	UE_INTERNAL virtual FString GetUserFacingObjectPath(const FSoftObjectPath& ObjectPath) const = 0;

	/**
	 * Return the paths to display to the user in editor UI for the given soft object paths.
	 * Attempts to convert each object path to a Verse path if ShowingContentVersePath is true.
	 * @note The array return value is guaranteed to have the same size as ObjectPaths.
	 */
	UE_INTERNAL virtual TArray<FString> GetUserFacingObjectPaths(TConstArrayView<FSoftObjectPath> ObjectPaths) const = 0;

	/**
	 * Return the path to display to the user in editor UI for the given package.
	 * Attempts to convert the primary asset in the package to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingLongPackageName(const UPackage& Package) const = 0;

	/**
	 * Return the paths to display to the user in editor UI for the given packages.
	 * Attempts to convert the primary asset in each package to a Verse path if ShowingContentVersePath is true.
	 * @note The array return value is guaranteed to have the same size as Packages.
	 */
	UE_INTERNAL virtual TArray<FString> GetUserFacingLongPackageNames(TConstArrayView<const UPackage*> Packages) const = 0;

	/**
	 * Return the full name to display to the user in editor UI for the given object.
	 * Attempts to convert the object path portion of the full name to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingFullName(const UObject& Object) const = 0;

	/**
	 * Return the path to display to the user in editor UI for the given asset data.
	 * Attempts to convert the object path to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingPathName(const UObject& Object) const = 0;

	/**
	 * Return the full name to display to the user in editor UI for the given object.
	 * Attempts to convert the asset path portion of the full name to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingFullName(const FAssetData& AssetData) const = 0;

	/**
	 * Return the path to display to the user in editor UI for the given asset data.
	 * Attempts to convert the asset path to a Verse path if ShowingContentVersePath is true.
	 */
	UE_INTERNAL virtual FString GetUserFacingPathName(const FAssetData& AssetData) const = 0;

	/** Returns true if all in list pass writable folder filter */
	virtual bool AllPassWritableFolderFilter(const TArray<FString>& InPaths) const = 0;

	/** Returns true if IsNameAllowedDelegate is not set, or if the name passes the filter function */
	virtual bool IsNameAllowed(const FString& Name, FText* OutErrorMessage = nullptr) const = 0;
	/** Allows setting of a global name filter that is applied to assets and folders */
	virtual void RegisterIsNameAllowedDelegate(const FName OwnerName, FIsNameAllowed Delegate) = 0;
	/** Remove a previously-set global name filter */
	virtual void UnregisterIsNameAllowedDelegate(const FName OwnerName) = 0;

	/** Returns true if IsObjectPathAllowedDelegate is not set, or if the name passes the filter function */
	virtual bool IsObjectPathAllowed(const FString& FullObjectPathWithName, FText* OutErrorMessage = nullptr) const = 0;
	/** Allows setting of a global name filter that is applied to assets and folders */
	virtual void RegisterIsObjectPathAllowedDelegate(const FName OwnerName, FIsObjectPathAllowed Delegate) = 0;
	/** Remove a previously-set global name filter */
	virtual void UnregisterIsObjectPathAllowedDelegate(const FName OwnerName) = 0;

	/** Sanitize the name by calling all registered delegates. */
	virtual bool SanitizeName(FString& NameToSanitize) = 0;
	/** Allows setting of a global name filter that is applied to assets and folders */
	virtual void RegisterSanitizeNameDelegate(const FName OwnerName, FSanitizeName Delegate) = 0;
	/** Remove a previously-set global name filter */
	virtual void UnregisterSanitizeNameDelegate(const FName OwnerName) = 0;

	/** Get the default name for the supplied class (if one is registered) to use when creating new assets of that type */
	virtual TOptional<FString> GetDefaultAssetNameForClass(const UClass* Class, const UFactory* Factory = nullptr, const UObject* SourceObject = nullptr) = 0;

	/** Get the default name for the supplied class (if one is registered) to use when creating a new asset of that type from the input source object */
	UE_DEPRECATED(5.8, "Use GetDefaultAssetNameForClass and pass a source object")
	virtual TOptional<FString> GetDefaultAssetNameForClassFromSource(const UClass* Class, const UObject* SourceObject, const UFactory* Factory = nullptr) final
	{
		return GetDefaultAssetNameForClass(Class, Factory, SourceObject);
	}
	/** Register a default name for the supplied class to use when creating new assets of that type */
	virtual void RegisterDefaultAssetNameForClass(const UClass* Class, const FString& DefaultName) = 0;
	/** Remove any previously registered name for the supplied class */
	virtual void UnregisterDefaultAssetNameForClass(const UClass* Class) = 0;

	/** Show notification that writable folder filter blocked an action */
	virtual void NotifyBlockedByWritableFolderFilter() const = 0;

	/** Allow to add some restrictions to the assets that can be migrated */
	virtual void RegisterCanMigrateAsset(const FName OwnerName, UE::AssetTools::FCanMigrateAsset Delegate) = 0;
	virtual void UnregisterCanMigrateAsset(const FName OwnerName) = 0;

	/** Returns whether the specified asset can be public (referenceable from another mount point / plugin) */
	virtual bool CanAssetBePublic(FStringView AssetPath, FText* DisallowReason = nullptr) const = 0;

	/**
	 * Register/unregister delegates to specify whether an asset can be made public (referenceable from another mount point / plugin)
	 * By default any asset can be public and if any delegate return false, the asset must be private
	 */
	virtual void RegisterCanAssetBePublic(const FName OwnerName, UE::AssetTools::FCanAssetBePublic Delegate) = 0;
	virtual void UnregisterCanAssetBePublic(const FName OwnerName) = 0;

	/** Syncs the primary content browser to the specified assets, whether or not it is locked. Most syncs that come from AssetTools -feel- like they came from the content browser, so this is okay. */
	virtual void SyncBrowserToAssets(const TArray<UObject*>& AssetsToSync) = 0;
	virtual void SyncBrowserToAssets(const TArray<FAssetData>& AssetsToSync) = 0;
	
};

UCLASS(transient)
class UAssetToolsHelpers : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	static TScriptInterface<IAssetTools> GetAssetTools();
};
