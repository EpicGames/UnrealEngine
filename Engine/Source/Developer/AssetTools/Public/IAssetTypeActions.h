// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Merge.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "AssetRegistry/AssetData.h"
#include "AssetDefinition.h"

class IToolkitHost;
enum class EAssetTypeActivationOpenedMethod : uint8;

namespace EAssetTypeActivationMethod
{
	enum Type
	{
		DoubleClicked,
		Opened,
		Previewed
	};
}


/** AssetTypeActions provide actions and other information about asset types */
class IAssetTypeActions : public TSharedFromThis<IAssetTypeActions>
{
public:
	/** Virtual destructor */
	virtual ~IAssetTypeActions()= default;

	/**
	 * Returns if this is actually just an UAssetDefinition.  Which doesn't exist in this module.  UAssetDefinition
	 * is replacing IAssetTypeActions, but for now we generate fake IAssetTypeActions that are secretly AssetDefinitions
	 */
	virtual bool IsAssetDefinitionInDisguise() const { return false; }

	/** Returns the name of this type */
	virtual FText GetName() const = 0;

	/** Get the supported class of this type. */
	virtual UClass* GetSupportedClass() const = 0;

	/** Returns the color associated with this type */
	virtual FColor GetTypeColor() const = 0;

	/** Returns true if we should call GetActions. */
	virtual bool ShouldCallGetActions() const { return true; }
	
	/** Returns true if this class can supply actions for InObjects. */
   	virtual bool HasActions( const TArray<UObject*>& InObjects ) const { return true; }

	/** Generates a menubuilder for the specified objects. */
	virtual void GetActions( const TArray<UObject*>& InObjects, class FMenuBuilder& MenuBuilder ) { }

	/** Generates a menu section for the specified objects. */
	virtual void GetActions(const TArray<UObject*>& InObjects, struct FToolMenuSection& Section) { }
	
	/** Returns true if an asset editor window should be created for this asset. */
	virtual bool ShouldFindEditorForAsset() const = 0;

	/** Opens the asset editor for the specified objects. If EditWithinLevelEditor is valid, the world-centric editor will be used. */
	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) = 0;

	/** Opens the asset editor for the specified objects. If EditWithinLevelEditor is valid, the world-centric editor will be used. */
	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, const EAssetTypeActivationOpenedMethod OpenedMethod, TSharedPtr<IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) = 0;

	/** Allows overriding asset activation to perform asset type specific activation for the supplied assets. This happens when the user double clicks, presses enter, or presses space. Return true if you have overridden the behavior. */
	virtual bool AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType) = 0;

	/** Returns true if this asset can be renamed */
	virtual bool CanRename(const FAssetData& InAsset, FText* OutErrorMsg) const = 0;

	/** Returns true if this asset can be duplicated */
	virtual bool CanDuplicate(const FAssetData& InAsset, FText* OutErrorMsg) const = 0;

	/** Returns the set of asset data that is valid to load. */
	virtual TArray<FAssetData> GetValidAssetsForPreviewOrEdit(TArrayView<const FAssetData> InAssetDatas, bool bIsPreview) = 0;

	/** Returns true if this class can be used as a filter in the content browser */
	virtual bool CanFilter() = 0;

	/** Returns name to use for filter in the content browser */
	virtual FName GetFilterName() const = 0;

	/** Returns true if this class can be localized */
	virtual bool CanLocalize() const = 0;

	/** Returns true if this class can be merged (either manually or automatically) */
	virtual bool CanMerge() const = 0;

	/** Begins a merge operation for InObject (automatically determines remote/base versions needed to resolve) */
	virtual void Merge( UObject* InObject ) = 0;

	/** Begins a merge between the specified assets */
	virtual void Merge(UObject* BaseAsset, UObject* RemoteAsset, UObject* LocalAsset, const FOnMergeResolved& ResolutionCallback) = 0;

	/** Returns the categories that this asset type appears in. The return value is one or more flags from EAssetTypeCategories.  */
	virtual uint32 GetCategories() = 0;

	/** Returns the display name for that object. */
	virtual FString GetObjectDisplayName(UObject* Object) const = 0;

	/** Returns array of sub-menu names that this asset type is parented under in the Asset Creation Context Menu. */
	virtual const TArray<FText>& GetSubMenus() const = 0;

	/** @return True if we should force world-centric mode for newly-opened assets */
	virtual bool ShouldForceWorldCentric() = 0;

	/** Performs asset-specific diff on the supplied asset */
	virtual void PerformAssetDiff(UObject* OldAsset, UObject* NewAsset, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const = 0;

	/** Returns the thumbnail info for the specified asset, if it has one. */
	virtual class UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const = 0;

	/** Returns the default thumbnail type that should be rendered when rendering primitive shapes.  This does not need to be implemented if the asset does not render a primitive shape */
	UE_DEPRECATED(5.2, "When overriding GetThumbnailInfo, The default thumbnail primitive is now a property on USceneThumbnailInfoWithPrimitive.  The default only applies to this thumbnail type, so it has been relocated as an implementation detail of it.")
	virtual EThumbnailPrimType GetDefaultThumbnailPrimitiveType(UObject* Asset) const = 0;

	/** Optionally returns a custom widget to overlay on top of this assets' thumbnail */
	UE_DEPRECATED(5.5, "Convert the Action class over to the Definition system and use the new GetThumbnailActionOverlay, see UAssetDefinition for more information")
	virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const = 0;

	/** Returns additional tooltip information for the specified asset, if it has any (otherwise return the null widget) */
	virtual FText GetAssetDescription(const FAssetData& AssetData) const = 0;

	/** Returns whether the asset was imported from an external source */
	virtual bool IsImportedAsset() const = 0;

	/** Collects the resolved source paths for the imported assets */
	virtual void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const = 0;
	
	/** Collects the source file labels for the imported assets */
	virtual void GetSourceFileLabels(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFileLabels) const = 0;

	/** Builds the filter for this class*/
	virtual void BuildBackendFilter(struct FARFilter& InFilter) = 0;

	/** Optionally gets a class display name for this asset (otherwise, returns empty text (e.g. `FText::GetEmpty()`) */
	virtual FText GetDisplayNameFromAssetData(const FAssetData& AssetData) const = 0;

	/** Sets whether or not this asset type is a supported type for this editor session. */
	UE_DEPRECATED(5.2, "Use IAssetTools::IsAssetClassSupported")
	virtual void SetSupported(bool bInSupported) final { ensureMsgf(false, TEXT("Deprecated")); };

	/** Is this asset type supported in the current session? */
	UE_DEPRECATED(5.2, "Use IAssetTools::IsAssetClassSupported")
	virtual bool IsSupported() const final { ensureMsgf(false, TEXT("Deprecated")); return true; }
	
	/** Returns class path name as a package + class FName pair */
	//UE_DEPRECATED(5.2, "Just call GetSupportedClass()->GetClassPathName();")
	virtual FTopLevelAssetPath GetClassPathName() const = 0;

	/** Does this asset support edit or view methods? */
	virtual bool SupportsOpenedMethod(const EAssetTypeActivationOpenedMethod OpenedMethod) const = 0;

	/** Returns thumbnail brush unique for the given asset data.  Returning null falls back to class thumbnail brush. */
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const = 0;

	/** Returns icon brush unique for the given asset data.  Returning null falls back to class icon brush. */
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const = 0;
};

/**
HOW TO ADD ASSET EXTENSION MENUS

void FMyAssetEditorModule::StartupModule()
{
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FSynthesisEditorModule::RegisterMenus));
}

void FSynthesisEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);
}

void FSynthesisEditorModule::RegisterMenus()
{
	FToolMenuOwnerScoped MenuOwner(this);
	FAudioImpulseResponseExtension::RegisterMenus();
}


class FMyAssetExtension
{
public:
	static void RegisterMenus();
	static void ExecuteMenuAction(const struct FToolMenuContext& MenuContext);
};

void FMyAssetExtension::RegisterMenus()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.MyAsset");
	
	FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
	Section.AddDynamicEntry("GetAssetActions_MyAsset", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const TAttribute<FText> Label = LOCTEXT("MyAsset_DoTheThing", "Do the thing");
		const TAttribute<FText> ToolTip = LOCTEXT("MyAsset_DoTheThingTooltip", "Does the thing.");
		const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MyAsset");
		const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&FMyAssetExtension::ExecuteDoTheThing);

		InSection.AddMenuEntry("MyAsset_DoTheThing", Label, ToolTip, Icon, UIAction);
	}));
}

void FMyAssetExtension::ExecuteDoTheThing(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		for (UMyAsset* MyAsset : Context->LoadSelectedObjects<UMyAsset>())
		{
			// Do the actual thing...
		}
	}
}
*/
