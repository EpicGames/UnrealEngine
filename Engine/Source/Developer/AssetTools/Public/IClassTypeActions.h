// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/** ClassTypeActions provide actions and other information about asset types that host classes */
class IClassTypeActions : public TSharedFromThis<IClassTypeActions>
{
public:
	/** Virtual destructor */
	virtual ~IClassTypeActions()= default;

	/** Checks to see if the specified object is handled by this type. */
	virtual UClass* GetSupportedClass() const = 0;

	/** Optionally returns a custom widget to overlay on top of this assets' thumbnail */
	virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const struct FAssetData& AssetData) const = 0;
};
