// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

enum class ELocalizedAssetsOnDiskResult : uint8
{
	/** Getting localized assets on disk succeeded */
	Success,
	/** PackageNames could not be converted to FAssetData */
	PackageNamesError,
};

enum class ELocalizedAssetsInSCCResult : uint8
{
	/** Getting localized assets in Revision Control succeeded */
	Success,
	/** Revision Control is required but unavailable */
	RevisionControlNotAvailable,
};

enum class ELocalizedAssetsResult : uint8
{
	/** Getting localized assets in Revision Control succeeded */
	Success,
	/** PackageNames could not be converted to FAssetData */
	PackageNamesError,
	/** Revision Control is required but unavailable */
	RevisionControlNotAvailable,
};

enum class ELocalizedVariantsInclusion : uint8
{
	/** Include localized variants (and possibly source assets) to an operation */
	Include,
	/** Exclude localized variants (or related source asset) from an operation */
	Exclude,
	/** Cancel the current operation if the user do not want to choose between Including or Excluding localized variants. */
	Cancel
};

class ILocalizedAssetTools
{
public:
	virtual bool CanLocalize(const UClass* Class) const = 0;

	/** Get localized variants on disk from an asset name list */
	virtual ELocalizedAssetsOnDiskResult GetLocalizedVariantsOnDisk(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySource, TArray<FName>* OutPackagesNotFound = nullptr) const = 0;

	/** Get localized variants in Revision Control from an asset name list */
	virtual ELocalizedAssetsInSCCResult GetLocalizedVariantsInRevisionControl(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySource, TArray<FName>* OutPackagesNotFound = nullptr) const = 0;

	/** Get localized variants on disk then in Revision Control if Project Settings requires it */
	virtual ELocalizedAssetsResult GetLocalizedVariants(const TArray<FName>& InPackages, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySourceOnDisk, bool bAlsoCheckInRevisionControl, TMap<FName, TArray<FName>>& OutLocalizedVariantsBySourceInRevisionControl, TArray<FName>* OutPackagesNotFound = nullptr) const = 0;

	/** Open a dialog to required Revision Control to be configured */
	virtual void OpenRevisionControlRequiredDialog() const = 0;

	/** Open a dialog to required files in Revision Control to be on disk */
	virtual void OpenFilesInRevisionControlRequiredDialog(const TArray<FText>& FileList) const = 0;

	/** Open a dialog with a file list (localized variants) with a custom message */
	virtual void OpenLocalizedVariantsListMessageDialog(const FText& Header, const FText& Message, const TArray<FText>& FileList) const = 0;

	/** Open a dialog to ask to include/exclude localized variants from an operation */
	virtual ELocalizedVariantsInclusion OpenIncludeLocalizedVariantsListDialog(const TArray<FText>& FileList, ELocalizedVariantsInclusion RecommendedBehavior = ELocalizedVariantsInclusion::Include, bool bAllowOperationCanceling = true, ELocalizedVariantsInclusion UnattendedDefaultBehavior = ELocalizedVariantsInclusion::Exclude) const = 0;

	/** Get shared text to warn that some files need to be on disk (not only in Revision Control) */
	virtual const FText& GetFilesNeedToBeOnDiskWarningText() const = 0;

	/** Get shared text to warn that Revision Control needs to be available */
	virtual const FText& GetRevisionControlIsNotAvailableWarningText() const = 0;
};
