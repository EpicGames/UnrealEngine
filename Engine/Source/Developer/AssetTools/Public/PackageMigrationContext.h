// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Containers/UnrealString.h"
#include "UObject/GCObject.h"

#define UE_API ASSETTOOLS_API

class UPackage;
class UAssetToolsImpl;

namespace UE::AssetTools
{

namespace Private
{
	struct FPackageMigrationImpl;
}

struct FPackageMigrationContext : public FGCObject
{
	enum class EPackageMigrationStep;

	UE_API EPackageMigrationStep GetCurrentStep() const;

	// Notify the user that files where migrated. It should tell the user what was migrated and where it is in the destination.
	UE_API void AddSucessfullMigrationMessage(const FText& InMessage);

	// Notify the user that some asset may require some extra work to fully migrate
	UE_API void AddWarningMigrationMessage(const FText& InMessage);

	// Notify the user that some asset couldn't be migrated
	UE_API void AddErrorMigrationMessage(const FText& InMessage);

	struct FScopedMountPoint;
	UE_API const FScopedMountPoint& GetDestinationMountPoint() const;

	struct FMigrationPackageData;
	// Return the data associated to a package that is migrated
	UE_API const TArray<FMigrationPackageData>& GetMigrationPackagesData() const;

	// Return the array of non instanced package where moved because they were in the way of the migration
	UE_API const TArray<UPackage*>& GetMovedOutOfTheWayPackages() const;

	// Return the array of package dependencies that where not migrated because there where excluded by some automated filters
	UE_API const TArray<FString>& GetExcludedDependencies() const;

	enum class EPackageMigrationStep
	{
		// Called before the migration begin
		BeginMigration,
		// Called after the non external packages to migrate where processed
		PostAssetMigrationPackageDataCreated,
		// Called after the non external packages to migrate where processed
		PostExternalMigrationPackageDataCreated,
		// Called after the excluded depencies where processed
		PostExcludedDependenciesCreated,
		// Called after all the package that might be in the way for the migration where move
		InTheWayPackagesMoved,
		// Called after the instanced package where created with their load linker setup being already set to load the right file
		InstancedPackagesCreated,
		// Called after the instanced package where loaded
		InstancedPackagesLoaded,
		// Called after the instanced package were saved into the destination
		InstancedPackagesSaved,
		// Called after the instanced package were removed
		PostCleaningInstancedPackages,
		// Called after all the package that were in the way for the migration where restored
		InTheWayPackagesRestored,
		// Called at the end of the migration. After the in the way package where restored but before the log is processed
		EndMigration
	};

	// Utility to manage the temp mount point if needed and to help convert the paths during the migration
	struct FScopedMountPoint
	{
	public:
		FScopedMountPoint() = delete;
		FScopedMountPoint(const FScopedMountPoint&) = delete;
		FScopedMountPoint& operator=(const FScopedMountPoint&) = delete;

		FScopedMountPoint(FScopedMountPoint&&) = default;
		FScopedMountPoint& operator=(FScopedMountPoint&&) = default;

		UE_API const FString& GetRootPath() const;
		UE_API const FString& GetContentPath() const;

		UE_API FString GetMigratedPackageFilename(const FString& LongPackageName, const FStringView& InExtension) const;

		// Get name of package when moved under the destination mount point
		UE_API FString GetNewPackageNameForMigration(const FString& LongPackageName) const;

	private:
		friend UAssetToolsImpl;
		friend FPackageMigrationContext;
		friend UE::AssetTools::Private::FPackageMigrationImpl;

		UE_API FScopedMountPoint(FString&& InRootPath, FString&& InContentPath);
		UE_API ~FScopedMountPoint();

		FString RootPath;
		FString ContentPath;
		bool bHasMountedANewMountPoint = false;
	};

	// The data associated to an package that take part of the migration process.
	struct FMigrationPackageData
	{
		UE_API const FString& GetInstancedPackageName() const;
		UE_API const FString& GetOriginalPackageName() const;
		UE_API const FString& GetDestinationFilename() const;

		UE_API UPackage* GetInstancedPackage() const;

		// Tell the migration that package must be loaded
		bool bNeedInstancedLoad = false;

		// Tell the migration that the package must be migrated by save
		bool bNeedToBeSaveMigrated = false;

		UE_API FMigrationPackageData(const FString& InInstancedPackageName, const FString& InOriginalPackageName, const FString& InDestinationFilename);

	private:
		friend UAssetToolsImpl;
		friend FPackageMigrationContext;
		friend UE::AssetTools::Private::FPackageMigrationImpl;

		FString InstancedPackageName;
		FString OriginalPackageName;

		FString DestinationFilename;
		TObjectPtr<UPackage> InstancedPackage = nullptr;
	};

	FPackageMigrationContext(const FPackageMigrationContext&) = delete;
	FPackageMigrationContext(FPackageMigrationContext&&) = delete;
	FPackageMigrationContext operator=(const FPackageMigrationContext&) = delete;
	FPackageMigrationContext operator=(FPackageMigrationContext&&) = delete;


	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	UE_API virtual FString GetReferencerName() const override;

private:

	friend UAssetToolsImpl;
	friend UE::AssetTools::Private::FPackageMigrationImpl;

	// Move a package out of the way for the duration of the migration
	UE_API void MoveInTheWayPackage(UPackage* Package);

	UE_API FPackageMigrationContext(FScopedMountPoint&& InMountPoint);

	~FPackageMigrationContext() = default;

	// Helper to move existing package out of the way during the migration
	struct FScopedTemporalyMovedPackage
	{
	public:
		UE_API FScopedTemporalyMovedPackage(UPackage* InPackageToMove);

		UE_API ~FScopedTemporalyMovedPackage();

		TObjectPtr<UPackage> PackageToMove = nullptr;
		FString OriginalName;
	};

	EPackageMigrationStep CurrentStep;

	// Message of the for the successfully migrated asset
	TArray<FText> MigratedPackageMessages;

	// Message to warn the user that some extra step may be required from him to complete the migration
	TArray<FText> WarningMessage;

	// General Error messages of the migration
	TArray<FText> ErrorMessages;

	FScopedMountPoint DestinationMountPoint;

	// Helper for the package that were moved because they were in the way of migrated packages
	TArray<FScopedTemporalyMovedPackage> TemporalyMovedPackages;

	// The package that have been moved during the migration
	TArray<TObjectPtr<UPackage>> PackagesThatWhereMoved;

	// The data associated to the packages that take part of the migration process.
	TArray<FMigrationPackageData> MigrationPackagesData;

	// The excluded dependencies of the asset that are being migrated (these don't include the asset a user might have exclude voluntary from the migration)
	TArray<FString> ExcludedDependencies;
};

}

#undef UE_API
